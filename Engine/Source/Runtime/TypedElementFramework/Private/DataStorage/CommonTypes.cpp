// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataStorage/CommonTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonTypes)

namespace UE
{
	namespace Editor::DataStorage
	{
		const FName& FValueTag::GetName() const
		{
			return Name;
		}
		
		FValueTag::FValueTag(const FName& InTypeName)
			: Name(InTypeName)
		{}
		
		uint32 GetTypeHash(const FValueTag& InName)
		{
			return GetTypeHash(InName.Name);
		}

		uint32 GetTypeHash(const FDynamicColumnDescription& Descriptor)
		{
			return HashCombineFast(PointerHash(Descriptor.TemplateType), GetTypeHash(Descriptor.Identifier));
		}
	}
}
