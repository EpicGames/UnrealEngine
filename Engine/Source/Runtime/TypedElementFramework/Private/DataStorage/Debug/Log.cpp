// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataStorage/Debug/Log.h"

DEFINE_LOG_CATEGORY(LogEditorDataStorage)