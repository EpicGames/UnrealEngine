// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataStorage/Features.h"

namespace UE::Editor::DataStorage
{
	const FName StorageFeatureName = "EditorDataStorage";
	const FName CompatibilityFeatureName = "EditorDataStorageCompatibility";
	const FName UiFeatureName = "EditorDataStorageUi";
	
	FSimpleMulticastDelegate& OnEditorDataStorageFeaturesEnabled()
	{
		static FSimpleMulticastDelegate OnTedsFeaturesEnabled;
		return OnTedsFeaturesEnabled;
	}

	bool AreEditorDataStorageFeaturesEnabled()
	{
		return IModularFeatures::Get().IsModularFeatureAvailable(StorageFeatureName)
			&& IModularFeatures::Get().IsModularFeatureAvailable(CompatibilityFeatureName)
			&& IModularFeatures::Get().IsModularFeatureAvailable(UiFeatureName);
	}
} // namespace UE::Editor::DataStorage
