// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataStorage/MapKey.h"

namespace UE::Editor::DataStorage
{
	struct FHasher
	{
		uint64 operator()(FEmptyVariantState)	{ return 0; }
		uint64 operator()(const void* Value)	{ return reinterpret_cast<uint64>(Value); }
		uint64 operator()(const UObject* Value) { return reinterpret_cast<uint64>(Value); }
		uint64 operator()(int64 Value)			{ return static_cast<uint64>(Value); }
		uint64 operator()(uint64 Value)			{ return static_cast<uint64>(Value); }
		uint64 operator()(float Value)			{ return static_cast<uint64>(Value); }
		uint64 operator()(double Value)			{ return static_cast<uint64>(Value); }
		uint64 operator()(const FString& Value)
		{
			return CityHash64(reinterpret_cast<const char*>(*Value), Value.Len() * sizeof(**Value));
		}
		uint64 operator()(const FStringView& Value)
		{
			return CityHash64(reinterpret_cast<const char*>(Value.GetData()), static_cast<uint32>(Value.NumBytes()));
		}
		uint64 operator()(const FName& Value)	{ return Value.ToUnstableInt(); }
		uint64 operator()(const FName* Value)	{ return Value->ToUnstableInt(); }
		uint64 operator()(const FSoftObjectPath* Value)
		{
			FTopLevelAssetPath TopLevelAssetPath = Value->GetAssetPath();
			uint64 Hash = TopLevelAssetPath.GetPackageName().ToUnstableInt();
			Hash = CityHash128to64({ Hash, TopLevelAssetPath.GetAssetName().ToUnstableInt() });
			const FString& SubPath = Value->GetSubPathString();
			return CityHash64WithSeed(reinterpret_cast<const char*>(*SubPath), SubPath.Len() * sizeof(**SubPath), Hash);
		}
		uint64 operator()(const TUniquePtr<FSoftObjectPath>& Value) { return operator()(Value.Get()); }
		uint64 operator()(const FKeyIssue& Value)
		{
			return operator()(Value.Description);
		}
		uint64 operator()(const FKeyIssueView& Value)
		{
			return operator()(Value.Description);
		}
	};

	struct FKeyToKeyViewConverter
	{
		template<typename T>
		FMapKeyView::KeyViewType operator()(const T& Value)
		{
			return FMapKeyView::KeyViewType(TInPlaceType<T>(), Value);
		}

		FMapKeyView::KeyViewType operator()(const FKeyIssue& Value)
		{
			return FMapKeyView::KeyViewType(TInPlaceType<FKeyIssueView>(), Value);
		}

		FMapKeyView::KeyViewType operator()(const FString& Value)
		{
			return FMapKeyView::KeyViewType(TInPlaceType<FStringView>(), Value);
		}

		FMapKeyView::KeyViewType operator()(const FName& Value)
		{
			return FMapKeyView::KeyViewType(TInPlaceType<const FName*>(), &Value);
		}

		FMapKeyView::KeyViewType operator()(const TUniquePtr<FSoftObjectPath>& Value)
		{
			return FMapKeyView::KeyViewType(TInPlaceType<const FSoftObjectPath*>(), Value.Get());
		}
	};

	struct FKeyViewToKeyConverter
	{
		FMapKey operator()(const FEmptyVariantState& Value)
		{
			return FMapKey();
		}

		template<typename T>
		FMapKey operator()(const T& Value)
		{
			return FMapKey(Value);
		}

		FMapKey operator()(const FKeyIssueView& Value)
		{
			return FMapKey(FKeyIssue(Value.Description));
		}

		FMapKey operator()(const FStringView& Value)
		{
			return FMapKey(FString(Value));
		}

		FMapKey operator()(const FName* Value)
		{
			return FMapKey(*Value);
		}

		FMapKey operator()(const FSoftObjectPath* Value)
		{
			return FMapKey(*Value);
		}
	};

	struct FKeyMove
	{
		FMapKey::KeyType operator()(FEmptyVariantState&)
		{
			return FMapKey::KeyType(TInPlaceType<FEmptyVariantState>());
		}

		template<typename T>
		FMapKey::KeyType operator()(T&& Other)
		{
			return FMapKey::KeyType(TInPlaceType<T>(), Forward<T>(Other));
		}
	};

	struct FKeyCopy
	{
		FMapKey::KeyType operator()(const FEmptyVariantState&)
		{
			return FMapKey::KeyType(TInPlaceType<FEmptyVariantState>());
		}

		template<typename T>
		FMapKey::KeyType operator()(const T& Other)
		{
			return FMapKey::KeyType(TInPlaceType<T>(), Other);
		}

		FMapKey::KeyType operator()(const TUniquePtr<FSoftObjectPath>& Other)
		{
			return FMapKey::KeyType(TInPlaceType<TUniquePtr<FSoftObjectPath>>(), MakeUnique<FSoftObjectPath>(*Other));
		}
	};

	struct FKeyComparer
	{
		const FMapKey::KeyType& Key;
		explicit FKeyComparer(const FMapKey::KeyType& Key) : Key(Key) {}

		bool operator()(FEmptyVariantState)
		{
			return Key.IsType<FEmptyVariantState>();
		}

		template<typename T>
		bool operator()(const T& Other)
		{
			const auto* Local = Key.TryGet<T>();
			return Local ? *Local == Other : false;
		}

		bool operator()(const TUniquePtr<FSoftObjectPath>& Other)
		{
			if (const TUniquePtr<FSoftObjectPath>* Local = Key.TryGet<TUniquePtr<FSoftObjectPath>>())
			{
				if (Local->IsValid() && Other.IsValid())
				{
					return *(Local->Get()) == *Other;
				}
			}
			return false;
		}
	};

	struct FKeyViewComparer
	{
		const FMapKeyView::KeyViewType& Key;
		explicit FKeyViewComparer(const FMapKeyView::KeyViewType& Key) : Key(Key) {}

		bool operator()(FEmptyVariantState)
		{
			return Key.IsType<FEmptyVariantState>();
		}

		template<typename T>
		bool operator()(const T& Other)
		{
			const T* Local = Key.TryGet<T>();
			return Local ? *Local == Other : false;
		}

		bool operator()(const FKeyIssue& Other)
		{
			const FKeyIssueView* Local = Key.TryGet<FKeyIssueView>();
			return Local ? *Local == Other : false;
		}

		bool operator()(const FKeyIssueView& Other)
		{
			const FKeyIssueView* Local = Key.TryGet<FKeyIssueView>();
			return Local ? *Local == Other : false;
		}

		bool operator()(const FString& Other)
		{
			const FStringView* Local = Key.TryGet<FStringView>();
			return Local ? *Local == Other : false;
		}

		bool operator()(const FStringView& Other)
		{
			const FStringView* Local = Key.TryGet<FStringView>();
			return Local ? *Local == Other : false;
		}

		bool operator()(const FName& Other)
		{
			const FName* const* Local = Key.TryGet<const FName*>();
			return Local ? **Local == Other : false;
		}

		bool operator()(const FName* Other)
		{
			const FName* const* Local = Key.TryGet<const FName*>();
			return Local ? **Local == *Other : false;
		}

		bool operator()(const TUniquePtr<FSoftObjectPath>& Other)
		{
			if (Other.IsValid())
			{
				const FSoftObjectPath* const* Local = Key.TryGet<const FSoftObjectPath*>();
				return Local ? **Local == *Other : false;
			}
			return false;
		}

		bool operator()(const FSoftObjectPath* Other)
		{
			const FSoftObjectPath* const* Local = Key.TryGet<const FSoftObjectPath*>();
			return Local ? **Local == *Other : false;
		}
	};

	struct FToString
	{
		FString operator()(FEmptyVariantState) { return TEXT("Empty"); }
		FString operator()(const FKeyIssue& Value) { return FString::Printf(TEXT("Issue: '%s'"), *Value.Description); }
		FString operator()(const FKeyIssueView& Value)
		{ 
			return Value.Description.IsEmpty()
				? FString(TEXT("Issue: <no description>"))
				: FString::Printf(TEXT("Issue: '%.*s'"), Value.Description.Len(), Value.Description.GetData());
		}
		FString operator()(const void* Value) { return FString::Printf(TEXT("Pointer(0x%p)"), Value); }
		FString operator()(const UObject* Value) { return FString::Printf(TEXT("UObject(0x%p)"), Value); }
		FString operator()(int64 Value) { return FString::Printf(TEXT("%lld"), Value); }
		FString operator()(uint64 Value) { return FString::Printf(TEXT("%llu"), Value); }
		FString operator()(float Value) { return FString::SanitizeFloat(Value);; }
		FString operator()(double Value) { return FString::SanitizeFloat(Value);; }
		FString operator()(const FString& Value) { return Value; }
		FString operator()(const FStringView& Value) { return FString(Value); }
		FString operator()(const FName& Value) { return Value.ToString(); }
		FString operator()(const FName* Value) { return Value->ToString(); }
		FString operator()(const FSoftObjectPath* Value) { return Value->ToString(); }
		FString operator()(const TUniquePtr<FSoftObjectPath>& Value) { return Value ? Value->ToString() : TEXT("Empty path"); }
	};

	//
	// FKeyIssue
	//

	FKeyIssue::FKeyIssue(const FString::ElementType* Description)
		: Description(Description)
	{
	}

	FKeyIssue::FKeyIssue(FString Description)
		: Description(MoveTemp(Description))
	{
	}
	
	FKeyIssue::FKeyIssue(FStringView Description)
		: Description(Description)
	{
	}

	FKeyIssue::FKeyIssue(const FKeyIssueView& Description)
		: Description(Description.Description)
	{
	}

	bool FKeyIssue::operator==(const FKeyIssueView& Rhs) const
	{
		return Description == Rhs.Description;
	}

	bool FKeyIssue::operator!=(const FKeyIssueView& Rhs) const
	{
		return Description != Rhs.Description;
	}


	//
	// FKeyIssueView
	//

	FKeyIssueView::FKeyIssueView(FStringView Description)
		: Description(Description)
	{
	}

	FKeyIssueView::FKeyIssueView(const FKeyIssue& Description)
		: Description(Description.Description)
	{
	}

	bool FKeyIssueView::operator==(const FKeyIssue& Rhs) const
	{
		return Description == Rhs.Description;
	}

	bool FKeyIssueView::operator!=(const FKeyIssue& Rhs) const
	{
		return Description != Rhs.Description;
	}



	//
	// FMapKey
	//

	FMapKey::FMapKey(FMapKey&& Rhs)
		: Key(Visit(FKeyMove(), MoveTemp(Rhs.Key)))
	{
		Rhs.Clear();
	}

	FMapKey& FMapKey::operator=(FMapKey&& Rhs)
	{
		Key = Visit(FKeyMove(), MoveTemp(Rhs.Key));
		Rhs.Clear();
		return *this;
	}

	FMapKey::FMapKey(const FMapKey& Key)
		: Key(Visit(FKeyCopy(), Key.Key))
	{
	}

	FMapKey::FMapKey(FKeyIssue Key)
		: Key(TInPlaceType<FKeyIssue>(), MoveTemp(Key))
	{
	}

	FMapKey::FMapKey(const void* Key)
		: Key(TInPlaceType<const void*>(), Key)
	{
	}

	FMapKey::FMapKey(const UObject* Key)
		: Key(TInPlaceType<const UObject*>(), Key)
	{
	}

	FMapKey::FMapKey(int64 Key)
		: Key(TInPlaceType<int64>(), Key)
	{
	}

	FMapKey::FMapKey(uint64 Key)
		: Key(TInPlaceType<uint64>(), Key)
	{
	}

	FMapKey::FMapKey(float Key)
		: Key(TInPlaceType<float>(), Key)
	{
	}

	FMapKey::FMapKey(double Key)
		: Key(TInPlaceType<double>(), Key)
	{
	}

	FMapKey::FMapKey(FString Key)
		: Key(TInPlaceType<FString>(), MoveTemp(Key))
	{
	}
	
	FMapKey::FMapKey(FName Key)
		: Key(TInPlaceType<FName>(), MoveTemp(Key))
	{
	}

	FMapKey::FMapKey(FSoftObjectPath Key)
		: Key(TInPlaceType<TUniquePtr<FSoftObjectPath>>(), MakeUnique<FSoftObjectPath>(MoveTemp(Key)))
	{
	}

	uint64 FMapKey::CalculateHash() const
	{
		return Visit(FHasher(), Key);
	}
	
	bool FMapKey::IsSet() const
	{
		return !Key.IsType<FEmptyVariantState>();
	}

	void FMapKey::Clear()
	{
		Key.Emplace<FEmptyVariantState>();
	}

	FString FMapKey::ToString() const
	{
		return Visit(FToString(), Key);
	}

	FMapKey& FMapKey::operator=(const FMapKey& Rhs)
	{
		Key = Visit(FKeyCopy(), Rhs.Key);
		return *this;
	}

	bool FMapKey::operator==(const FMapKey& Rhs) const
	{
		return Visit(FKeyComparer(Key), Rhs.Key);
	}

	bool FMapKey::operator==(const FMapKeyView& Rhs) const
	{
		return Rhs == *this;
	}

	bool FMapKey::operator!=(const FMapKey& Rhs) const
	{
		return !(*this == Rhs);
	}

	bool FMapKey::operator!=(const FMapKeyView& Rhs) const
	{
		return !(*this == Rhs);
	}



	//
	// FMapKeyView
	//

	FMapKeyView::FMapKeyView(const FMapKey& InKey)
		: Key(Visit(FKeyToKeyViewConverter(), InKey.Key))
	{
	}

	FMapKeyView::FMapKeyView(const FKeyIssue& Key)
		: Key(TInPlaceType<FKeyIssueView>(), Key)
	{
	}

	FMapKeyView::FMapKeyView(FKeyIssueView Key)
		: Key(TInPlaceType<FKeyIssueView>(), Key)
	{
	}

	FMapKeyView::FMapKeyView(const void* Key)
		: Key(TInPlaceType<const void*>(), Key)
	{
	}

	FMapKeyView::FMapKeyView(const UObject* Key)
		: Key(TInPlaceType<const UObject*>(), Key)
	{
	}

	FMapKeyView::FMapKeyView(int64 Key)
		: Key(TInPlaceType<int64>(), Key)
	{
	}

	FMapKeyView::FMapKeyView(uint64 Key)
		: Key(TInPlaceType<uint64>(), Key)
	{
	}

	FMapKeyView::FMapKeyView(float Key)
		: Key(TInPlaceType<float>(), Key)
	{
	}

	FMapKeyView::FMapKeyView(double Key)
		: Key(TInPlaceType<double>(), Key)
	{
	}

	FMapKeyView::FMapKeyView(const FString& Key)
		: Key(TInPlaceType<FStringView>(), Key)
	{
	}
	
	FMapKeyView::FMapKeyView(FStringView Key)
		: Key(TInPlaceType<FStringView>(), Key)
	{
	}

	FMapKeyView::FMapKeyView(const FName& Key)
		: Key(TInPlaceType<const FName*>(), &Key)
	{
	}

	FMapKeyView::FMapKeyView(const FSoftObjectPath& Key)
		: Key(TInPlaceType<const FSoftObjectPath*>(), &Key)
	{
	}

	uint64 FMapKeyView::CalculateHash() const
	{
		return Visit(FHasher(), Key);
	}

	FString FMapKeyView::ToString() const
	{
		return Visit(FToString(), Key);
	}

	FMapKeyView& FMapKeyView::operator=(const FMapKey& InKey)
	{
		Key = Visit(FKeyToKeyViewConverter(), InKey.Key);
		return *this;
	}

	FMapKey FMapKeyView::CreateKey() const
	{
		return Visit(FKeyViewToKeyConverter(), Key);
	}

	bool FMapKeyView::operator==(const FMapKey& Rhs) const
	{
		return Visit(FKeyViewComparer(Key), Rhs.Key);
	}

	bool FMapKeyView::operator==(const FMapKeyView& Rhs) const
	{
		return Visit(FKeyViewComparer(Key), Rhs.Key);
	}

	bool FMapKeyView::operator!=(const FMapKey& Rhs) const
	{
		return !(*this == Rhs);
	}

	bool FMapKeyView::operator!=(const FMapKeyView& Rhs) const
	{
		return !(*this == Rhs);
	}
}
// namespace UE::Editor::DataStorage
