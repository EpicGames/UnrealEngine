// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataStorage/Queries/Conditions.h"

#include "RHIStats.h"
#include "Elements/Framework/TypedElementColumnUtils.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "UObject/Class.h"

namespace UE::Editor::DataStorage::Queries
{
	FEditorStorageQueryConditionCompileContext::FEditorStorageQueryConditionCompileContext(ICoreProvider* InDataStorage)
		: DataStorage(InDataStorage)
	{
	}

	const UScriptStruct* FEditorStorageQueryConditionCompileContext::GenerateDynamicColumn(
		const FDynamicColumnDescription& Description) const
	{
		return DataStorage->GenerateDynamicColumn(Description);
	}

	FConditions::FConditions()
	{
		Identifiers.Init(NAME_None, MaxColumnCount);
		Columns.Init(nullptr, MaxColumnCount);
		Tokens.Init(Token::None, MaxTokenCount);
	}

	FConditions::FConditions(FColumnBase Column)
		: ColumnCount(1)
	{
		Identifiers.Init(NAME_None, MaxColumnCount);
		Columns.Init(nullptr, MaxColumnCount);
		Tokens.Init(Token::None, MaxTokenCount);
		
		Columns[0] = Column.TypeInfo;
		Identifiers[0] = Column.Identifier;
	}

	void FConditions::AppendToString(FString& Output) const
	{
		checkf(bIsCompiled, TEXT("Query Conditions must call Compile() before you can use them"));
		
		if (TokenCount > 0)
		{
			Output += "{ ";
			uint8_t ColumnIndex = 0;
			if (Tokens[0] != Token::ScopeOpen)
			{
				ColumnIndex = 1;
				AppendName(Output, Columns[0]);
			}
			for (uint8_t TokenIndex = 0; TokenIndex < TokenCount; ++TokenIndex)
			{
				switch (Tokens[TokenIndex])
				{
				case Token::And:
					Output += " && ";
					if (!EntersScopeNext(TokenIndex))
					{
						AppendName(Output, Columns[ColumnIndex++]);
					}
					break;
				case Token::Or:
					Output += " || ";
					if (!EntersScopeNext(TokenIndex))
					{
						AppendName(Output, Columns[ColumnIndex++]);
					}
					break;
				case Token::ScopeOpen:
					Output += "( ";
					if (!EntersScopeNext(TokenIndex))
					{
						AppendName(Output, Columns[ColumnIndex++]);
					}
					break;
				case Token::ScopeClose:
					Output += " )";
					break;
				default:
					checkf(false, TEXT("Invalid query token"));
					break;
				}
			}
			Output += " }";
		}
	}

	template <typename AvailableColumnType, typename ProjectionFunctionType>
	bool FConditions::VerifyWithDynamicColumn(TConstArrayView<AvailableColumnType> AvailableColumns, uint64& Matches, ProjectionFunctionType Projection) const
	{
		// Upfront process AvailableColumns since AvailableColumns may be searched through more than once
		TBitArray<TInlineAllocator<512>> DerivedFromDynamicTemplate;
#if WITH_EDITORONLY_DATA
		DerivedFromDynamicTemplate.SetNum(AvailableColumns.Num(), false);
		for (int32 AvailableColumnIndex = 0; AvailableColumnIndex < AvailableColumns.Num(); ++AvailableColumnIndex)
		{
			const TWeakObjectPtr<const UScriptStruct>& AvailableColumn = Projection(AvailableColumns[AvailableColumnIndex]);
			if (const UScriptStruct* AvailableColumnStruct = AvailableColumn.Get())
			{
				DerivedFromDynamicTemplate[AvailableColumnIndex] = ColumnUtils::IsDerivedFromDynamicTemplate(AvailableColumnStruct);
			}
		}
#endif
		
		bool Result = VerifyBootstrap(
			[this, &Matches, &AvailableColumns, &DerivedFromDynamicTemplate, &Projection](uint8_t ColumnIndex, TWeakObjectPtr<const UScriptStruct> Column)
			{
				bool bContains = false;
				// Looping through each column in the condition, if the column is a template then determine if any of the available
				// columns that are being matched against are derived from that template
				if ((ColumnFlags[ColumnIndex] & EColumnFlags::DynamicColumnTemplate) == EColumnFlags::DynamicColumnTemplate)
				{
					for (int32 AvailableColumnIndex = 0; AvailableColumnIndex < AvailableColumns.Num(); ++AvailableColumnIndex)
					{
						// Only check if this available column is derived from our dynamic template if we know it derives from any DynamicTemplate
						if (DerivedFromDynamicTemplate[AvailableColumnIndex])
						{
							if (Projection(AvailableColumns[AvailableColumnIndex])->IsChildOf(Column.Get()))
							{
								Matches |= (uint64)1 << ColumnIndex;
								bContains = true;
							}
						}
						else
						{
							if (Projection(AvailableColumns[AvailableColumnIndex]) == Column.Get())
							{
								Matches |= (uint64)1 << ColumnIndex;
								bContains = true;
							}
						}
					}
				}
				else
				{
					for (int32 AvailableColumnIndex = 0; AvailableColumnIndex < AvailableColumns.Num(); ++AvailableColumnIndex)
					{
						if (Projection(AvailableColumns[AvailableColumnIndex]) == Column.Get())
						{
							Matches |= (uint64)1 << ColumnIndex;
							bContains = true;
						}
					}
					//auto FindPredicate = ;
					if ( AvailableColumns.FindByPredicate([&Column, &Projection](const AvailableColumnType& Entry) { return Projection(Entry) == Column.Get(); }) != nullptr)
					{
						Matches |= (uint64)1 << ColumnIndex;
						bContains = true;
					}
				}
				return bContains;
			});

		return Result;
	}
	
	bool FConditions::Verify(TConstArrayView<FColumnBase> AvailableColumns) const
	{
		checkf(bIsCompiled, TEXT("Query Conditions must call Compile() before you can use them"));

		bool Result;
		if (!UsesDynamicTemplates())
		{
			Result = VerifyBootstrap(
				[&AvailableColumns](uint8_t ColumnIndex, TWeakObjectPtr<const UScriptStruct> Column)
				{
					for (const FColumnBase& Target : AvailableColumns)
					{
						if (Target.TypeInfo == Column)
						{
							return true;
						}
					}
					return false;
				});
		}
		else
		{
			uint64 Matches = 0;
			Result = VerifyWithDynamicColumn(AvailableColumns, Matches, [](const FColumnBase& Column) { return Column.TypeInfo; });
		}
		
		return Result;
	}

	bool FConditions::Verify(TArray<TWeakObjectPtr<const UScriptStruct>>& MatchedColumns, TConstArrayView<FColumnBase> AvailableColumns,
		bool AvailableColumnsAreSorted) const
	{
		static_assert(MaxColumnCount < 64, "Query conditions use a bit mask to locate matches. As a result MaxColumnCount can be larger than 64.");

		checkf(bIsCompiled, TEXT("Query Conditions must call Compile() before you can use them"));
		
		uint64 Matches = 0;
		bool Result;
		if (!UsesDynamicTemplates())
		{
			Result = AvailableColumnsAreSorted
				? VerifyBootstrap(
					[&Matches, &AvailableColumns](uint8_t ColumnIndex, TWeakObjectPtr<const UScriptStruct> Column)
					{
						auto Projection = [](const FColumnBase& ColumnBase)
						{
							return ColumnBase.TypeInfo.Get();
						};
						if (Algo::BinarySearchBy(AvailableColumns, Column.Get(), Projection) != INDEX_NONE)
						{
							Matches |= (uint64)1 << ColumnIndex;
							return true;
						}
						return false;
					})
				: VerifyBootstrap(
					[&Matches, &AvailableColumns](uint8_t ColumnIndex, TWeakObjectPtr<const UScriptStruct> Column)
					{
						for (const FColumnBase& Target : AvailableColumns)
						{
							if (Target.TypeInfo == Column)
							{
								Matches |= (uint64)1 << ColumnIndex;
								return true;
							}
						}
						return false;
					});
		}
		else
		{
			Result = VerifyWithDynamicColumn(AvailableColumns, Matches, [](const FColumnBase& ColumnBase) { return ColumnBase.TypeInfo; });
		}
		if (Result)
		{
			ConvertColumnBitToArray(MatchedColumns, Matches);
		}
		return Result;
	}

	bool FConditions::Verify(TConstArrayView<TWeakObjectPtr<const UScriptStruct>> AvailableColumns,
		bool AvailableColumnsAreSorted) const
	{
		checkf(bIsCompiled, TEXT("Query Conditions must call Compile() before you can use them"));

		bool Result;
		if (!UsesDynamicTemplates())
		{
			Result = AvailableColumnsAreSorted
				? VerifyBootstrap(
					[&AvailableColumns](uint8_t ColumnIndex, TWeakObjectPtr<const UScriptStruct> Column)
					{
						auto Projection = [](TWeakObjectPtr<const UScriptStruct> ColumnBase)
						{
							return ColumnBase.Get();
						};
						return Algo::BinarySearchBy(AvailableColumns, Column.Get(), Projection) != INDEX_NONE;
					})
				: VerifyBootstrap(
					[&AvailableColumns](uint8_t ColumnIndex, TWeakObjectPtr<const UScriptStruct> Column)
					{
						return AvailableColumns.Find(Column) != INDEX_NONE;
					});
		}
		else
		{
			uint64 Matches = 0;
			Result = VerifyWithDynamicColumn(AvailableColumns, Matches, [](TWeakObjectPtr<const UScriptStruct> ColumnType) { return ColumnType; });
		}
		return Result;
	}

	bool FConditions::Verify(TArray<TWeakObjectPtr<const UScriptStruct>>& MatchedColumns,
		TConstArrayView<TWeakObjectPtr<const UScriptStruct>> AvailableColumns,
		bool AvailableColumnsAreSorted) const
	{
		static_assert(MaxColumnCount < 64, "Query conditions use a bit mask to locate matches. As a result MaxColumnCount cannot be larger than 64.");
		
		checkf(bIsCompiled, TEXT("Query Conditions must call Compile() before you can use them"));
		
		uint64 Matches = 0;
		bool Result;
		if (!UsesDynamicTemplates())
		{
			Result = AvailableColumnsAreSorted
				? VerifyBootstrap(
					[&Matches, &AvailableColumns](uint8_t ColumnIndex, TWeakObjectPtr<const UScriptStruct> Column)
					{
						auto Projection = [](TWeakObjectPtr<const UScriptStruct> ColumnBase)
						{
							return ColumnBase.Get();
						};
						if (Algo::BinarySearchBy(AvailableColumns, Column.Get(), Projection) != INDEX_NONE)
						{
							Matches |= (uint64)1 << ColumnIndex;
							return true;
						}
						return false;
					})
				: VerifyBootstrap(
					[&Matches, &AvailableColumns](uint8_t ColumnIndex, TWeakObjectPtr<const UScriptStruct> Column)
					{
						if ( AvailableColumns.Find(Column) != INDEX_NONE)
						{
							Matches |= (uint64)1 << ColumnIndex;
							return true;
						}
						return false;
					});
		}
		else
		{
			Result = VerifyWithDynamicColumn(AvailableColumns, Matches, [](TWeakObjectPtr<const UScriptStruct> ColumnType) { return ColumnType; });
		}
		if (Result)
		{
			ConvertColumnBitToArray(MatchedColumns, Matches);
		}
		return Result;
	}

	bool FConditions::Verify(TSet<TWeakObjectPtr<const UScriptStruct>> AvailableColumns) const
	{
		checkf(bIsCompiled, TEXT("Query Conditions must call Compile() before you can use them"));
		
		return VerifyBootstrap(
			[&AvailableColumns](uint8_t ColumnIndex, TWeakObjectPtr<const UScriptStruct> Column)
			{
				return AvailableColumns.Find(Column) != nullptr;
			});
	}

	bool FConditions::Verify(ContainsCallback Callback) const
	{
		checkf(bIsCompiled, TEXT("Query Conditions must call Compile() before you can use them"));
		
		return VerifyBootstrap(Callback);
	}

	uint8_t FConditions::MinimumColumnMatchRequired() const
	{
		if (TokenCount > 0)
		{
			uint8_t Front = 0;
			return MinimumColumnMatchRequiredRange(Front);
		}
		else if (ColumnCount == 1)
		{
			return 1;
		}
		else
		{
			return 0;
		}
	}

	TConstArrayView<TWeakObjectPtr<const UScriptStruct>> FConditions::GetColumns() const
	{
		checkf(bIsCompiled, TEXT("Query Conditions must call Compile() before you can use them"));
		
		return TConstArrayView<TWeakObjectPtr<const UScriptStruct>>(Columns.GetData(), ColumnCount);
	}

	bool FConditions::IsEmpty() const
	{
		return ColumnCount == 0;
	}

	FConditions& FConditions::Compile(const IQueryConditionCompileContext& CompileContext)
	{
		if(bIsCompiled)
		{
			return *this;
		}
		
		ColumnFlags.SetNumZeroed(ColumnCount);

		// Resolve all dynamic columns by generating their UScriptStruct
		for (uint8_t ColumnIndex = 0; ColumnIndex < ColumnCount; ++ColumnIndex)
		{
			if(Identifiers[ColumnIndex] != NAME_None)
			{
				const UScriptStruct* ResolvedDynamicColumn = CompileContext.GenerateDynamicColumn(
					FDynamicColumnDescription
						{
							.TemplateType = Columns[ColumnIndex].Get(),
							.Identifier = Identifiers[ColumnIndex]
						});
				
				Columns[ColumnIndex] = ResolvedDynamicColumn;
			}
			else
			{
#if WITH_EDITORONLY_DATA
				bool bIsDynamicTemplate = ColumnUtils::IsDynamicTemplate(Columns[ColumnIndex].Get());
				if (bIsDynamicTemplate)
				{
					ColumnFlags[ColumnIndex] |= EColumnFlags::DynamicColumnTemplate;
				}
#endif
			}
		}

		bIsCompiled = true;

		return *this;
	}

	bool FConditions::IsCompiled() const
	{
		return bIsCompiled;
	}

	void FConditions::AppendName(FString& Output, TWeakObjectPtr<const UScriptStruct> TypeInfo) const
	{
#if WITH_EDITORONLY_DATA
		static FName DisplayNameName("DisplayName");
		if (const FString* Name = TypeInfo->FindMetaData(DisplayNameName))
		{
			Output += *Name;
		}
#else
		Output += TEXT("<Unavailable>");
#endif
	}

	bool FConditions::EntersScopeNext(uint8_t Index) const
	{
		return Index < (TokenCount - 1) && Tokens[Index + 1] == Token::ScopeOpen;
	}

	bool FConditions::EntersScope(uint8_t Index) const
	{
		return Tokens[Index] == Token::ScopeOpen;
	}

	bool FConditions::Contains(TWeakObjectPtr<const UScriptStruct> ColumnType, const TArray<FColumnBase>& AvailableColumns) const
	{
		for (const FColumnBase& Column : AvailableColumns)
		{
			if (ColumnType == Column.TypeInfo)
			{
				return true;
			}
		}
		return false;
	}

	bool FConditions::VerifyBootstrap(ContainsCallback Contains) const
	{
		if (TokenCount > 0)
		{
			uint8_t TokenIndex = 0;
			uint8_t ColumnIndex = 0;
			return VerifyRange(TokenIndex, ColumnIndex, Forward<ContainsCallback>(Contains));
		}
		else if (ColumnCount == 1)
		{
			return Contains(0, Columns[0]);
		}
		// If there are no columns in the condition, everything passes it
		else
		{
			return true;
		}
	}

	bool FConditions::VerifyRange(uint8_t& TokenIndex, uint8_t& ColumnIndex, ContainsCallback Contains) const
	{
		auto Init = [&]()
		{
			if (EntersScope(TokenIndex))
			{
				return VerifyRange(++TokenIndex, ColumnIndex, Contains);
			}
			else
			{
				bool Result = Contains(ColumnIndex, Columns[ColumnIndex]);
				ColumnIndex++;
				return Result;
			}
		};
		bool Result = Init();

		while (TokenIndex < TokenCount)
		{
			Token T = Tokens[TokenIndex];
			++TokenIndex;
			switch (T)
			{
			case Token::And:
				if (EntersScope(TokenIndex))
				{
					Result = VerifyRange(++TokenIndex, ColumnIndex, Contains) && Result;
				}
				else
				{
					Result = Contains(ColumnIndex, Columns[ColumnIndex]) && Result;
					ColumnIndex++;
				}
				break;
			case Token::Or:
				if (EntersScope(TokenIndex))
				{
					Result = VerifyRange(++TokenIndex, ColumnIndex, Contains) || Result;
				}
				else
				{
					Result = Contains(ColumnIndex, Columns[ColumnIndex]) || Result;
					ColumnIndex++;
				}
				break;
			case Token::ScopeOpen:
				checkf(false, TEXT("The scope open in a query should be called during processing as it should be captured by an earlier statement."));
				break;
			case Token::ScopeClose:
				return Result;
			default:
				checkf(false, TEXT("Encountered an unknown query token."));
				break;
			}
		}
		return Result;
	}

	void FConditions::ConvertColumnBitToArray(TArray<TWeakObjectPtr<const UScriptStruct>>& MatchedColumns, uint64 ColumnBits) const
	{
		int Index = 0;
		while (ColumnBits)
		{
			if (ColumnBits & 1)
			{
				MatchedColumns.Add(Columns[Index]);
			}
			++Index;
			ColumnBits >>= 1;
		}
	}

	uint8_t FConditions::MinimumColumnMatchRequiredRange(uint8_t& Front) const
	{
		uint8_t Result = EntersScope(Front) ? MinimumColumnMatchRequiredRange(++Front) : 1;
		for (; Front < TokenCount; ++Front)
		{
			Token T = Tokens[Front];
			switch (T)
			{
			case Token::And:
				if (EntersScopeNext(Front))
				{
					Front += 2;
					Result += MinimumColumnMatchRequiredRange(Front);
				}
				else
				{
					++Result;
				}
				break;
			case Token::Or:
				if (EntersScopeNext(Front))
				{
					Front += 2;
					uint8_t Lhs = MinimumColumnMatchRequiredRange(Front);
					Result = FMath::Min(Result, Lhs);
				}
				break;
			case Token::ScopeOpen:
				checkf(false, TEXT("The scope open in a query should be called during processing as it should be captured by an earlier statement."));
				break;
			case Token::ScopeClose:
				++Front;
				return Result;
			default:
				checkf(false, TEXT("Encountered an unknown query token."));
				break;
			}
		}
		return Result;
	}

	bool FConditions::UsesDynamicTemplates() const
	{
		for (EColumnFlags Flags : ColumnFlags)
		{
			if((Flags & EColumnFlags::DynamicColumnTemplate) == EColumnFlags::DynamicColumnTemplate)
			{
				return true;
			}
		}
		return false;
	}

	void FConditions::AppendQuery(FConditions& Target, const FConditions& Source)
	{
		checkf(Target.ColumnCount + Source.ColumnCount < MaxColumnCount, TEXT("Too many columns in the query."));
		for (uint8_t Index = 0; Index < Source.ColumnCount; ++Index)
		{
			Target.Columns[Target.ColumnCount + Index] = Source.Columns[Index];
			Target.Identifiers[Target.ColumnCount + Index] = Source.Identifiers[Index];
		}

		checkf(Target.TokenCount + Source.TokenCount < MaxTokenCount, TEXT("Too many operations in the query. Try simplifying your query."));
		for (uint8_t Index = 0; Index < Source.TokenCount; ++Index)
		{
			Target.Tokens[Target.TokenCount + Index] = Source.Tokens[Index];
		}

		Target.ColumnCount += Source.ColumnCount;
		Target.TokenCount += Source.TokenCount;
	}

	FConditions operator&&(const FConditions& Lhs, FColumnBase Rhs)
	{
		FConditions Result = Lhs;
		Result.Columns[Result.ColumnCount] = Rhs.TypeInfo;
		Result.Identifiers[Result.ColumnCount++] = Rhs.Identifier;
		Result.Tokens[Result.TokenCount++] = FConditions::Token::And;
		return Result;
	}

	FConditions operator&&(const FConditions& Lhs, const FConditions& Rhs)
	{
		if (Lhs.IsEmpty())
		{
			return Rhs;
		}
		else if (Rhs.IsEmpty())
		{
			return Lhs;
		}
		
		FConditions Result;
		Result.Tokens[Result.TokenCount++] = FConditions::Token::ScopeOpen;
		FConditions::AppendQuery(Result, Lhs);
		Result.Tokens[Result.TokenCount++] = FConditions::Token::ScopeClose;

		Result.Tokens[Result.TokenCount++] = FConditions::Token::And;

		Result.Tokens[Result.TokenCount++] = FConditions::Token::ScopeOpen;
		FConditions::AppendQuery(Result, Rhs);
		Result.Tokens[Result.TokenCount++] = FConditions::Token::ScopeClose;

		return Result;
	}

	FConditions operator&&(FColumnBase Lhs, FColumnBase Rhs)
	{
		FConditions Result(Lhs);
		Result.Columns[Result.ColumnCount] = Rhs.TypeInfo;
		Result.Identifiers[Result.ColumnCount++] = Rhs.Identifier;
		Result.Tokens[Result.TokenCount++] = FConditions::Token::And;
		return Result;
	}

	FConditions operator&&(FColumnBase Lhs, const FConditions& Rhs)
	{
		FConditions Result(Lhs);

		if (!Rhs.IsEmpty())
		{
			Result.Tokens[Result.TokenCount++] = FConditions::Token::And;
			Result.Tokens[Result.TokenCount++] = FConditions::Token::ScopeOpen;
			FConditions::AppendQuery(Result, Rhs);
			Result.Tokens[Result.TokenCount++] = FConditions::Token::ScopeClose;
		}
		
		return Result;
	}

	FConditions operator||(const FConditions& Lhs, FColumnBase Rhs)
	{
		FConditions Result = Lhs;
		Result.Columns[Result.ColumnCount] = Rhs.TypeInfo;
		Result.Identifiers[Result.ColumnCount++] = Rhs.Identifier;
		Result.Tokens[Result.TokenCount++] = FConditions::Token::Or;
		return Result;
	}

	FConditions operator||(const FConditions& Lhs, const FConditions& Rhs)
	{
		if (Lhs.IsEmpty())
		{
			return Rhs;
		}
		else if (Rhs.IsEmpty())
		{
			return Lhs;
		}
		
		FConditions Result;
		Result.Tokens[Result.TokenCount++] = FConditions::Token::ScopeOpen;
		FConditions::AppendQuery(Result, Lhs);
		Result.Tokens[Result.TokenCount++] = FConditions::Token::ScopeClose;

		Result.Tokens[Result.TokenCount++] = FConditions::Token::Or;

		Result.Tokens[Result.TokenCount++] = FConditions::Token::ScopeOpen;
		FConditions::AppendQuery(Result, Rhs);
		Result.Tokens[Result.TokenCount++] = FConditions::Token::ScopeClose;

		return Result;
	}

	FConditions operator||(FColumnBase Lhs, FColumnBase Rhs)
	{
		FConditions Result(Lhs);
		Result.Columns[Result.ColumnCount] = Rhs.TypeInfo;
		Result.Identifiers[Result.ColumnCount++] = Rhs.Identifier;
		Result.Tokens[Result.TokenCount++] = FConditions::Token::Or;
		return Result;
	}

	FConditions operator||(FColumnBase Lhs, const FConditions& Rhs)
	{
		FConditions Result(Lhs);
		
		if (!Rhs.IsEmpty())
		{
			Result.Tokens[Result.TokenCount++] = FConditions::Token::Or;
			Result.Tokens[Result.TokenCount++] = FConditions::Token::ScopeOpen;
			FConditions::AppendQuery(Result, Rhs);
			Result.Tokens[Result.TokenCount++] = FConditions::Token::ScopeClose;

		}
		return Result;
	}

	ENUM_CLASS_FLAGS(FConditions::EColumnFlags)
} // namespace UE::Editor::DataStorage::Queries