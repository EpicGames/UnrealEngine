// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataStorage/Scope/EditorDataScope.h"

namespace UE::Editor::DataStorage::Scope
{

// ============================================================================
// Thread-Local Current Context
// ============================================================================

static thread_local RowHandle GCurrentScope = InvalidRowHandle;

RowHandle GetCurrentScope()
{
	return GCurrentScope;
}

RowHandle SetCurrentScope(RowHandle NewContext)
{
	RowHandle Previous = GCurrentScope;
	GCurrentScope = NewContext;
	return Previous;
}

} // namespace UE::Editor::DataStorage::Scope
