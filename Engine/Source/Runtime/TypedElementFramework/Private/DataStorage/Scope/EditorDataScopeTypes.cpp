// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataStorage/Scope/EditorDataScopeTypes.h"

namespace UE::Editor::DataStorage::Scope
{

FScopeDataVersion::FScopeDataVersion() = default;

bool FScopeDataVersion::operator==(const FScopeDataVersion& Other) const
{
	return SourceRow == Other.SourceRow && Version == Other.Version;
}

bool FScopeDataVersion::operator!=(const FScopeDataVersion& Other) const
{
	return !(*this == Other);
}

bool FScopeDataVersion::IsValid() const
{
	return SourceRow != InvalidRowHandle;
}

FScopeDataVersion FScopeDataVersion::Make(RowHandle InSourceRow, uint64 InVersion)
{
	FScopeDataVersion Result;
	Result.SourceRow = InSourceRow;
	Result.Version = InVersion;
	return Result;
}

} // namespace UE::Editor::DataStorage::Scope
