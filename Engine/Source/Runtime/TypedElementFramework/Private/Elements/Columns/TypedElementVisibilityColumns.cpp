// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Columns/TypedElementVisibilityColumns.h"

#if WITH_EDITOR
#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#endif //WITH_EDITOR

#if WITH_EDITOR
namespace UE::Editor::DataStorage
{
void FVisibilityColumnChange::Apply(UObject* Object)
{
	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	if (!ensure(DataStorage))
	{
		return;
	}

	for (RowHandle Row : Rows.GetRows())
	{
		if (FVisibleInEditorColumn* VisibilityColumn = DataStorage->GetColumn<FVisibleInEditorColumn>(Row))
		{
			if (VisibilityColumn->bIsVisibleInEditor != bVisibleOnApply)
			{
				VisibilityColumn->bIsVisibleInEditor = bVisibleOnApply;
				DataStorage->AddColumn<FVisibilityChangedTag>(Row);
			}
		}
	}
}

void FVisibilityColumnChange::Revert(UObject* Object)
{
	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	if (!ensure(DataStorage))
	{
		return;
	}

	const bool bVisibleOnRevert = !bVisibleOnApply;
	for (RowHandle Row : Rows.GetRows())
	{
		if (FVisibleInEditorColumn* VisibilityColumn = DataStorage->GetColumn<FVisibleInEditorColumn>(Row))
		{
			if (VisibilityColumn->bIsVisibleInEditor != bVisibleOnRevert)
			{
				VisibilityColumn->bIsVisibleInEditor = bVisibleOnRevert;
				DataStorage->AddColumn<FVisibilityChangedTag>(Row);
			}
		}
	}
}
}//end UE::Editor::DataStorage

#endif // WITH_EDITOR
