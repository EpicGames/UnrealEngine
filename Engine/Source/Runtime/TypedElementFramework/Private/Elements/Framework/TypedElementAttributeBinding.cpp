// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/TypedElementAttributeBinding.h"

#include "DataStorage/Features.h"

namespace UE::Editor::DataStorage
{
	FAttributeBinder::FAttributeBinder(RowHandle InTargetRow)
		: FAttributeBinder(InTargetRow, GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
	{
	}

	FAttributeBinder::FAttributeBinder(
		RowHandle InTargetRow, ICoreProvider* InDataStorage)
		: TargetRow(InTargetRow)
		, DataStorage(InDataStorage)
	{
		ensureMsgf(DataStorage, TEXT("The Editor Data Storage plugin needs to be enabled to use attribute bindings."));
	}

	FTextAttributeFormatted FAttributeBinder::BindTextFormat(FTextFormat Format) const
	{
		return FTextAttributeFormatted(MoveTemp(Format), TargetRow, DataStorage);
	}
} // namespace UE::Editor::DataStorage
