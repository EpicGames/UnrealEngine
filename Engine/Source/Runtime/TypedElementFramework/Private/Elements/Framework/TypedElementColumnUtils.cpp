// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/TypedElementColumnUtils.h"

#include "UObject/Class.h"
#include "UObject/UnrealType.h"

namespace UE::Editor::DataStorage::ColumnUtils
{
	namespace Private
	{
		static const FName NAME_DynamicTemplateMetadata("EditorDataStorage_DynamicColumnTemplate");
		static const FName NAME_DerivedFromDynamicTemplateMetadata("EditorDataStorage_DerivedFromDynamicTemplate");
	}
	
	void SetColumnValue(void* ColumnData, const UScriptStruct* ColumnType, FName ArgumentName, const FString& ArgumentValue)
	{
		FProperty* Property = ColumnType->FindPropertyByName(ArgumentName);
		if (!Property)
		{
			Property = ColumnType->CustomFindProperty(ArgumentName);
		}
		if (Property)
		{
			Property->ImportText_Direct(*ArgumentValue, Property->ContainerPtrToValuePtr<uint8>(ColumnData, 0), nullptr, 0);
		}
	}

	void SetColumnValues(void* ColumnData, const UScriptStruct* ColumnType, TConstArrayView<Argument> Arguments)
	{
		for (const Argument& Arg : Arguments)
		{
			SetColumnValue(ColumnData, ColumnType, Arg.Name, Arg.Value);
		}
	}

	bool IsDynamicTemplate(const UScriptStruct* InColumn)
	{
#if WITH_EDITORONLY_DATA
		return InColumn->HasMetaData(Private::NAME_DynamicTemplateMetadata);
#else
		return false;
#endif
	}

	bool IsDerivedFromDynamicTemplate(const UScriptStruct* InColumn)
	{
#if WITH_EDITORONLY_DATA
		return InColumn->HasMetaData(Private::NAME_DerivedFromDynamicTemplateMetadata);
#else
		return false;
#endif
	}

	FName GetDynamicColumnIdentifier(const UScriptStruct* InColumn)
	{
#if WITH_EDITORONLY_DATA
		if (const FString* FoundMetaData = InColumn->FindMetaData(Private::NAME_DerivedFromDynamicTemplateMetadata))
		{
			return FName(*FoundMetaData);
		}
#endif
		return NAME_None;
	}
} // namespace TypedElement::ColumnUtils