// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/TypedElementConcurrentRowHandleCollector.h"

namespace UE::Editor::DataStorage
{
	template<EConcurrentRowHandleCollectorType Type>
	FConcurrentRowHandleCollector<Type>::~FConcurrentRowHandleCollector()
	{
		checkf(WorkingSet.load() == nullptr, TEXT("Attempting to destroy a FConcurrentRowHandleCollector while there are still values. "
			"This can indicate that there are still threads actively collecting results."));
	}

	template<EConcurrentRowHandleCollectorType Type>
	void FConcurrentRowHandleCollector<Type>::Append(FRowHandleArray&& LocalArray)
	{
		if (!LocalArray.IsEmpty())
		{
			if constexpr (Type == EConcurrentRowHandleCollectorType::Set)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_STR(TEXT("[TEDS] FConcurrentRowHandleCollector - Sort"));
				LocalArray.Sort();
			}

			{
				TRACE_CPUPROFILER_EVENT_SCOPE_STR(TEXT("[TEDS] FConcurrentRowHandleCollector - Merge"));

				// Grab the main array and merge the local results into it, or create a new main array if it doesn't exist yet.
				FRowHandleArray* WorkingSetPtr = WorkingSet.load();
				while (!WorkingSet.compare_exchange_strong(WorkingSetPtr, nullptr));
				if (WorkingSetPtr == nullptr)
				{
					WorkingSetPtr = new FRowHandleArray(MoveTemp(LocalArray));
				}
				else
				{
					TRACE_CPUPROFILER_EVENT_SCOPE_STR(TEXT("[TEDS] FConcurrentRowHandleCollector - Union"));
					if constexpr (Type == EConcurrentRowHandleCollectorType::Array)
					{
						WorkingSetPtr->Append(LocalArray.GetRows());
					}
					else
					{
						WorkingSetPtr->SortedMerge(MoveTemp(LocalArray));
					}
				}

				// Promote the local array to become the main array. If there already is a main array, try to claim that array
				// as well and merge the local array into it to form the new main array and try to promote it again.
				FRowHandleArray* NewSetPtr = nullptr;
				while (!WorkingSet.compare_exchange_strong(NewSetPtr, WorkingSetPtr))
				{
					// There was an existing list so try to claim it and merge the local array, then try to set again.
					if (WorkingSet.compare_exchange_strong(NewSetPtr, nullptr))
					{
						TRACE_CPUPROFILER_EVENT_SCOPE_STR(TEXT("[TEDS] FConcurrentRowHandleCollector - Union (retry)"));
						if constexpr (Type == EConcurrentRowHandleCollectorType::Array)
						{
							NewSetPtr->Append(WorkingSetPtr->GetRows());
						}
						else
						{
							NewSetPtr->SortedMerge(MoveTemp(*WorkingSetPtr));
						}
						delete WorkingSetPtr;
						WorkingSetPtr = NewSetPtr;
					}
					NewSetPtr = nullptr;
				}
			}
		}
	}

	template<EConcurrentRowHandleCollectorType Type>
	bool FConcurrentRowHandleCollector<Type>::Collect(FRowHandleArray& AccumulatedCollection)
	{
		bool Result = false;

		// Get the current list. Threads can continue write and this will merge what has been collected up until that point.
		FRowHandleArray* FinalList = WorkingSet.load();
		while (!WorkingSet.compare_exchange_strong(FinalList, nullptr));

		// Merge the results with the provided array.
		if (FinalList)
		{
			if (!FinalList->IsEmpty())
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_STR(TEXT("[TEDS] FConcurrentRowHandleCollector Collect"));

				if constexpr (Type == EConcurrentRowHandleCollectorType::Array)
				{
					AccumulatedCollection.Append(FinalList->GetRows());
				}
				else
				{
					AccumulatedCollection.SortedMerge(MoveTemp(*FinalList));
					AccumulatedCollection.MakeUnique();
				}
				Result = true;
			}
			delete FinalList;
		}
		return Result;
	}

	template FConcurrentRowHandleCollector<EConcurrentRowHandleCollectorType::Array>::~FConcurrentRowHandleCollector();
	template void FConcurrentRowHandleCollector<EConcurrentRowHandleCollectorType::Array>::Append(FRowHandleArray&&);
	template bool FConcurrentRowHandleCollector<EConcurrentRowHandleCollectorType::Array>::Collect(FRowHandleArray&);
	
	template FConcurrentRowHandleCollector<EConcurrentRowHandleCollectorType::Set>::~FConcurrentRowHandleCollector();
	template void FConcurrentRowHandleCollector<EConcurrentRowHandleCollectorType::Set>::Append(FRowHandleArray&&);
	template bool FConcurrentRowHandleCollector<EConcurrentRowHandleCollectorType::Set>::Collect(FRowHandleArray&);
} // namespace UE::Editor::DataStorage