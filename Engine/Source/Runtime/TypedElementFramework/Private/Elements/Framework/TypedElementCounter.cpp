// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/TypedElementCounter.h"
#include "Elements/Framework/TypedElementRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementCounter)

FTypedElementCounter::FTypedElementCounter(UTypedElementRegistry* InRegistry)
{
	Initialize(InRegistry);
}

void FTypedElementCounter::Initialize(UTypedElementRegistry* InRegistry)
{
	checkf(!Registry.Get(), TEXT("Initialize has already been called!"));
	Registry = InRegistry;
	checkf(InRegistry, TEXT("Registry is null!"));
}

void FTypedElementCounter::AddElement(const FTypedElementHandle& InElementHandle)
{
	UTypedElementRegistry* RegistryPtr = Registry.Get();
	if (!RegistryPtr || !InElementHandle)
	{
		return;
	}

	IncrementCounter(GetElementTypeCategoryName(), InElementHandle.GetId().GetTypeId());
	
	if (const TTypedElement<ITypedElementCounterInterface> ElementCounterHandle = RegistryPtr->GetElement<ITypedElementCounterInterface>(InElementHandle))
	{
		ElementCounterHandle.IncrementCountersForElement(*this);
	}
}

void FTypedElementCounter::RemoveElement(const FTypedElementHandle& InElementHandle)
{
	UTypedElementRegistry* RegistryPtr = Registry.Get();
	if (!RegistryPtr || !InElementHandle)
	{
		return;
	}

	DecrementCounter(GetElementTypeCategoryName(), InElementHandle.GetId().GetTypeId());

	if (const TTypedElement<ITypedElementCounterInterface> ElementCounterHandle = RegistryPtr->GetElement<ITypedElementCounterInterface>(InElementHandle))
	{
		ElementCounterHandle.DecrementCountersForElement(*this);
	}
}

void FTypedElementCounter::ClearCounters(const FName InCategory)
{
	CounterCategories.Remove(InCategory);
}

void FTypedElementCounter::ClearCounters()
{
	CounterCategories.Reset();
}

FName FTypedElementCounter::GetElementTypeCategoryName()
{
	static const FName NAME_ElementType = "ElementType";
	return NAME_ElementType;
}

