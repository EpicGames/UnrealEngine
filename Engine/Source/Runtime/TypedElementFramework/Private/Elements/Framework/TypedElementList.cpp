// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/TypedElementList.h"
#include "Elements/Framework/TypedElementRegistry.h"

namespace TypedElementList_Private
{

void GetElementImpl(const UTypedElementRegistry* InRegistry, const FTypedElementHandle& InElementHandle, const TSubclassOf<UInterface>& InBaseInterfaceType, FTypedElement& OutElement)
{
	if (InRegistry)
	{
		InRegistry->Private_GetElementImpl(InElementHandle, InBaseInterfaceType, OutElement);
	}
	else
	{
		OutElement.Release();
	}
}

void GetElementImpl(const UTypedElementRegistry* InRegistry, const FScriptTypedElementHandle& InElementHandle, const TSubclassOf<UInterface>& InBaseInterfaceType, FTypedElement& OutElement)
{
	if (InRegistry && InElementHandle)
	{
		InRegistry->Private_GetElementImpl(InElementHandle.GetTypedElementHandle(), InBaseInterfaceType, OutElement);
	}
	else
	{
		OutElement.Release();
	}
}

} // namespace TypedElementList_Private


template<class HandleType>
TTypedElementList<HandleType>::FLegacySync::FLegacySync(const TTypedElementList& InElementList)
	: ElementList(InElementList)
{
}

template<class HandleType>
typename TTypedElementList<HandleType>::FLegacySync::FOnSyncEvent&  TTypedElementList<HandleType>::FLegacySync::OnSyncEvent()
{
	return OnSyncEventDelegate;
}

template<class HandleType>
void TTypedElementList<HandleType>::FLegacySync::Private_EmitSyncEvent(const ESyncType InSyncType, const HandleType& InElementHandle)
{
	const bool bIsWithinBatchOperation = IsRunningBatchOperation();
	bBatchOperationIsDirty |= bIsWithinBatchOperation;
	OnSyncEventDelegate.Broadcast(ElementList, InSyncType, InElementHandle, bIsWithinBatchOperation);
}

template<class HandleType>
bool TTypedElementList<HandleType>::FLegacySync::IsRunningBatchOperation() const
{
	return NumOpenBatchOperations > 0;
}

template<class HandleType>
void TTypedElementList<HandleType>::FLegacySync::BeginBatchOperation()
{
	++NumOpenBatchOperations;
}

template<class HandleType>
void TTypedElementList<HandleType>::FLegacySync::EndBatchOperation(const bool InNotify)
{
	checkf(NumOpenBatchOperations > 0, TEXT("Batch operation underflow!"));

	if (--NumOpenBatchOperations == 0)
	{
		const bool bNotifyChange = bBatchOperationIsDirty && InNotify;
		bBatchOperationIsDirty = false;

		if (bNotifyChange)
		{
			Private_EmitSyncEvent(ESyncType::BatchComplete);
			check(!bBatchOperationIsDirty); // This should still be false after emitting the notification!
		}
	}
}

template<class HandleType>
bool TTypedElementList<HandleType>::FLegacySync::IsBatchOperationDirty() const
{
	return bBatchOperationIsDirty;
}

template<class HandleType>
void TTypedElementList<HandleType>::FLegacySync::ForceBatchOperationDirty()
{
	if (NumOpenBatchOperations > 0)
	{
		bBatchOperationIsDirty = true;
	}
}


template<class HandleType>
TTypedElementList<HandleType>::FLegacySyncScopedBatch::FLegacySyncScopedBatch(const TTypedElementList& InElementList, const bool InNotify)
	: ElementListLegacySync(InElementList.Legacy_GetSyncPtr())
	, bNotify(InNotify)
{
	if (ElementListLegacySync)
	{
		ElementListLegacySync->BeginBatchOperation();
	}
}

template<class HandleType>
TTypedElementList<HandleType>::FLegacySyncScopedBatch::~FLegacySyncScopedBatch()
{
	if (ElementListLegacySync)
	{
		ElementListLegacySync->EndBatchOperation(bNotify);
	}
}

template<class HandleType>
bool TTypedElementList<HandleType>::FLegacySyncScopedBatch::IsDirty() const
{
	return ElementListLegacySync
		&& ElementListLegacySync->IsBatchOperationDirty();
}

template<class HandleType>
void TTypedElementList<HandleType>::FLegacySyncScopedBatch::ForceDirty()
{
	if (ElementListLegacySync)
	{
		ElementListLegacySync->ForceBatchOperationDirty();
	}
}

template<class HandleType>
typename TTypedElementList<HandleType>::TTypedElementListRef TTypedElementList<HandleType>::Private_CreateElementList(UTypedElementRegistry* InRegistry)
{
	return MakeShareable(new TTypedElementList(InRegistry));
}

template<class HandleType>
TTypedElementList<HandleType>::TTypedElementList(UTypedElementRegistry* InRegistry)
	: Registry(InRegistry)
{
	checkf(InRegistry, TEXT("Registry is null!"));

	ElementCounts.Initialize(InRegistry);

	InRegistry->Private_OnElementListCreated(this);
}

template<class HandleType>
TTypedElementList<HandleType>::~TTypedElementList()
{
	Empty();
	LegacySync.Reset();
	if (UTypedElementRegistry* RegistryPtr = Registry.Get())
	{
		RegistryPtr->Private_OnElementListDestroyed(this);
		Registry = nullptr;
	}
}

template<class HandleType>
typename TTypedElementList<HandleType>::TTypedElementListRef TTypedElementList<HandleType>::Clone() const
{
	TTypedElementListRef ClonedElementList = Private_CreateElementList(Registry.Get());
	for (const HandleType& ElementHandle : ElementHandles)
	{
		ClonedElementList->Add(ElementHandle);
	}
	return ClonedElementList;
}

template<class HandleType>
UObject* TTypedElementList<HandleType>::GetElementInterface(const HandleType& InElementHandle, const TSubclassOf<UInterface>& InBaseInterfaceType) const
{
	UTypedElementRegistry* RegistryPtr = Registry.Get();
	return RegistryPtr
		? Registry->GetElementInterface(InElementHandle, InBaseInterfaceType)
		: nullptr;
}

template<class HandleType>
bool TTypedElementList<HandleType>::HasElements(const TSubclassOf<UInterface>& InBaseInterfaceType) const
{
	bool bHasFilteredElements = false;

	if (InBaseInterfaceType)
	{
		ForEachElementHandle([&bHasFilteredElements](const HandleType&)
		{
			bHasFilteredElements = true;
			return false;
		}, InBaseInterfaceType);
	}
	else
	{
		bHasFilteredElements = Num() > 0;
	}

	return bHasFilteredElements;
}

template<class HandleType>
int32 TTypedElementList<HandleType>::CountElements(const TSubclassOf<UInterface>& InBaseInterfaceType) const
{
	int32 NumFilteredElements = 0;

	if (InBaseInterfaceType)
	{
		ForEachElementHandle([&NumFilteredElements](const HandleType&)
		{
			++NumFilteredElements;
			return true;
		}, InBaseInterfaceType);
	}
	else
	{
		NumFilteredElements = Num();
	}

	return NumFilteredElements;
}

template<class HandleType>
bool TTypedElementList<HandleType>::HasElementsOfType(const FName InElementTypeName) const
{
	return CountElementsOfType(InElementTypeName) > 0;
}

template<class HandleType>
bool TTypedElementList<HandleType>::HasElementsOfType(const FTypedHandleTypeId InElementTypeId) const
{
	return CountElementsOfType(InElementTypeId) > 0;
}

template<class HandleType>
int32 TTypedElementList<HandleType>::CountElementsOfType(const FName InElementTypeName) const
{
	if (UTypedElementRegistry* RegistryPtr = Registry.Get())
	{
		const FTypedHandleTypeId ElementTypeId = RegistryPtr->GetRegisteredElementTypeId(InElementTypeName);
		if (ElementTypeId > 0)
		{
			return CountElementsOfType(ElementTypeId);
		}
	}
	return 0;
}

template<class HandleType>
int32 TTypedElementList<HandleType>::CountElementsOfType(const FTypedHandleTypeId InElementTypeId) const
{
	return ElementCounts.GetCounterValue(FTypedElementCounter::GetElementTypeCategoryName(), InElementTypeId);
}

template<class HandleType>
TArray<HandleType> TTypedElementList<HandleType>::GetElementHandles(const TSubclassOf<UInterface>& InBaseInterfaceType) const
{
	TArray<HandleType> FilteredElementHandles;
	FilteredElementHandles.Reserve(ElementHandles.Num());

	ForEachElementHandle([&FilteredElementHandles](const HandleType& InElementHandle)
	{
		FilteredElementHandles.Add(InElementHandle);
		return true;
	}, InBaseInterfaceType);

	return FilteredElementHandles;
}

template<class HandleType>
void TTypedElementList<HandleType>::ForEachElementHandle(TFunctionRef<bool(const HandleType&)> InCallback, const TSubclassOf<UInterface>& InBaseInterfaceType) const
{
	for (const HandleType& ElementHandle : ElementHandles)
	{
		if (ElementHandle && (!InBaseInterfaceType || GetElementInterface(ElementHandle, InBaseInterfaceType)))
		{
			if (!InCallback(ElementHandle))
			{
				break;
			}
		}
	}
}

template<class HandleType>
bool TTypedElementList<HandleType>::AddElementImpl(HandleType&& InElementHandle)
{
	if (!InElementHandle)
	{
		return false;
	}

	NoteListMayChange();

	bool bAlreadyAdded = false;
	ElementCombinedIds.Add(InElementHandle.GetId().GetCombinedId(), &bAlreadyAdded);

	if (!bAlreadyAdded)
	{
		const HandleType& AddedElementHandle = ElementHandles.Add_GetRef(MoveTemp(InElementHandle));

		if constexpr (std::is_same<HandleType, FTypedElementHandle>::value)
		{
			ElementCounts.AddElement(AddedElementHandle);
		}
		else
		{
			ElementCounts.AddElement(AddedElementHandle.GetTypedElementHandle());
		}
		NoteListChanged(EChangeType::Added, AddedElementHandle);
	}

	return !bAlreadyAdded;
}

template<class HandleType>
bool TTypedElementList<HandleType>::RemoveElementImpl(const FTypedElementId& InElementId)
{
	if (!InElementId)
	{
		return false;
	}

	NoteListMayChange();

	const bool bRemoved = ElementCombinedIds.Remove(InElementId.GetCombinedId()) > 0;

	if (bRemoved)
	{
		const int32 ElementHandleIndexToRemove = ElementHandles.IndexOfByPredicate([&InElementId](const HandleType& InElementHandle)
		{
			return InElementHandle.GetId() == InElementId;
		});
		checkSlow(ElementHandleIndexToRemove != INDEX_NONE);

		HandleType RemovedElementHandle = MoveTemp(ElementHandles[ElementHandleIndexToRemove]);
		ElementHandles.RemoveAt(ElementHandleIndexToRemove, EAllowShrinking::No);

		if constexpr (std::is_same<HandleType, FTypedElementHandle>::value)
		{
			ElementCounts.RemoveElement(RemovedElementHandle);
		}
		else
		{
			ElementCounts.RemoveElement(RemovedElementHandle.GetTypedElementHandle());
		}
	
		NoteListChanged(EChangeType::Removed, RemovedElementHandle);
	}

	return bRemoved;
}

template<class HandleType>
int32 TTypedElementList<HandleType>::RemoveAllElementsImpl(TFunctionRef<bool(const HandleType&)> InPredicate)
{
	if (ElementHandles.Num() > 0)
	{
		FLegacySyncScopedBatch LegacySyncBatch(*this);

		NoteListMayChange();

		return ElementHandles.RemoveAll([this, InPredicate](HandleType& InHandle)
			{
				if (InPredicate(InHandle))
				{
					HandleType RemovedElementHandle = MoveTemp(InHandle);
					ElementCombinedIds.Remove(RemovedElementHandle.GetId().GetCombinedId());

					if constexpr (std::is_same<HandleType, FTypedElementHandle>::value)
					{
						ElementCounts.RemoveElement(RemovedElementHandle);
					}
					else
					{
						ElementCounts.RemoveElement(RemovedElementHandle.GetTypedElementHandle());
					}

					NoteListChanged(EChangeType::Removed, RemovedElementHandle);

					return true;
				}

				return false;
			});
	}

	return 0;
}

template<class HandleType>
bool TTypedElementList<HandleType>::ContainsElementImpl(const FTypedElementId& InElementId) const
{
	return InElementId 
		&& ElementCombinedIds.Contains(InElementId.GetCombinedId());
}

template<class HandleType>
typename TTypedElementList<HandleType>::FLegacySync& TTypedElementList<HandleType>::Legacy_GetSync()
{
	if (!LegacySync)
	{
		LegacySync = MakeUnique<FLegacySync>(*this);
	}
	return *LegacySync;
}

template<class HandleType>
typename TTypedElementList<HandleType>::FLegacySync* TTypedElementList<HandleType>::Legacy_GetSyncPtr() const
{
	return LegacySync.Get();
}

template<class HandleType>
bool TTypedElementList<HandleType>::NotifyPendingChanges()
{
	if (bHasPendingNotify)
	{
		bHasPendingNotify = false;
		OnChangedDelegate.Broadcast(*this);
		return true;
	}

	return false;
}

template<class HandleType>
TTypedElementList<HandleType>::FScopedClearNewPendingChange::FScopedClearNewPendingChange(TTypedElementList& InTypeElementList)
{
	bool bCanClearNewPendingChange = !InTypeElementList.bHasPendingNotify;

	if (InTypeElementList.LegacySync)
	{
		bCanClearNewPendingChange &= !InTypeElementList.LegacySync->IsRunningBatchOperation();
	}
	
	if (bCanClearNewPendingChange)
	{
		TypedElementList = &InTypeElementList;
	}
}

template<class HandleType>
TTypedElementList<HandleType>::FScopedClearNewPendingChange::FScopedClearNewPendingChange(FScopedClearNewPendingChange&& Other)
	: TypedElementList(Other.TypedElementList)
{
	Other.TypedElementList = nullptr;
}

template<class HandleType>
typename TTypedElementList<HandleType>::FScopedClearNewPendingChange& TTypedElementList<HandleType>::FScopedClearNewPendingChange::operator=(FScopedClearNewPendingChange&& Other)
{
	TypedElementList = Other.TypedElementList;
	Other.TypedElementList = nullptr;

	return *this;
}

template<class HandleType>
TTypedElementList<HandleType>::FScopedClearNewPendingChange::~FScopedClearNewPendingChange()
{
	if (TypedElementList)
	{
		TypedElementList->bHasPendingNotify = false;
	}
}

template<class HandleType>
typename TTypedElementList<HandleType>::FScopedClearNewPendingChange TTypedElementList<HandleType>::GetScopedClearNewPendingChange()
{
	return FScopedClearNewPendingChange(*this);
}

template<class HandleType>
void TTypedElementList<HandleType>::NoteListMayChange()
{
	OnPreChangeDelegate.Broadcast(*this);
}

template<class HandleType>
void TTypedElementList<HandleType>::NoteListChanged(const EChangeType InChangeType, const HandleType& InElementHandle)
{
	bHasPendingNotify = true;

	if (LegacySync)
	{
		typename FLegacySync::ESyncType SyncType = FLegacySync::ESyncType::Modified;
		switch (InChangeType)
		{
		case EChangeType::Added:
			SyncType = FLegacySync::ESyncType::Added;
			break;

		case EChangeType::Removed:
			SyncType = FLegacySync::ESyncType::Removed;
			break;

		case EChangeType::Cleared:
			SyncType = FLegacySync::ESyncType::Cleared;
			break;

		default:
			break;
		}

		LegacySync->Private_EmitSyncEvent(SyncType, InElementHandle);
	}
}

template class TTypedElementList<FTypedElementHandle>;
template class TTypedElementList<FScriptTypedElementHandle>;


namespace UE::TypedElementFramework
{
	/**
	 * Functions to convert a script list to a native one
	 * @note Only copies elements; does not copy any bindings!
	 */
	FTypedElementListPtr ConvertToNativeTypedElementList(const FScriptTypedElementListConstPtr& ScriptList)
	{
		if (ScriptList)
		{
			if (UTypedElementRegistry* RawRegistry = ScriptList->GetRegistry())
			{
				FTypedElementListPtr NativeList = FTypedElementList::Private_CreateElementList(RawRegistry);
				NativeList->Reserve(ScriptList->CountElements());
				ScriptList->ForEachElementHandle([&NativeList](const FScriptTypedElementHandle& ScriptHandle)
					{
						if (FTypedElementHandle NativeHandle = ScriptHandle.GetTypedElementHandle())
						{
							NativeList->Add(NativeHandle);
						}
						return true;
					});

				return NativeList;
			}
		}

		return {};
	}

	/**
	 * Functions to convert a script list to a native one
	 * @note Only copies elements; does not copy any bindings!
	 */
	FScriptTypedElementListPtr ConvertToScriptTypedElementList(const FTypedElementListConstPtr& NativeList)
	{
		if (NativeList)
		{
			if (UTypedElementRegistry* RawRegistry = NativeList->GetRegistry())
			{
				FScriptTypedElementListPtr ScriptList = FScriptTypedElementList::Private_CreateElementList(RawRegistry);
				ScriptList->Reserve(NativeList->CountElements());
				NativeList->ForEachElementHandle([&ScriptList, &RawRegistry](const FTypedElementHandle& NativeHandle)
					{
						ScriptList->Add(RawRegistry->CreateScriptHandle(NativeHandle.GetId()));
						return true;
					});

				return ScriptList;
			}
		}

		return {};
	}
}
