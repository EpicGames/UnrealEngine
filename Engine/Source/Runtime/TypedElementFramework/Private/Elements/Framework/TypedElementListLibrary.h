// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/TypedElementListProxy.h"
#include "Elements/Framework/TypedElementList.h"
#include "TypedElementListLibrary.generated.h"

UCLASS()
class UTypedElementListLibrary : public UObject
{
	GENERATED_BODY()
	
public:
	/**
	 * Create an empty list of elements associated with the given registry.
	 */
	UFUNCTION(BlueprintCallable, DisplayName="Create Element List", Category = "TypedElementFramework|Registry", meta=(ScriptMethod, ScriptName="CreateElementList"))
	static FScriptTypedElementListProxy CreateScriptElementList(UTypedElementRegistry* Registry)
	{
		return Registry->CreateScriptElementList();
	}

	/**
	 * Clone this list instance.
	 * @note Only copies elements; does not copy any bindings!
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|List", meta=(ScriptMethod))
	static FScriptTypedElementListProxy Clone(const FScriptTypedElementListProxy ElementList)
	{
		FScriptTypedElementListConstPtr ElementListPtr = ElementList.GetElementList();
		return ElementListPtr ? ElementListPtr->Clone() : FScriptTypedElementListProxy();
	}

	/**
	 * Get the element handle at the given index.
	 * @note Use IsValidIndex to test for validity.
	 */
	UFUNCTION(BlueprintPure, Category="TypedElementFramework|List", meta=(ScriptMethod))
	static FScriptTypedElementHandle GetElementHandleAt(const FScriptTypedElementListProxy ElementList, const int32 Index)
	{
		FScriptTypedElementListConstPtr ElementListPtr = ElementList.GetElementList();
		return ElementListPtr ? ElementListPtr->GetElementHandleAt(Index) : FScriptTypedElementHandle();
	}

	/**
	 * Get the element interface from the given handle.
	 */
	UFUNCTION(BlueprintPure, Category="TypedElementFramework|List", meta=(ScriptMethod))
	static UObject* GetElementInterface(const FScriptTypedElementListProxy ElementList, const FScriptTypedElementHandle& ElementHandle, const TSubclassOf<UInterface> BaseInterfaceType)
	{
		FScriptTypedElementListConstPtr ElementListPtr = ElementList.GetElementList();
		return ElementListPtr ? ElementListPtr->GetElementInterface(ElementHandle, BaseInterfaceType) : nullptr;
	}

	/**
	 * Test whether there are elements in this list, optionally filtering to elements that implement the given interface.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|List", meta=(ScriptMethod))
	static bool HasElements(const FScriptTypedElementListProxy ElementList, const TSubclassOf<UInterface> BaseInterfaceType = {})
	{
		FScriptTypedElementListConstPtr ElementListPtr = ElementList.GetElementList();
		return ElementListPtr && ElementListPtr->HasElements(BaseInterfaceType);
	}

	/**
	 * Count the number of elements in this list, optionally filtering to elements that implement the given interface.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|List", meta=(ScriptMethod))
	static int32 CountElements(const FScriptTypedElementListProxy ElementList, const TSubclassOf<UInterface> BaseInterfaceType = {})
	{
		FScriptTypedElementListConstPtr ElementListPtr = ElementList.GetElementList();
		return ElementListPtr ? ElementListPtr->CountElements(BaseInterfaceType) : 0;
	}

	/**
	 * Test whether there are elements in this list of the given type.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|List", meta=(ScriptMethod))
	static bool HasElementsOfType(const FScriptTypedElementListProxy ElementList, const FName ElementTypeName)
	{
		FScriptTypedElementListConstPtr ElementListPtr = ElementList.GetElementList();
		return ElementListPtr && ElementListPtr->HasElementsOfType(ElementTypeName);
	}

	/**
	 * Count the number of elements in this list of the given type.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|List", meta=(ScriptMethod))
	static int32 CountElementsOfType(const FScriptTypedElementListProxy ElementList, const FName ElementTypeName)
	{
		FScriptTypedElementListConstPtr ElementListPtr = ElementList.GetElementList();
		return ElementListPtr ? ElementListPtr->CountElementsOfType(ElementTypeName) : 0;
	}

	/**
	 * Get the handle of every element in this list, optionally filtering to elements that implement the given interface.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|List", meta=(ScriptMethod))
	static TArray<FScriptTypedElementHandle> GetElementHandles(const FScriptTypedElementListProxy ElementList, const TSubclassOf<UInterface> BaseInterfaceType)
	{
		FScriptTypedElementListConstPtr ElementListPtr = ElementList.GetElementList();
		return ElementListPtr ? ElementListPtr->GetElementHandles(BaseInterfaceType) : TArray<FScriptTypedElementHandle>();
	}

	/**
	 * Is the given index a valid entry within this element list?
	 */
	UFUNCTION(BlueprintPure, Category="TypedElementFramework|List", meta=(ScriptMethod))
	static bool IsValidIndex(const FScriptTypedElementListProxy ElementList, const int32 Index)
	{
		FScriptTypedElementListConstPtr ElementListPtr = ElementList.GetElementList();
		return ElementListPtr && ElementListPtr->IsValidIndex(Index);
	}

	/**
	 * Get the number of entries within this element list.
	 */
	UFUNCTION(BlueprintPure, Category="TypedElementFramework|List", meta=(ScriptMethod))
	static int32 Num(const FScriptTypedElementListProxy ElementList)
	{
		FScriptTypedElementListConstPtr ElementListPtr = ElementList.GetElementList();
		return ElementListPtr ? ElementListPtr->Num() : 0;
	}

	/**
	 * Shrink this element list storage to avoid slack.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|List", meta=(ScriptMethod))
	static void Shrink(FScriptTypedElementListProxy ElementList)
	{
		if (FScriptTypedElementListPtr ElementListPtr = ElementList.GetElementList())
		{
			ElementListPtr->Shrink();
		}
	}

	/**
	 * Pre-allocate enough memory in this element list to store the given number of entries.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|List", meta=(ScriptMethod))
	static void Reserve(FScriptTypedElementListProxy ElementList, const int32 Size)
	{
		if (FScriptTypedElementListPtr ElementListPtr = ElementList.GetElementList())
		{
			ElementListPtr->Reserve(Size);
		}
	}

	/**
	 * Remove all entries from this element list, potentially leaving space allocated for the given number of entries.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|List", meta=(ScriptMethod))
	static void Empty(FScriptTypedElementListProxy ElementList, const int32 Slack = 0)
	{
		if (FScriptTypedElementListPtr ElementListPtr = ElementList.GetElementList())
		{
			ElementListPtr->Empty(Slack);
		}
	}

	/**
	 * Remove all entries from this element list, preserving existing allocations.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|List", meta=(ScriptMethod))
	static void Reset(FScriptTypedElementListProxy ElementList)
	{
		if (FScriptTypedElementListPtr ElementListPtr = ElementList.GetElementList())
		{
			ElementListPtr->Reset();
		}
	}

	/**
	 * Does this element list contain an entry for the given element handle?
	 */
	UFUNCTION(BlueprintPure, Category="TypedElementFramework|List", meta=(ScriptMethod))
	static bool Contains(const FScriptTypedElementListProxy ElementList, const FScriptTypedElementHandle& ElementHandle)
	{
		FScriptTypedElementListConstPtr ElementListPtr = ElementList.GetElementList();
		return ElementListPtr && ElementListPtr->Contains(ElementHandle);
	}

	/**
	 * Add the given element handle to this element list, if it isn't already in the list.
	 * @return True if the element handle was added, false if it is already in the list.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|List", meta=(ScriptMethod))
	static bool Add(FScriptTypedElementListProxy ElementList, const FScriptTypedElementHandle& ElementHandle)
	{
		FScriptTypedElementListPtr ElementListPtr = ElementList.GetElementList();
		return ElementListPtr && ElementListPtr->Add(ElementHandle);
	}

	/**
	 * Append the given element handles to this element list.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|List", meta=(ScriptMethod))
	static void Append(FScriptTypedElementListProxy ElementList, const TArray<FScriptTypedElementHandle>& ElementHandles)
	{
		if (FScriptTypedElementListPtr ElementListPtr = ElementList.GetElementList())
		{
			ElementListPtr->Append(ElementHandles);
		}
	}

	/**
	 * Append the another element list to this element list.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|List", meta=(ScriptMethod))
	static void AppendList(FScriptTypedElementListProxy ElementList, const FScriptTypedElementListProxy OtherElementList)
	{
		FScriptTypedElementListPtr ElementListPtr = ElementList.GetElementList();
		FScriptTypedElementListConstPtr OtherElementListPtr = OtherElementList.GetElementList();
		if (ElementListPtr && OtherElementListPtr)
		{
			ElementListPtr->Append(OtherElementListPtr.ToSharedRef());
		}
	}

	/**
	 * Remove the given element handle from this element list, if it is in the list.
	 * @return True if the element handle was removed, false if it isn't in the list.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|List", meta=(ScriptMethod))
	static bool Remove(FScriptTypedElementListProxy ElementList, const FScriptTypedElementHandle& ElementHandle)
	{
		FScriptTypedElementListPtr ElementListPtr = ElementList.GetElementList();
		return ElementListPtr && ElementListPtr->Remove(ElementHandle);
	}
};
