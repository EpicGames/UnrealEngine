// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/TypedElementMetaData.h"

#include "DataStorage/Queries/Description.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"

namespace UE::Editor::DataStorage
{
	template<class... Ts>
	struct TOverloaded : Ts... 
	{ 
		using Ts::operator()...;
	};
	
	template<class... Ts>
	TOverloaded(Ts...) -> TOverloaded<Ts...>;

	//
	// FMetaDataBase
	//

	FMetaDataEntryView FMetaDataBase::Find(FName Name) const
	{
		if (const MetaDataType* Result = ImmutableData.Find(Name))
		{
			return FMetaDataEntryView(*Result);
		}

		if (const MetaDataType* Result = MutableData.Find(Name))
		{
			return FMetaDataEntryView(*Result);
		}

		return FMetaDataEntryView();
	}

	void FMetaDataBase::Shrink()
	{
		ImmutableData.Shrink();
		MutableData.Shrink();
	}

	bool FMetaDataBase::IsEmpty() const
	{
		return ImmutableData.IsEmpty() && MutableData.IsEmpty();
	}


	//
	// FColumnMetadata
	//

	FColumnMetaData::FColumnMetaData(const UScriptStruct* InColumnType, EFlags InFlags)
		: ColumnType(InColumnType)
		, Flags(InFlags)
	{}

	FMetaDataEntryView FColumnMetaData::Find(FName Name) const
	{
		if (Name == IsEditableName)
		{
			return FMetaDataEntryView((Flags & EFlags::IsMutable) != EFlags::None);
		}
		else if (Name == IsConstName)
		{
			return FMetaDataEntryView(!(Flags & EFlags::IsMutable));
		}

		FMetaDataEntryView Result = FMetaDataBase::Find(Name);

#if WITH_EDITORONLY_DATA
		if (ColumnType && !Result.IsSet())
		{
			if (const FString* FoundMetaData = ColumnType->FindMetaData(Name))
			{
				return FMetaDataEntryView(*FoundMetaData);
			}
		}
#endif

		return Result;
	}

	const UScriptStruct* FColumnMetaData::GetColumnType() const
	{
		return ColumnType;
	}

	bool FColumnMetaData::IsEmpty() const
	{
		return FMetaDataBase::IsEmpty() && ColumnType == nullptr;
	}


	//
	// FMetaDataEntryView
	// 

	FMetaDataEntryView::FMetaDataEntryView()
		: DataView(TInPlaceType<FEmptyVariantState>())
	{}

	FMetaDataEntryView::FMetaDataEntryView(const MetaDataType& MetaData)
	{
		Visit(TOverloaded
			{
				[this](const auto& Value)
				{
					using TargetType = typename TDecay<decltype(Value)>::Type;
					DataView.Emplace<TargetType>(Value);
				},
				[this](const FString& String)
				{
					DataView.Emplace<const FString*>(&String);
				}
			}, MetaData);
	}

	FMetaDataEntryView::FMetaDataEntryView(const FString& MetaDataString)
		: DataView(TInPlaceType<const FString*>(), &MetaDataString)
	{}

	bool FMetaDataEntryView::IsSet() const
	{
		return !DataView.IsType<FEmptyVariantState>();
	}



	//
	// FMetaDataView
	//
	
	FMetaDataEntryView FMetaDataView::FindGeneric(FName AttributeName) const
	{
		return FMetaDataEntryView();
	}

	FMetaDataEntryView FMetaDataView::FindForColumn(TWeakObjectPtr<const UScriptStruct> Column, FName AttributeName) const
	{
		return FMetaDataEntryView();
	}

	FMetaDataEntryView FMetaDataView::FindForColumnOrGeneric(TWeakObjectPtr<const UScriptStruct> Column, FName AttributeName) const
	{
		FMetaDataEntryView ColumnMetaData = FindForColumn(Column, AttributeName);
		return ColumnMetaData.IsSet() ? ColumnMetaData : FindGeneric(AttributeName);
	}

	FMetaDataEntryView FMetaDataView::FindForColumnsOrGeneric(const TConstArrayView<TWeakObjectPtr<const UScriptStruct>> Columns, FName AttributeName) const
	{
		for (const TWeakObjectPtr<const UScriptStruct>& Column : Columns)
		{
			FMetaDataEntryView ColumnMetaData = FindForColumn(Column, AttributeName);
			if (ColumnMetaData.IsSet())
			{
				return ColumnMetaData;
			}
		}
		return FindGeneric(AttributeName);
	}


	//
	// FQueryMetaDataView
	//

	FQueryMetaDataView::FQueryMetaDataView(const FQueryDescription& InQuery)
		:Query(InQuery)
	{
	}

	FMetaDataEntryView FQueryMetaDataView::FindGeneric(FName AttributeName) const
	{
		return Query.MetaData.Find(AttributeName);
	}

	FMetaDataEntryView FQueryMetaDataView::FindForColumn(
		TWeakObjectPtr<const UScriptStruct> Column, FName AttributeName) const
	{
		int32 Index = 0;
		if (Query.SelectionTypes.Find(Column, Index))
		{
			FMetaDataEntryView Result = Query.SelectionMetaData[Index].Find(AttributeName);
			if (Result.IsSet())
			{
				return Result;
			}
		}

		return FMetaDataEntryView();
	}



	//
	// FColumnsMetaDataView
	//

	FColumnsMetaDataView::FColumnsMetaDataView(TConstArrayView<TWeakObjectPtr<const UScriptStruct>> InColumns)
		: Columns(InColumns)
	{
	}

	FColumnsMetaDataView::FColumnsMetaDataView(TConstArrayView<TSharedPtr<FColumnMetaData>> InColumnsMetaData)
		: ColumnsMetaData(InColumnsMetaData)
	{
	}

	FMetaDataEntryView FColumnsMetaDataView::FindForColumn(
		TWeakObjectPtr<const UScriptStruct> Column, FName AttributeName) const
	{
#if WITH_EDITORONLY_DATA
		if (Columns.Find(Column) != INDEX_NONE)
		{
			if (const UScriptStruct* ColumnType = Column.Get())
			{
				if (const FString* FoundMetaData = ColumnType->FindMetaData(AttributeName))
				{
					return FMetaDataEntryView(*FoundMetaData);
				}
			}
		}
#endif

		const TSharedPtr<FColumnMetaData>* MetaData = ColumnsMetaData.FindByPredicate([Column](const TSharedPtr<FColumnMetaData>& MetaData)
			{
				return MetaData && MetaData->GetColumnType() == Column;
			});
		
		if (MetaData && *MetaData)
		{
			return (*MetaData)->Find(AttributeName);
		}
		
		return FMetaDataEntryView();
	}

	

	//
	// FForwardingMetaDataView
	//

	FForwardingMetaDataView::FForwardingMetaDataView(const FMetaDataView& InView)
		: View(InView)
	{
	}

	FMetaDataEntryView FForwardingMetaDataView::FindGeneric(FName AttributeName) const
	{
		return View.FindGeneric(AttributeName);
	}

	FMetaDataEntryView FForwardingMetaDataView::FindForColumn(
		TWeakObjectPtr<const UScriptStruct> Column, FName AttributeName) const
	{
		return View.FindForColumn(Column, AttributeName);
	}

	
	
	//
	// FGenericMetaDataView
	//
	FGenericMetaDataView::FGenericMetaDataView(const FMetaData& InMetaData)
		: MetaData(InMetaData)
	{
	}
	
	FMetaDataEntryView FGenericMetaDataView::FindGeneric(FName AttributeName) const
	{
		return MetaData.Find(AttributeName);
	}
} // namespace UE::Editor::DataStorage
