// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/TypedElementQueryBuilder.h"

#include "Algo/BinarySearch.h"
#include "DataStorage/Debug/Log.h"
#include "Elements/Framework/TypedElementMetaData.h"
#include "GenericPlatform/GenericPlatformMath.h"

#define LOCTEXT_NAMESPACE "TypedElementQueryUtils"


namespace UE::Editor::DataStorage::Queries
{
	const UScriptStruct* Type(FTopLevelAssetPath Name)
	{
		const UScriptStruct* StructInfo = TypeOptional(Name);
		checkf(StructInfo, TEXT("Type name '%s' used as part of building a typed element query was not found."), *Name.ToString());
		return StructInfo;
	}

	const UScriptStruct* TypeOptional(FTopLevelAssetPath Name)
	{
		return static_cast<const UScriptStruct*>(StaticFindObject(UScriptStruct::StaticClass(), Name, EFindObjectFlags::ExactClass));
	}

	const UScriptStruct* operator""_Type(const char* Name, std::size_t NameSize)
	{
		return Type(FTopLevelAssetPath{ FAnsiStringView{ Name, IntCastChecked<int32>(NameSize) } });
	}

	const UScriptStruct* operator""_TypeOptional(const char* Name, std::size_t NameSize)
	{
		return TypeOptional(FTopLevelAssetPath{ FAnsiStringView{ Name, IntCastChecked<int32>(NameSize) } });
	}



	//
	// DependsOn
	//

	FDependency::FDependency(FQueryDescription* Query)
		: Query(Query)
	{
	}

	FDependency& FDependency::ReadOnly(const UClass* Target)
	{
		checkf(Target, TEXT("The Dependency section in the Typed Elements query builder doesn't support nullptrs as Read-Only input."));
		Query->DependencyTypes.Emplace(Target);
		Query->DependencyFlags.Emplace(EQueryDependencyFlags::ReadOnly);
		Query->CachedDependencies.AddDefaulted();
		return *this;
	}

	FDependency& FDependency::ReadOnly(TConstArrayView<const UClass*> Targets)
	{
		int32 NewSize = Query->CachedDependencies.Num() + Targets.Num();
		Query->DependencyTypes.Reserve(NewSize);
		Query->CachedDependencies.Reserve(NewSize);
		Query->DependencyFlags.Reserve(NewSize);
		
		for (const UClass* Target : Targets)
		{
			ReadOnly(Target);
		}
		return *this;
	}

	FDependency& FDependency::ReadWrite(const UClass* Target)
	{
		checkf(Target, TEXT("The Dependency section in the Typed Elements query builder doesn't support nullptrs as Read/Write input."));
		Query->DependencyTypes.Emplace(Target);
		Query->DependencyFlags.Emplace(EQueryDependencyFlags::None);
		Query->CachedDependencies.AddDefaulted();
		return *this;
	}

	FDependency& FDependency::ReadWrite(TConstArrayView<const UClass*> Targets)
	{
		int32 NewSize = Query->CachedDependencies.Num() + Targets.Num();
		Query->DependencyTypes.Reserve(NewSize);
		Query->CachedDependencies.Reserve(NewSize);
		Query->DependencyFlags.Reserve(NewSize);
		
		for (const UClass* Target : Targets)
		{
			ReadWrite(Target);
		}
		return *this;
	}

	FDependency& FDependency::SubQuery(QueryHandle Handle)
	{
		checkf(Query->Callback.ExecutionMode != EExecutionMode::ThreadedChunks,
			TEXT("TEDS sub-queries can not be added to queries with a callback that process chunks in parallel."));
		Query->Subqueries.Add(Handle);
		return *this;
	}
	
	FDependency& FDependency::SubQuery(TConstArrayView<QueryHandle> Handles)
	{
		checkf(Query->Callback.ExecutionMode != EExecutionMode::ThreadedChunks,
			TEXT("TEDS sub-queries can not be added to queries with a callback that process chunks in parallel."));
		Query->Subqueries.Insert(Handles.GetData(), Handles.Num(), Query->Subqueries.Num());
		return *this;
	}

	FQueryDescription&& FDependency::Compile()
	{
		return MoveTemp(*Query);
	}


	/**
	 * Simple Query
	 */

	FSimpleQuery::FSimpleQuery(FQueryDescription* Query)
		: Query(Query)
	{
	}

	FSimpleQuery& FSimpleQuery::All(const UScriptStruct* Target)
	{
		if (Target)
		{
			Query->ConditionTypes.Add(FQueryDescription::EOperatorType::SimpleAll);
			Query->ConditionOperators.AddZeroed_GetRef().Type = Target;
		}
		return *this;
	}

	FSimpleQuery& FSimpleQuery::All(TConstArrayView<const UScriptStruct*> Targets)
	{
		int32 NewSize = Query->ConditionTypes.Num() + Targets.Num();
		Query->ConditionTypes.Reserve(NewSize);
		Query->ConditionOperators.Reserve(NewSize);
		
		for (const UScriptStruct* Target : Targets)
		{
			All(Target);
		}
		return *this;
	}

	FSimpleQuery& FSimpleQuery::All(const FValueTag& Tag, const FName& Value)
	{
		Query->ValueTags.Emplace(
			FQueryDescription::FValueTagData
			{
				.Tag = Tag,
				.MatchValue = Value
			});
		return *this;
	}

	FSimpleQuery& FSimpleQuery::All(const UEnum& Enum)
	{
		return All(FValueTag(Enum.GetFName()));
	}

	FSimpleQuery& FSimpleQuery::All(const UEnum& Enum, int64 Value)
	{
		const FName ValueName = Enum.GetNameByValue(Value);
		if (ValueName == NAME_None)
		{
			UE_LOGF(LogEditorDataStorage, Warning, "Invalid value '%lld' for enum '%ls'", Value, *Enum.GetName());
			return *this;
		}
		return All(FValueTag(Enum.GetFName()), ValueName);
	}
	
	FSimpleQuery& FSimpleQuery::All(const FDynamicColumnDescription& Description)
	{
		Query->DynamicConditionDescriptions.Add(Description);
		Query->DynamicConditionOperations.Add(FQueryDescription::EOperatorType::SimpleAll);
		return *this;
	}

	FSimpleQuery& FSimpleQuery::All(const FValueTag& Tag)
	{
		return All(Tag, NAME_None);
	}

	FSimpleQuery& FSimpleQuery::Any(const UScriptStruct* Target)
	{
		if (Target)
		{
			Query->ConditionTypes.Add(FQueryDescription::EOperatorType::SimpleAny);
			Query->ConditionOperators.AddZeroed_GetRef().Type = Target;
		}
		return *this;
	}

	FSimpleQuery& FSimpleQuery::Any(TConstArrayView<const UScriptStruct*> Targets)
	{
		int32 NewSize = Query->ConditionTypes.Num() + Targets.Num();
		Query->ConditionTypes.Reserve(NewSize);
		Query->ConditionOperators.Reserve(NewSize);

		for (const UScriptStruct* Target : Targets)
		{
			Any(Target);
		}
		return *this;
	}

	FSimpleQuery& FSimpleQuery::Any(const FDynamicColumnDescription& Description)
	{
		Query->DynamicConditionDescriptions.Add(Description);
		Query->DynamicConditionOperations.Add(FQueryDescription::EOperatorType::SimpleAny);
		return *this;
	}

	FSimpleQuery& FSimpleQuery::None(const UScriptStruct* Target)
	{
		if (Target)
		{
			Query->ConditionTypes.Add(FQueryDescription::EOperatorType::SimpleNone);
			Query->ConditionOperators.AddZeroed_GetRef().Type = Target;
		}
		return *this;
	}

	FSimpleQuery& FSimpleQuery::None(TConstArrayView<const UScriptStruct*> Targets)
	{
		int32 NewSize = Query->ConditionTypes.Num() + Targets.Num();
		Query->ConditionTypes.Reserve(NewSize);
		Query->ConditionOperators.Reserve(NewSize);

		for (const UScriptStruct* Target : Targets)
		{
			None(Target);
		}
		return *this;
	}

	FSimpleQuery& FSimpleQuery::None(const FDynamicColumnDescription& Description)
	{
		Query->DynamicConditionDescriptions.Add(Description);
		Query->DynamicConditionOperations.Add(FQueryDescription::EOperatorType::SimpleNone);
		return *this;
	}

	// ============================================================================
	// Relation Conditions
	// ============================================================================

	FSimpleQuery& FSimpleQuery::IsSubjectOf(RelationTypeHandle RelationType)
	{
		FQueryDescription::FRelationCondition Condition;
		Condition.RelationType = RelationType;
		Condition.Role = FQueryDescription::ERelationRole::Subject;
		Condition.TargetRow = InvalidRowHandle;
		Query->RelationConditions.Add(Condition);
		return *this;
	}

	FSimpleQuery& FSimpleQuery::IsSubjectOf(RelationTypeHandle RelationType, RowHandle Object)
	{
		FQueryDescription::FRelationCondition Condition;
		Condition.RelationType = RelationType;
		Condition.Role = FQueryDescription::ERelationRole::Subject;
		Condition.TargetRow = Object;
		Query->RelationConditions.Add(Condition);
		return *this;
	}

	FSimpleQuery& FSimpleQuery::IsObjectOf(RelationTypeHandle RelationType)
	{
		FQueryDescription::FRelationCondition Condition;
		Condition.RelationType = RelationType;
		Condition.Role = FQueryDescription::ERelationRole::Object;
		Condition.TargetRow = InvalidRowHandle;
		Query->RelationConditions.Add(Condition);
		return *this;
	}

	FSimpleQuery& FSimpleQuery::IsObjectOf(RelationTypeHandle RelationType, RowHandle Subject)
	{
		FQueryDescription::FRelationCondition Condition;
		Condition.RelationType = RelationType;
		Condition.Role = FQueryDescription::ERelationRole::Object;
		Condition.TargetRow = Subject;
		Query->RelationConditions.Add(Condition);
		return *this;
	}

	// ============================================================================
	// Hierarchy Conditions
	// ============================================================================

	FSimpleQuery& FSimpleQuery::IsDescendantOf(RelationTypeHandle RelationType, RowHandle Ancestor, bool bIncludeSelf)
	{
		FQueryDescription::FHierarchyCondition Condition;
		Condition.RelationType = RelationType;
		Condition.ReferenceRow = Ancestor;
		Condition.bIncludeSelf = bIncludeSelf;
		Query->DescendantOfConditions.Add(Condition);
		return *this;
	}

	FSimpleQuery& FSimpleQuery::IsAncestorOf(RelationTypeHandle RelationType, RowHandle Descendant, bool bIncludeSelf)
	{
		FQueryDescription::FHierarchyCondition Condition;
		Condition.RelationType = RelationType;
		Condition.ReferenceRow = Descendant;
		Condition.bIncludeSelf = bIncludeSelf;
		Query->AncestorOfConditions.Add(Condition);
		return *this;
	}

	// ============================================================================
	// Relation Predicate Factory Functions
	// ============================================================================

	FQueryDescription::FRelationPredicate IsSubjectOf(RelationTypeHandle RelationType)
	{
		FQueryDescription::FRelationPredicate Pred;
		Pred.Type = FQueryDescription::FRelationPredicate::EType::SubjectOf;
		Pred.RelationType = RelationType;
		return Pred;
	}

	FQueryDescription::FRelationPredicate IsSubjectOf(RelationTypeHandle RelationType, RowHandle Object)
	{
		FQueryDescription::FRelationPredicate Pred;
		Pred.Type = FQueryDescription::FRelationPredicate::EType::SubjectOf;
		Pred.RelationType = RelationType;
		Pred.ReferenceRow = Object;
		return Pred;
	}

	FQueryDescription::FRelationPredicate IsObjectOf(RelationTypeHandle RelationType)
	{
		FQueryDescription::FRelationPredicate Pred;
		Pred.Type = FQueryDescription::FRelationPredicate::EType::ObjectOf;
		Pred.RelationType = RelationType;
		return Pred;
	}

	FQueryDescription::FRelationPredicate IsObjectOf(RelationTypeHandle RelationType, RowHandle Subject)
	{
		FQueryDescription::FRelationPredicate Pred;
		Pred.Type = FQueryDescription::FRelationPredicate::EType::ObjectOf;
		Pred.RelationType = RelationType;
		Pred.ReferenceRow = Subject;
		return Pred;
	}

	FQueryDescription::FRelationPredicate IsDescendantOf(RelationTypeHandle RelationType, RowHandle Ancestor, bool bIncludeSelf)
	{
		FQueryDescription::FRelationPredicate Pred;
		Pred.Type = FQueryDescription::FRelationPredicate::EType::DescendantOf;
		Pred.RelationType = RelationType;
		Pred.ReferenceRow = Ancestor;
		Pred.bIncludeSelf = bIncludeSelf;
		return Pred;
	}

	FQueryDescription::FRelationPredicate IsAncestorOf(RelationTypeHandle RelationType, RowHandle Descendant, bool bIncludeSelf)
	{
		FQueryDescription::FRelationPredicate Pred;
		Pred.Type = FQueryDescription::FRelationPredicate::EType::AncestorOf;
		Pred.RelationType = RelationType;
		Pred.ReferenceRow = Descendant;
		Pred.bIncludeSelf = bIncludeSelf;
		return Pred;
	}

	// ============================================================================
	// FQueryExpression operator implementations
	// ============================================================================

	namespace
	{
		/** Dispatch a FRelationPredicate to the appropriate FQueryDescription condition array. */
		static void AddPredicateToDescription(FQueryDescription& Desc, const FQueryDescription::FRelationPredicate& Pred)
		{
			using EType = FQueryDescription::FRelationPredicate::EType;
			switch (Pred.Type)
			{
				case EType::SubjectOf:
				{
					FQueryDescription::FRelationCondition Cond;
					Cond.RelationType = Pred.RelationType;
					Cond.Role = FQueryDescription::ERelationRole::Subject;
					Cond.TargetRow = Pred.ReferenceRow;
					Desc.RelationConditions.Add(Cond);
					break;
				}
				case EType::ObjectOf:
				{
					FQueryDescription::FRelationCondition Cond;
					Cond.RelationType = Pred.RelationType;
					Cond.Role = FQueryDescription::ERelationRole::Object;
					Cond.TargetRow = Pred.ReferenceRow;
					Desc.RelationConditions.Add(Cond);
					break;
				}
				case EType::DescendantOf:
				{
					FQueryDescription::FHierarchyCondition Cond;
					Cond.RelationType = Pred.RelationType;
					Cond.ReferenceRow = Pred.ReferenceRow;
					Cond.bIncludeSelf = Pred.bIncludeSelf;
					Desc.DescendantOfConditions.Add(Cond);
					break;
				}
				case EType::AncestorOf:
				{
					FQueryDescription::FHierarchyCondition Cond;
					Cond.RelationType = Pred.RelationType;
					Cond.ReferenceRow = Pred.ReferenceRow;
					Cond.bIncludeSelf = Pred.bIncludeSelf;
					Desc.AncestorOfConditions.Add(Cond);
					break;
				}
			}
		}
	}

	FQueryExpression operator&&(FColumnBase Lhs, const FQueryDescription::FRelationPredicate& Rhs)
	{
		FQueryExpression Result;
		Result.ColumnConditions = FConditions(Lhs);
		Result.RelationAtoms.Add(Rhs);
		return Result;
	}

	FQueryExpression operator&&(const FQueryDescription::FRelationPredicate& Lhs, FColumnBase Rhs)
	{
		FQueryExpression Result;
		Result.RelationAtoms.Add(Lhs);
		Result.ColumnConditions = FConditions(Rhs);
		return Result;
	}

	FQueryExpression operator&&(const FConditions& Lhs, const FQueryDescription::FRelationPredicate& Rhs)
	{
		FQueryExpression Result;
		Result.ColumnConditions = Lhs;
		Result.RelationAtoms.Add(Rhs);
		return Result;
	}

	FQueryExpression operator&&(const FQueryDescription::FRelationPredicate& Lhs, const FConditions& Rhs)
	{
		FQueryExpression Result;
		Result.RelationAtoms.Add(Lhs);
		Result.ColumnConditions = Rhs;
		return Result;
	}

	FQueryExpression operator&&(const FQueryDescription::FRelationPredicate& Lhs, const FQueryDescription::FRelationPredicate& Rhs)
	{
		FQueryExpression Result;
		Result.RelationAtoms.Add(Lhs);
		Result.RelationAtoms.Add(Rhs);
		return Result;
	}

	FQueryExpression operator&&(FQueryExpression Lhs, const FQueryDescription::FRelationPredicate& Rhs)
	{
		Lhs.RelationAtoms.Add(Rhs);
		return Lhs;
	}

	FQueryExpression operator&&(FQueryExpression Lhs, FColumnBase Rhs)
	{
		if (Lhs.ColumnConditions.IsSet())
		{
			Lhs.ColumnConditions = Lhs.ColumnConditions.GetValue() && Rhs;
		}
		else
		{
			Lhs.ColumnConditions = FConditions(Rhs);
		}
		return Lhs;
	}

	FQueryExpression operator&&(FQueryExpression Lhs, const FConditions& Rhs)
	{
		if (Lhs.ColumnConditions.IsSet())
		{
			Lhs.ColumnConditions = Lhs.ColumnConditions.GetValue() && Rhs;
		}
		else
		{
			Lhs.ColumnConditions = Rhs;
		}
		return Lhs;
	}

	FDependency FSimpleQuery::DependsOn()
	{
		return FDependency{ Query };
	}

	FQueryDescription&& FSimpleQuery::Compile()
	{
		Query->Callback.BeforeGroups.Shrink();
		Query->Callback.AfterGroups.Shrink();
		Query->SelectionTypes.Shrink();
		Query->SelectionAccessTypes.Shrink();
		for (FColumnMetaData& Metadata : Query->SelectionMetaData)
		{
			Metadata.Shrink();
		}
		Query->SelectionMetaData.Shrink();
		Query->ConditionTypes.Shrink();
		Query->ConditionOperators.Shrink();
		Query->DynamicConditionDescriptions.Shrink();
		Query->DynamicConditionOperations.Shrink();
		Query->DynamicSelectionAccessTypes.Shrink();
		Query->DynamicSelectionMetaData.Shrink();
		Query->DynamicSelectionTypes.Shrink();
		Query->DependencyTypes.Shrink();
		Query->DependencyFlags.Shrink();
		Query->CachedDependencies.Shrink();
		Query->Subqueries.Shrink();
		Query->MetaData.Shrink();
		Query->RelationConditions.Shrink();
		Query->DescendantOfConditions.Shrink();
		Query->AncestorOfConditions.Shrink();
		return MoveTemp(*Query);
	}


	/**
	 * FProcessor
	 */
	FProcessor::FProcessor(EQueryTickPhase Phase, FName Group)
		: Phase(Phase)
		, Group(Group)
	{}

	FProcessor& FProcessor::SetPhase(EQueryTickPhase NewPhase)
	{
		Phase = NewPhase;
		return *this;
	}

	FProcessor& FProcessor::SetGroup(FName GroupName)
	{
		Group = GroupName;
		return *this;
	}

	FProcessor& FProcessor::SetBeforeGroup(FName GroupName)
	{
		BeforeGroup = GroupName;
		return *this;
	}

	FProcessor& FProcessor::SetAfterGroup(FName GroupName)
	{
		AfterGroup = GroupName;
		return *this;
	}

	FProcessor& FProcessor::SetExecutionMode(EExecutionMode Mode)
	{
		ExecutionMode = Mode;
		return *this;
	}

	FProcessor& FProcessor::MakeActivatable(FName Name)
	{
		ActivationName = Name;
		return *this;
	}

	FProcessor& FProcessor::BatchModifications(bool bBatch)
	{
		bBatchModifications = bBatch;
		return *this;
	}

	/**
	 * FObserver
	 */
	
	FObserver::FObserver(EEvent MonitorForEvent, const UScriptStruct* MonitoredColumn)
		: Monitor(MonitoredColumn)
		, Event(MonitorForEvent)
	{}

	FObserver& FObserver::SetEvent(EEvent MonitorForEvent)
	{
		Event = MonitorForEvent;
		return *this;
	}

	FObserver& FObserver::SetMonitoredColumn(const UScriptStruct* MonitoredColumn)
	{
		Monitor = MonitoredColumn;
		return *this;
	}

	FObserver& FObserver::SetExecutionMode(EExecutionMode Mode)
	{
		ExecutionMode = Mode;
		return *this;
	}

	FObserver& FObserver::MakeActivatable(FName Name)
	{
		ActivationName = Name;
		return *this;
	}


	/**
	 * FPhaseAmble
	 */

	FPhaseAmble::FPhaseAmble(ELocation InLocation, EQueryTickPhase InPhase)
		: Phase(InPhase)
		, Location(InLocation)
	{}

	FPhaseAmble& FPhaseAmble::SetLocation(ELocation NewLocation)
	{
		Location = NewLocation;
		return *this;
	}

	FPhaseAmble& FPhaseAmble::SetPhase(EQueryTickPhase NewPhase)
	{
		Phase = NewPhase;
		return *this;
	}

	FPhaseAmble& FPhaseAmble::SetExecutionMode(EExecutionMode Mode)
	{
		ExecutionMode = Mode;
		return *this;
	}

	FPhaseAmble& FPhaseAmble::MakeActivatable(FName Name)
	{
		ActivationName = Name;
		return *this;
	}


	/**
	 * Select
	 */

	FDependency FQueryConditionQuery::DependsOn()
	{
		return FDependency{ Query };
	}

	FQueryDescription&& FQueryConditionQuery::Compile()
	{
		return MoveTemp(*Query);
	}

	FQueryConditionQuery::FQueryConditionQuery(FQueryDescription* InQuery)
		: Query(InQuery)
	{
		
	}

	Select::Select()
	{
		Query.Action = FQueryDescription::EActionType::Select;
	}
	
	Select& Select::ReadOnly(const UScriptStruct* Target)
	{
		checkf(Target, TEXT("The Select section in the Typed Elements query builder doesn't support nullptrs as Read-Only input."));
		Query.SelectionTypes.Emplace(Target);
		Query.SelectionAccessTypes.Emplace(EQueryAccessType::ReadOnly);
		Query.SelectionMetaData.Emplace(Target, FColumnMetaData::EFlags::None);
		
		return *this;
	}

	Select& Select::ReadOnly(TConstArrayView<const UScriptStruct*> Targets)
	{
		int32 NewCount = Query.SelectionTypes.Num() + Targets.Num();
		Query.SelectionTypes.Reserve(NewCount);
		Query.SelectionAccessTypes.Reserve(NewCount);
		Query.SelectionMetaData.Reserve(NewCount);

		for (const UScriptStruct* Target : Targets)
		{
			ReadOnly(Target);
		}
		return *this;
	}

	Select& Select::ReadOnly(const FDynamicColumnDescription& Description)
	{
		Query.DynamicSelectionTypes.Emplace(Description);
		Query.DynamicSelectionAccessTypes.Emplace(EQueryAccessType::ReadOnly);
		Query.DynamicSelectionMetaData.Emplace(FColumnMetaData::EFlags::None);
		return *this;
	}

	Select& Select::ReadOnly(const UScriptStruct* Target, EOptional Optional)
	{
		checkf(Target, TEXT("The Select section in the Typed Elements query builder doesn't support nullptrs as Read-Only input."));
		Query.SelectionTypes.Emplace(Target);
		Query.SelectionAccessTypes.Emplace(Optional == EOptional::Yes
			? EQueryAccessType::OptionalReadOnly
			: EQueryAccessType::ReadOnly);
		Query.SelectionMetaData.Emplace(Target, FColumnMetaData::EFlags::None);

		return *this;
	}

	Select& Select::ReadOnly(TConstArrayView<const UScriptStruct*> Targets, EOptional Optional)
	{
		int32 NewCount = Query.SelectionTypes.Num() + Targets.Num();
		Query.SelectionTypes.Reserve(NewCount);
		Query.SelectionAccessTypes.Reserve(NewCount);
		Query.SelectionMetaData.Reserve(NewCount);

		for (const UScriptStruct* Target : Targets)
		{
			ReadOnly(Target, Optional);
		}
		return *this;
	}

	Select& Select::ReadWrite(const UScriptStruct* Target)
	{
		checkf(Target, TEXT("The Select section in the Typed Elements query builder doesn't support nullptrs as Read/Write input."));
		Query.SelectionTypes.Emplace(Target);
		Query.SelectionAccessTypes.Emplace(EQueryAccessType::ReadWrite);
		Query.SelectionMetaData.Emplace(Target, FColumnMetaData::EFlags::IsMutable);
		return *this;
	}

	Select& Select::ReadWrite(TConstArrayView<const UScriptStruct*> Targets)
	{
		int32 NewCount = Query.SelectionTypes.Num() + Targets.Num();
		Query.SelectionTypes.Reserve(NewCount);
		Query.SelectionAccessTypes.Reserve(NewCount);
		Query.SelectionMetaData.Reserve(NewCount);

		for (const UScriptStruct* Target : Targets)
		{
			ReadWrite(Target);
		}
		return *this;
	}

	Select& Select::ReadWrite(const FDynamicColumnDescription& Description)
	{
		if (ensureMsgf(!Description.Identifier.IsNone(), TEXT("Cannot pass special identifier None to select a specific dynamic column")))
		{
			Query.DynamicSelectionTypes.Emplace(Description);
			Query.DynamicSelectionAccessTypes.Emplace(EQueryAccessType::ReadWrite);
			Query.DynamicSelectionMetaData.Emplace(FColumnMetaData::EFlags::IsMutable);
		}
		return *this;
	}

	FSimpleQuery Select::Where()
	{
		return FSimpleQuery{ &Query };
	}

	FDependency Select::DependsOn()
	{
		return FDependency{ &Query };
	}

	Select& Select::AccessesHierarchy(const FName& HierarchyName)
	{
		Query.Hierarchy = HierarchyName;
		return *this;
	}

	FQueryConditionQuery Select::Where(Queries::FConditions Condition)
	{
		Query.Conditions.Emplace(MoveTemp(Condition));
		return FQueryConditionQuery(&Query);
	}

	FQueryConditionQuery Select::Where(Queries::FQueryExpression Expression)
	{
		if (Expression.ColumnConditions.IsSet())
		{
			Query.Conditions.Emplace(MoveTemp(Expression.ColumnConditions.GetValue()));
		}
		for (FQueryDescription::FRelationPredicate& Pred : Expression.RelationAtoms)
		{
			AddPredicateToDescription(Query, Pred);
		}
		return FQueryConditionQuery(&Query);
	}

	FQueryDescription&& Select::Compile()
	{
		return MoveTemp(Query);
	}


	/**
	 * Count
	 */

	Count::Count()
	{
		Query.Action = FQueryDescription::EActionType::Count;
	}

	FSimpleQuery Count::Where()
	{
		return FSimpleQuery{ &Query };
	}

	FDependency Count::DependsOn()
	{
		return FDependency{ &Query };
	}
	
	/**
	 * MergeQueries
	 */

	namespace Private
	{
		// Enum to get the result when two TEDS operators need to be tested against each other
		enum EConflictResolution
		{
			First, // The first operator is stronger
			Second, // The second operator is stronger
			Conflict, // There is a conflict between the two operators
			Unknown // An unknown operator was found
		};

		EConflictResolution Resolve(EQueryAccessType First, EQueryAccessType Second)
		{
			switch (First)
			{
			case EQueryAccessType::ReadWrite:

				// ReadWrite is the strongest access type
				return EConflictResolution::First;
				
			case EQueryAccessType::ReadOnly:

				// ReadWrite is a stronger access type than ReadOnly
				return Second == EQueryAccessType::ReadWrite ? EConflictResolution::Second : EConflictResolution::First;
				
			case EQueryAccessType::OptionalReadOnly:

				// OptionalReadOnly is the weakest access type
				return EConflictResolution::Second;
				
			default:
				checkf(false, TEXT("Unknown EQueryAccessType value found when trying to resolve conditions."))
				return EConflictResolution::Unknown;
			}
		}
		
		EConflictResolution Resolve(FQueryDescription::EOperatorType First, FQueryDescription::EOperatorType Second)
		{
			switch (First)
			{
			case FQueryDescription::EOperatorType::SimpleAll:

				// All is the strongest operator, but it conflicts with None
				return Second == FQueryDescription::EOperatorType::SimpleNone ? EConflictResolution::Conflict : EConflictResolution::First;
				
			case FQueryDescription::EOperatorType::SimpleAny:

				// Any conflicts with None, and is weaker than All
				if (Second == FQueryDescription::EOperatorType::SimpleNone)
				{
					return EConflictResolution::Conflict;
				}
				else if (Second == FQueryDescription::EOperatorType::SimpleAll)
				{
					return EConflictResolution::Second;
				}

				return EConflictResolution::First;
				
			case FQueryDescription::EOperatorType::SimpleNone:

				// None conflicts with everything except itself
				return Second != FQueryDescription::EOperatorType::SimpleNone ? EConflictResolution::Conflict : EConflictResolution::First;
				
			
			case FQueryDescription::EOperatorType::SimpleOptional:
				
				// SimpleOptional is currently unused, so we'll treat it as the weakest
				return EConflictResolution::Second;
				
			default:
				checkf(false, TEXT("Unknown EOperatorType value found when trying to resolve conditions."))
				return EConflictResolution::Unknown;
			}
		}
		
		EConflictResolution Resolve(EQueryAccessType First, FQueryDescription::EOperatorType Second)
		{
			switch (First)
			{
			case EQueryAccessType::ReadWrite:

				// Selection takes precedence over conditions, but ReadWrite conflicts with SimpleNone
				return Second == FQueryDescription::EOperatorType::SimpleNone ? EConflictResolution::Conflict : EConflictResolution::First;
				
			case EQueryAccessType::ReadOnly:

				// Selection takes precedence over conditions, but ReadOnly conflicts with SimpleNone
				return Second == FQueryDescription::EOperatorType::SimpleNone ? EConflictResolution::Conflict : EConflictResolution::First;
				
			case EQueryAccessType::OptionalReadOnly:

				// OptionalReadOnly conflicts with a SimpleNone
				if (Second == FQueryDescription::EOperatorType::SimpleNone)
				{
					return EConflictResolution::Conflict;
				}

				// OptionalReadOnly is weaker than SimpleAll or SimpleAny, but stronger than SimpleOptional because selection wins over conditions
				// NOTE: SimpleOptional is currently unused
				return Second == FQueryDescription::EOperatorType::SimpleOptional ?	EConflictResolution::First : EConflictResolution::Second;
				
			default:
				checkf(false, TEXT("Unknown EQueryAccessType value found when trying to resolve conditions."))
				return EConflictResolution::Unknown;
			}
		}

		void RemoveSelection(FQueryDescription& Query, int32 SelectionIndex, bool bDynamic)
		{
			if (bDynamic)
			{
				Query.DynamicSelectionAccessTypes.RemoveAt(SelectionIndex);
				Query.DynamicSelectionTypes.RemoveAt(SelectionIndex);
				Query.DynamicSelectionMetaData.RemoveAt(SelectionIndex);
			}
			else
			{
				Query.SelectionAccessTypes.RemoveAt(SelectionIndex);
				Query.SelectionTypes.RemoveAt(SelectionIndex);
				Query.SelectionMetaData.RemoveAt(SelectionIndex);
			}
		};

		void RemoveCondition(FQueryDescription& Query, int32 ConditionIndex, bool bDynamic)
		{
			if (bDynamic)
			{
				Query.DynamicConditionOperations.RemoveAt(ConditionIndex);
				Query.DynamicConditionDescriptions.RemoveAt(ConditionIndex);
			}
			else
			{
				Query.ConditionOperators.RemoveAt(ConditionIndex);
				Query.ConditionTypes.RemoveAt(ConditionIndex);
			}
		};
		
		// Util function to resolve duplicate columns/tags at a specific index inside the Select() part of a query
		void ResolveSelectDuplicate(FQueryDescription& Query, int32 OriginalIndex, int32 DuplicateIndex, bool bDynamic)
		{
			EQueryAccessType OriginalAccessType = bDynamic ?
				Query.DynamicSelectionAccessTypes[OriginalIndex] : Query.SelectionAccessTypes[OriginalIndex];
			
			EQueryAccessType DuplicateAccessType = bDynamic ? Query.DynamicSelectionAccessTypes[DuplicateIndex] : Query.SelectionAccessTypes[DuplicateIndex];

			// Remove the weaker of the two operators to resolve the duplicate
			switch (Resolve(OriginalAccessType, DuplicateAccessType))
			{
			case EConflictResolution::First:
				RemoveSelection(Query, DuplicateIndex, bDynamic);
				break;
				
			case EConflictResolution::Second:
				RemoveSelection(Query, OriginalIndex, bDynamic);
				break;

			// Two Selects can never conflict with each other, something went wrong (i.e one of them was unknown)
			case EConflictResolution::Conflict:
				checkf(false, TEXT("Two Selects() cannot have a conflict as it only supports simple requirements, one of the queries is probably invalid."))
				break;
			case EConflictResolution::Unknown:
			default:

				checkf(false, TEXT("Unknown EQueryAccessType value found when trying to merge queries."))
				break;
			}
		}
		
		// Util function to resolve duplicate columns/tags at a specific index inside the conditions of two queries
		bool ResolveConditionDuplicate(FQueryDescription& Query, int32 OriginalIndex, int32 DuplicateIndex, bool bDynamic, FText* OutErrorMessage)
		{
			FQueryDescription::EOperatorType OriginalOperatorType = bDynamic ?
				Query.DynamicConditionOperations[OriginalIndex] : Query.ConditionTypes[OriginalIndex];

			FQueryDescription::EOperatorType DuplicateOperatorType = bDynamic ?
				Query.DynamicConditionOperations[DuplicateIndex] : Query.ConditionTypes[DuplicateIndex];
			
			// Remove the weaker of the two conditions, or report an error if they conflict
			switch (Resolve(OriginalOperatorType, DuplicateOperatorType))
			{
			case EConflictResolution::First:
				RemoveCondition(Query, DuplicateIndex, bDynamic);
				return true;
				
			case EConflictResolution::Second:
				RemoveCondition(Query, OriginalIndex, bDynamic);
				return true;

			case EConflictResolution::Conflict:

				if (OutErrorMessage)
				{
					*OutErrorMessage = LOCTEXT("CannotMerge_ConditionConflict",
						"Cannot merge two queries if they contain the same requirement under conflicting conditions (None and All/Any)");
				}

				return false;
			case EConflictResolution::Unknown:
			default:

				checkf(false, TEXT("Unknown EOperatorType value found when trying to merge queries."))
				return false;
			}
		}

		// Util function to resolve duplicates between a condition type and a selection type on a single query
		bool ResolveConditionSelectionDuplicate(FQueryDescription& Query, int32 SelectionIndex, int32 ConditionIndex, bool bDynamic, FText* OutErrorMessage)
		{
			EQueryAccessType SelectionAccessType = bDynamic ? Query.DynamicSelectionAccessTypes[SelectionIndex] : Query.SelectionAccessTypes[SelectionIndex];

			FQueryDescription::EOperatorType ConditionOperatorType = bDynamic ?
				Query.DynamicConditionOperations[ConditionIndex] : Query.ConditionTypes[ConditionIndex];

			// Remove the weaker of the two, or report any conflicts
			switch (Resolve(SelectionAccessType, ConditionOperatorType))
			{
			case EConflictResolution::First:
				RemoveCondition(Query, ConditionIndex, bDynamic);
				return true;
				
			case EConflictResolution::Second:
				RemoveSelection(Query, SelectionIndex, bDynamic);
				return true;

			case EConflictResolution::Conflict:

				if (OutErrorMessage)
				{
					*OutErrorMessage = LOCTEXT("CannotMerge_SelectAndConditionConflict",
						"Cannot merge two queries if they contain conflicting requirements in the Select() and Where() clauses");
				}
				return false;
				
			case EConflictResolution::Unknown:
			default:

				checkf(false, TEXT("Unknown EOperatorType value found when trying to merge queries."))
				return false;
			}
		}

		// Resolve any duplicates or conflicts that may have resulted from the merge
		bool ResolveDuplicatesInQuery(FQueryDescription& Query, FText* OutErrorMessage)
		{
			// NOTE: We don't need to check the FConditions because they currently don't support NOT and cannot conflict, and the query matching logic
			// for them is also overriden by TEDS so they can work with duplicates

			// Check the selection types
			for (int32 SelectionIndex = 0; SelectionIndex < Query.SelectionTypes.Num(); SelectionIndex++)
			{
				// Check if the same selection type is present twice in the query.
				// Using FindLastByPredicate is fine because we can only have two copies of the same type
				int32 DuplicateIndex = Query.SelectionTypes.FindLastByPredicate([&Query, SelectionIndex](const TWeakObjectPtr<const UScriptStruct>& SelectionType)
				{
					return SelectionType == Query.SelectionTypes[SelectionIndex];
				});

				if (DuplicateIndex != INDEX_NONE && DuplicateIndex != SelectionIndex)
				{
					ResolveSelectDuplicate(Query, SelectionIndex, DuplicateIndex, false);
				}

				// Check if the selection type is present in the conditions of the query
				int32 ConditionDuplicateIndex = Query.ConditionOperators.IndexOfByPredicate([&Query, SelectionIndex](const FQueryDescription::FOperator& Operator)
				{
					return Operator.Type == Query.SelectionTypes[SelectionIndex];
				});

				if (ConditionDuplicateIndex != INDEX_NONE)
				{
					if (!ResolveConditionSelectionDuplicate(Query, SelectionIndex, ConditionDuplicateIndex, false, OutErrorMessage))
					{
						return false;
					}
				}
			}

			// Do the same for the dynamic selection types
			for (int32 SelectionIndex = 0; SelectionIndex < Query.DynamicSelectionTypes.Num(); SelectionIndex++)
			{
				// Check if the same selection type is present twice in the query.
				// Using FindLastByPredicate is fine because we can only have two copies of the same type
				int32 DuplicateIndex = Query.DynamicSelectionTypes.FindLastByPredicate([&Query, SelectionIndex](const FDynamicColumnDescription& Description)
				{
					return Description == Query.DynamicSelectionTypes[SelectionIndex];
				});

				if (DuplicateIndex != INDEX_NONE && DuplicateIndex != SelectionIndex)
				{
					ResolveSelectDuplicate(Query, SelectionIndex, DuplicateIndex, true);
				}

				// Check if the selection type is present in the conditions of the query
				int32 ConditionDuplicateIndex = Query.DynamicConditionDescriptions.IndexOfByPredicate([&Query, SelectionIndex](const FDynamicColumnDescription& Description)
				{
					return Description == Query.DynamicSelectionTypes[SelectionIndex];
				});

				if (ConditionDuplicateIndex != INDEX_NONE)
				{
					if (!ResolveConditionSelectionDuplicate(Query, SelectionIndex, ConditionDuplicateIndex, true, OutErrorMessage))
					{
						return false;
					}
				}
			}

			// Check if the same condition operator is present twice in the query.
			for (int32 ConditionIndex = 0; ConditionIndex < Query.ConditionOperators.Num(); ConditionIndex++)
			{
				// NOTE: We don't need to test against selection again because the selection loops handle that

				// Using FindLastByPredicate is fine because we can only have two copies of the same type
				int32 ConditionDuplicateIndex = Query.ConditionOperators.FindLastByPredicate([&Query, ConditionIndex](const FQueryDescription::FOperator& Operator)
				{
					return Operator.Type == Query.ConditionOperators[ConditionIndex].Type;
				});

				if (ConditionDuplicateIndex != INDEX_NONE && ConditionDuplicateIndex != ConditionIndex)
				{
					if (!ResolveConditionDuplicate(Query, ConditionIndex, ConditionDuplicateIndex, false, OutErrorMessage))
					{
						return false;
					}
				}
			}

			// Check if the same dynamic condition operator is present twice in the query.
			for (int32 ConditionIndex = 0; ConditionIndex < Query.DynamicConditionDescriptions.Num(); ConditionIndex++)
			{
				// NOTE: We don't need to test against selection again because the selection loops handle that

				// Using FindLastByPredicate is fine because we can only have two copies of the same type
				int32 ConditionDuplicateIndex = Query.DynamicConditionDescriptions.FindLastByPredicate([&Query, ConditionIndex](const FDynamicColumnDescription& Description)
				{
					return Description == Query.DynamicConditionDescriptions[ConditionIndex];
				});

				if (ConditionDuplicateIndex != INDEX_NONE && ConditionDuplicateIndex != ConditionIndex)
				{
					if (!ResolveConditionDuplicate(Query, ConditionIndex, ConditionDuplicateIndex, true, OutErrorMessage))
					{
						return false;
					}
				}
			}
			return true;
		}

		// Util function for some upfront checks to see if the two queries can be merged
		bool AreMergeablePrecheck(const FQueryDescription& Destination, const FQueryDescription& Source, FText* OutErrorMessage)
		{
			if (Destination.Callback.Function.IsSet() && Source.Callback.Function.IsSet())
			{
				if (OutErrorMessage)
				{
					*OutErrorMessage = LOCTEXT("CannotMerge_Callbacks", "Cannot merge two queries if both contain a callback");
				}
				return false;
			}

			// If one of the queries is using FConditions and the other one has anything in the old Where() syntax, they are incompatible
			if ((Destination.Conditions && Source.ConditionTypes.Num()) || (Source.Conditions && Destination.ConditionTypes.Num()))
			{
				if (OutErrorMessage)
				{
					*OutErrorMessage = LOCTEXT("CannotMerge_FConditions",
						"Cannot merge two queries if one of them uses the FConditionsSyntax and the other uses the legacy .All/.Any/.None syntax");
				}
				return false;
			}

			if (!Destination.MetaData.IsEmpty() && !Source.MetaData.IsEmpty())
			{
				if (OutErrorMessage)
				{
					*OutErrorMessage = LOCTEXT("CannotMerge_Metadata",
						"Cannot merge two queries if they both contain metadata, since FMetaData currently does not support merging");
				}
				return false;
			}

			// If both queries have a condition in .Any<>, they cannot be merged as combining .Any<> conditions results in an OR operation
			// e.g .Any<ColumnA, ColumnB> + .Any<ColumnC, ColumnD> results in .Any<ColumnA, ColumnB, ColumnC, ColumnD> which is incorrect since
			// the expected result is (ColumnA || ColumnB) && (ColumnC || ColumnD)
			auto HasAnyCondition = [](const FQueryDescription& QueryDescription)
			{
				return QueryDescription.ConditionTypes.FindByKey(FQueryDescription::EOperatorType::SimpleAny)
					|| QueryDescription.DynamicConditionOperations.FindByKey(FQueryDescription::EOperatorType::SimpleAny);
			};

			if (HasAnyCondition(Source) && HasAnyCondition(Destination))
			{
				if (OutErrorMessage)
				{
					*OutErrorMessage = LOCTEXT("CannotMerge_Any",
						"Cannot merge two queries if they both .Any<> conditions, use the FConditions syntax instead to merge them properly");
				}
				return false;
			}
			
			return true;
		}

		// Merge the Select() section of the two queries
		void MergeSelect(FQueryDescription& Destination, const FQueryDescription& Source)
		{
			Destination.SelectionTypes.Append(Source.SelectionTypes);
			Destination.SelectionAccessTypes.Append(Source.SelectionAccessTypes);
			Destination.SelectionMetaData.Append(Source.SelectionMetaData);
			
			Destination.DynamicSelectionTypes.Append(Source.DynamicSelectionTypes);
			Destination.DynamicSelectionAccessTypes.Append(Source.DynamicSelectionAccessTypes);
			Destination.DynamicSelectionMetaData.Append(Source.DynamicSelectionMetaData);
		}

		// Merge the Where() section of the two queries
		void MergeConditions(FQueryDescription& Destination, const FQueryDescription& Source)
		{
			Destination.ConditionOperators.Append(Source.ConditionOperators);
			Destination.ConditionTypes.Append(Source.ConditionTypes);
			
			Destination.DynamicConditionDescriptions.Append(Source.DynamicConditionDescriptions);
			Destination.DynamicConditionOperations.Append(Source.DynamicConditionOperations);
		}
	}
	
	bool MergeQueries(FQueryDescription& Destination, const FQueryDescription& Source, FText* OutErrorMessage)
	{
		if (!Private::AreMergeablePrecheck(Destination, Source, OutErrorMessage))
		{
			return false;
		}

		// Keep a copy of the original destination in case we detect that the merge is not possible partway through the process (e.g conflicting conditions)
		FQueryDescription OriginalDestination = Destination;

		// If we are combining a select and a count query, the final query will be treated as a select
		Destination.Action = Source.Action == FQueryDescription::EActionType::Select ? Source.Action : Destination.Action;

		// If we are at this point we are guaranteed only one of the two queries has a callback
		if (Source.Callback.Function.IsSet())
		{
			Destination.Callback.Function = Source.Callback.Function;
			Destination.bShouldBatchModifications = Source.bShouldBatchModifications;
		}

		// Copy over the trivially copyable data
		Destination.ValueTags.Append(Source.ValueTags);
		Destination.DependencyTypes.Append(Source.DependencyTypes);
		Destination.DependencyFlags.Append(Source.DependencyFlags);
		Destination.CachedDependencies.Append(Source.CachedDependencies);
		Destination.Subqueries.Append(Source.Subqueries);

		// Merge the Select() portion of the queries
		Private::MergeSelect(Destination, Source);

		// Since the FConditions cannot co-exist with the ConditionOperators because of the pre-check, we only need to merge one of the two
		if (Destination.Conditions || Source.Conditions)
		{
			Destination.Conditions = Destination.Conditions.Get(FConditions()) && Source.Conditions.Get(FConditions());
		}
		else
		{
			Private::MergeConditions(Destination, Source);
		}

		// Validate the merged query
		if (!Private::ResolveDuplicatesInQuery(Destination, OutErrorMessage))
		{
			// Restore the destination back to the original value if the merge failed
			Destination = OriginalDestination;
			return false;
		}
		
		return true;
	}
} // namespace UE::Editor::DataStorage::Queries

#undef LOCTEXT_NAMESPACE