// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/TypedElementRegistry.h"

#include "DataStorage/Features.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ScopeLock.h"
#include "UObject/StrongObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementRegistry)

const FTypedElementId FTypedElementId::Unset;

#if UE_TYPED_ELEMENT_HAS_REFTRACKING
namespace TypedElementReferences
{
static int32 GEnableReferenceTracking = 0;
static FAutoConsoleVariableRef CVarEnableReferenceTracking(
	TEXT("TypedElements.EnableReferenceTracking"),
	GEnableReferenceTracking,
	TEXT("Is support for element reference tracking enabled?")
	);
} // namespace TypedElementReference

bool FTypedElementReferences::ReferenceTrackingEnabled()
{
	return TypedElementReferences::GEnableReferenceTracking != 0;
}
#endif	// UE_TYPED_ELEMENT_HAS_REFTRACKING

TStrongObjectPtr<UTypedElementRegistry>& GetTypedElementRegistryInstance()
{
	static TStrongObjectPtr<UTypedElementRegistry> TypedElementRegistryInstance;
	return TypedElementRegistryInstance;
}

UTypedElementRegistry::UTypedElementRegistry()
{
	LLM_SCOPE(ELLMTag::EngineMisc);
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		FCoreDelegates::OnBeginFrame.AddUObject(this, &UTypedElementRegistry::OnBeginFrame);
		FCoreDelegates::OnEndFrame.AddUObject(this, &UTypedElementRegistry::OnEndFrame);
		FCoreUObjectDelegates::GetPostGarbageCollect().AddUObject(this, &UTypedElementRegistry::OnPostGarbageCollect);
	}
}

void UTypedElementRegistry::Private_InitializeInstance()
{
	TStrongObjectPtr<UTypedElementRegistry>& Instance = GetTypedElementRegistryInstance();
	checkf(!Instance, TEXT("Instance was already initialized!"));
	Instance.Reset(NewObject<UTypedElementRegistry>());
}

void UTypedElementRegistry::Private_ShutdownInstance()
{
	TStrongObjectPtr<UTypedElementRegistry>& Instance = GetTypedElementRegistryInstance();
	Instance.Reset();
}

UTypedElementRegistry* UTypedElementRegistry::GetInstance()
{
	TStrongObjectPtr<UTypedElementRegistry>& Instance = GetTypedElementRegistryInstance();
	return Instance.Get();
}

void UTypedElementRegistry::FinishDestroy()
{
	TStrongObjectPtr<UTypedElementRegistry>& Instance = GetTypedElementRegistryInstance();
	if (Instance.Get() == this)
	{
		Instance.Reset();
	}

	ProcessDeferredElementsToDestroy();

	Super::FinishDestroy();
}

void UTypedElementRegistry::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UTypedElementRegistry* This = CastChecked<UTypedElementRegistry>(InThis);

	UE::TReadScopeLock RegisteredElementTypesLock(This->RegisteredElementTypesRW);
	
	for (TUniquePtr<FRegisteredElementType>& RegisteredElementType : This->RegisteredElementTypes)
	{
		if (RegisteredElementType)
		{
			for (auto& InterfacesPair : RegisteredElementType->Interfaces)
			{
				Collector.AddReferencedObject(InterfacesPair.Value);
			}
		}
	}
}

void UTypedElementRegistry::RegisterElementTypeImpl(const FName InElementTypeName, TUniquePtr<FRegisteredElementType>&& InRegisteredElementType)
{
	// Query whether this type has previously been registered in any type registry, and if so re-use that ID
	// If not (or if the element is typeless) then assign the next available ID
	FTypedHandleTypeId TypeId = InRegisteredElementType->GetDataTypeId();
	if (TypeId == 0)
	{
		static FCriticalSection NextTypeIdCS;
		static FTypedHandleTypeId NextTypeId = 1;

		FScopeLock NextTypeIdLock(&NextTypeIdCS);

		checkf(NextTypeId <= TypedHandleMaxTypeId, TEXT("Ran out of typed element type IDs!"));

		TypeId = NextTypeId++;
		InRegisteredElementType->SetDataTypeId(TypeId);
	}

	InRegisteredElementType->TypeId = TypeId;
	InRegisteredElementType->TypeName = InElementTypeName;
	AddRegisteredElementType(MoveTemp(InRegisteredElementType));
}

void UTypedElementRegistry::RegisterElementInterfaceImpl(const FName InElementTypeName, UObject* InElementInterface, const TSubclassOf<UInterface>& InBaseInterfaceType, const bool InAllowOverride)
{
	LLM_SCOPE(ELLMTag::EngineMisc);

	checkf(InElementInterface->GetClass()->ImplementsInterface(InBaseInterfaceType), TEXT("Interface '%s' of type '%s' does not derive from '%s'!"), *InElementInterface->GetPathName(), *InElementInterface->GetClass()->GetName(), *InBaseInterfaceType->GetName());

	FRegisteredElementType* RegisteredElementType = GetRegisteredElementTypeFromName(InElementTypeName);
	checkf(RegisteredElementType, TEXT("Element type '%s' has not been registered!"), *InElementTypeName.ToString());

	checkf(InAllowOverride || !RegisteredElementType->Interfaces.Contains(InBaseInterfaceType->GetFName()), TEXT("Element type '%s' has already registered an interface for '%s'!"), *InElementTypeName.ToString(), *InBaseInterfaceType->GetName());
	RegisteredElementType->Interfaces.Add(InBaseInterfaceType->GetFName(), ObjectPtrWrap(InElementInterface));
}

UObject* UTypedElementRegistry::GetElementInterfaceImpl(const FTypedHandleTypeId InElementTypeId, const TSubclassOf<UInterface>& InBaseInterfaceType) const
{
	if (!InElementTypeId)
	{
		return nullptr;
	}

	FRegisteredElementType* RegisteredElementType = GetRegisteredElementTypeFromId(InElementTypeId);
	checkf(RegisteredElementType, TEXT("Element type ID '%d' has not been registered!"), InElementTypeId);

	return RegisteredElementType->Interfaces.FindRef(InBaseInterfaceType->GetFName());
}

void UTypedElementRegistry::ProcessDeferredElementsToDestroy()
{
	OnProcessingDeferredElementsToDestroyDelegate.Broadcast();

	UE::TReadScopeLock RegisteredElementTypesLock(RegisteredElementTypesRW);

	LLM_SCOPE(ELLMTag::EngineMisc);
	for (TUniquePtr<FRegisteredElementType>& RegisteredElementType : RegisteredElementTypes)
	{
		if (RegisteredElementType)
		{
			RegisteredElementType->ProcessDeferredElementsToRemove();
		}
	}
}

void UTypedElementRegistry::ReleaseElementId(FTypedElementId& InOutElementId)
{
	if (!InOutElementId)
	{
		return;
	}

	FRegisteredElementType* RegisteredElementType = GetRegisteredElementTypeFromId(InOutElementId.GetTypeId());
	checkf(RegisteredElementType, TEXT("Element type ID '%d' has not been registered!"), InOutElementId.GetTypeId());

	const FTypedElementInternalData& ElementData = RegisteredElementType->GetDataForElement(InOutElementId.GetElementId());
	ElementData.ReleaseRef(INDEX_NONE); // Cannot track element ID references as we have no space to store the reference ID

	InOutElementId.Private_DestroyNoRef();
}

FTypedElementHandle UTypedElementRegistry::GetElementHandle(const FTypedElementId& InElementId) const
{
	if (!InElementId)
	{
		return FTypedElementHandle();
	}

	FRegisteredElementType* RegisteredElementType = GetRegisteredElementTypeFromId(InElementId.GetTypeId());
	checkf(RegisteredElementType, TEXT("Element type ID '%d' has not been registered!"), InElementId.GetTypeId());

	FTypedElementHandle ElementHandle;
	ElementHandle.Private_InitializeAddRef(RegisteredElementType->GetDataForElement(InElementId.GetElementId()));

	return ElementHandle;
}

FTypedElementListRef UTypedElementRegistry::CreateElementList(TArrayView<const FTypedElementId> InElementIds)
{
	FTypedElementListRef ElementList = CreateElementList();

	for (const FTypedElementId& ElementId : InElementIds)
	{
		if (FTypedElementHandle ElementHandle = GetElementHandle(ElementId))
		{
			ElementList->Add(MoveTemp(ElementHandle));
		}
	}

	return ElementList;
}

FTypedElementListRef UTypedElementRegistry::CreateElementList(TArrayView<const FTypedElementHandle> InElementHandles)
{
	FTypedElementListRef ElementList = CreateElementList();
	ElementList->Append(InElementHandles);
	return ElementList;
}

FString UTypedElementRegistry::RegistredElementTypesAndInterfacesToString() const
{
	TArray<FString> Lines;

	{
		UE::TReadScopeLock RegisteredElementTypesLock(RegisteredElementTypesRW);
		Lines.Reserve(RegisteredElementTypesNameToId.Num());

		{
			FStringFormatOrderedArguments FormatArguments;
			FormatArguments.Add(GetPathName());
			Lines.Add(FString::Format(TEXT("Registred Type Elements and their Interfaces for the registry: {0}"), FormatArguments));
		}

		for (const TPair<FName, FTypedHandleTypeId>& TypePair : RegisteredElementTypesNameToId)
		{
			if (FRegisteredElementType* RegistredElementType = GetRegisteredElementTypeFromId(TypePair.Value))
			{
				Lines.Reserve(RegistredElementType->Interfaces.Num() + 1 + Lines.Num());
				{
					FStringFormatOrderedArguments FormatArguments;
					FormatArguments.Add(TypePair.Key.ToString());
					Lines.Add(FString::Format(TEXT("	Type: {0}"), FormatArguments));
				}
				for (auto& InterfacePair : RegistredElementType->Interfaces)
				{
					FStringFormatOrderedArguments FormatArguments;
					FormatArguments.Reserve(2);
					FormatArguments.Add(InterfacePair.Key.ToString());
					FormatArguments.Add(InterfacePair.Value->GetClass()->GetPathName());
					Lines.Add(FString::Format(TEXT("		Interface {0} is implemented by {1}"), FormatArguments));
				}
			}
		}

		FString Output;
		int32 OutputSize = 0;
		for (const FString& Line : Lines)
		{
			OutputSize += Line.Len();
			// for the line termination char
			++OutputSize;
		}

		Output.Reserve(OutputSize);
		for (const FString& Line : Lines)
		{
			Output += Line;
			Output.AppendChar(TEXT('\n'));
		}

		return Output;
	}

}

void UTypedElementRegistry::NotifyElementListPendingChanges()
{
	{
		/** 
		 * We use a critical section here since we need the called function to be able to create or delete TypedElementLists.
		 * Critical sections are recursive. They can be lock multiple times by the same thread without blocking.
		 */
		FScopeLock ActiveElementListsLock(&ActiveElementListsCS);
		TArray<FTypedElementList*> ElementListToNotify = ActiveElementLists.Array();

		bool bHasListPotentialyChanged = false;
		for (FTypedElementList* ActiveElementList : ElementListToNotify)
		{
			if (bHasListPotentialyChanged)
			{
				/**
				 * One of the callbacks could have modified the ActiveElementLists by deleting a ElementList.
				 * So we need to validate that the ActiveElementList still exist.
				 */
				if (ActiveElementLists.Contains(ActiveElementList))
				{
					ActiveElementList->NotifyPendingChanges();
				}
			}
			else
			{
				bHasListPotentialyChanged = ActiveElementList->NotifyPendingChanges();
			}
		}
	}

	// Script List (no need for a critical section)
	bool bHasListPotentialyChanged = false;
	TArray<FScriptTypedElementList*> ElementListToNotify = ActiveScriptElementLists.Array();
	for (FScriptTypedElementList* ActiveElementList : ElementListToNotify)
	{
		if (bHasListPotentialyChanged)
		{
			/**
				* One of the callbacks could have modified the ActiveElementLists by deleting a ElementList.
				* So we need to validate that the ActiveElementList still exist.
				*/
			if (ActiveScriptElementLists.Contains(ActiveElementList))
			{
				ActiveElementList->NotifyPendingChanges();
			}
		}
		else
		{
			bHasListPotentialyChanged = ActiveElementList->NotifyPendingChanges();
		}
	}
}

void UTypedElementRegistry::OnBeginFrame()
{
	// Prevent auto-GC reference collection during this frame
	IncrementDisableElementDestructionOnGCCount();
	bIsWithinFrame = true;
}

void UTypedElementRegistry::OnEndFrame()
{
	NotifyElementListPendingChanges();
	ProcessDeferredElementsToDestroy();

	if (bIsWithinFrame)
	{
		// Allow auto-GC reference collection until the start of the next frame
		DecrementDisableElementDestructionOnGCCount();
		bIsWithinFrame = false;
	}
}

void UTypedElementRegistry::OnPostGarbageCollect()
{
	if (DisableElementDestructionOnGCCount == 0)
	{
		ProcessDeferredElementsToDestroy();
	}
}
