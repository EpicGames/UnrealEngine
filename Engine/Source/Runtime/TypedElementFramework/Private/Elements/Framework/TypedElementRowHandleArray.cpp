// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/TypedElementRowHandleArray.h"

#include <algorithm>
#include <iterator>
#include <limits>
#include "Algo/BinarySearch.h"
#include "Algo/IndexOf.h"
#include "HAL/UnrealMemory.h"
#include "Templates/Sorting.h"
#include "Templates/UnrealTemplate.h"

namespace UE::Editor::DataStorage
{
	FRowHandleArray::FRowHandleArray(FRowHandleArrayView InRows)
		: bIsSorted(InRows.IsSorted())
		, bIsUnique(InRows.IsUnique())
	{
		Rows.AddUninitialized(InRows.Num());
		FMemory::Memcpy(Rows.GetData(), InRows.GetData(), InRows.NumBytes());
	}

	int32 FRowHandleArray::Num() const
	{
		return Rows.Num();
	}

	void FRowHandleArray::Reserve(int32 Count)
	{
		Rows.Reserve(Count);
	}

	void FRowHandleArray::Shrink()
	{
		Rows.Shrink();
	}

	void FRowHandleArray::Add(RowHandle Row, EFlags Guarantee)
	{
		bIsSorted = bIsSorted && (Rows.IsEmpty() || Rows.Last() <= Row);
		bIsUnique = bIsUnique && (bIsSorted ? (Rows.IsEmpty() || Rows.Last() < Row) : (EnumHasAllFlags(Guarantee, EFlags::IsUnique)));
		Rows.Add(Row);
	}

	void FRowHandleArray::Append(TConstArrayView<RowHandle> AdditionalRows, EFlags Guarantee)
	{
		if (!AdditionalRows.IsEmpty())
		{
			if (!Rows.IsEmpty())
			{
				RowHandle CurrentLast = Rows.Last();
				RowHandle FirstNew = *AdditionalRows.begin();
				bIsSorted = bIsSorted && EnumHasAllFlags(Guarantee, EFlags::IsSorted) && (CurrentLast <= FirstNew);
				bIsUnique = bIsSorted && bIsUnique && EnumHasAllFlags(Guarantee, EFlags::IsUnique) && (CurrentLast < FirstNew);
			}
			else
			{
				bIsSorted = EnumHasAllFlags(Guarantee, EFlags::IsSorted);
				bIsUnique = EnumHasAllFlags(Guarantee, EFlags::IsUnique);
			}

			int32 StartIndex = Rows.AddUninitialized(AdditionalRows.Num());
			FMemory::Memcpy(Rows.GetData() + StartIndex, AdditionalRows.GetData(), AdditionalRows.NumBytes());
		}
	}

	void FRowHandleArray::Append(FRowHandleArrayView AdditionalRows)
	{
		if (!AdditionalRows.IsEmpty())
		{
			if (!Rows.IsEmpty())
			{
				RowHandle CurrentLast = Rows.Last();
				RowHandle FirstNew = AdditionalRows.First();
				bIsSorted = bIsSorted && AdditionalRows.IsSorted() && (CurrentLast <= FirstNew);
				bIsUnique = bIsSorted && bIsUnique && AdditionalRows.IsUnique() && (CurrentLast < FirstNew);
			}
			else
			{
				bIsSorted = AdditionalRows.IsSorted();
				bIsUnique = AdditionalRows.IsUnique();
			}
			
			int32 StartIndex = Rows.AddUninitialized(AdditionalRows.Num());
			FMemory::Memcpy(Rows.GetData() + StartIndex, AdditionalRows.GetData(), AdditionalRows.NumBytes());
		}
	}

	void FRowHandleArray::Remove(RowHandle Row)
	{
		if (bIsSorted)
		{
			if (int32 Index = Algo::BinarySearch(Rows, Row); Index >= 0)
			{
				Rows.RemoveAt(Index, EAllowShrinking::No);
			}
		}
		else
		{
			Rows.RemoveSingleSwap(Row, EAllowShrinking::No);
		}
	}

	void FRowHandleArray::Remove(TConstArrayView<RowHandle> RowsToRemove)
	{
		if (bIsSorted)
		{
			for (RowHandle Row : RowsToRemove)
			{
				if (int32 Index = Algo::BinarySearch(Rows, Row); Index >= 0)
				{
					Rows.RemoveAt(Index, EAllowShrinking::No);
				}
			}
		}
		else
		{
			for (RowHandle Row : RowsToRemove)
			{
				Rows.RemoveSingleSwap(Row, EAllowShrinking::No);
			}
		}
	}

	void FRowHandleArray::Remove(FRowHandleArrayView RowsToRemove)
	{
		if (bIsSorted && RowsToRemove.IsSorted() && !RowsToRemove.IsEmpty())
		{
			const RowHandle* RowsToRemoveIt = RowsToRemove.begin();
			const RowHandle* RowsToRemoveEnd = RowsToRemove.end();
			
			// Search for the first row that can be deleted.
			int32 StartIndex = -1;
			do
			{
				StartIndex = Algo::BinarySearch(Rows, *RowsToRemoveIt);
				RowsToRemoveIt++;
			} while (StartIndex < 0 && RowsToRemoveIt < RowsToRemoveEnd);

			if (StartIndex >= 0)
			{
				int32 DeletedCount = 1;
				RowHandle* RowsInsert = Rows.GetData() + StartIndex;
				RowHandle* RowsIt = RowsInsert + 1;
				RowHandle* RowsEnd = Rows.GetData() + Rows.Num();

				// Copy down the remaining list, increasing the gap whenever a new
				// match in the rows to be removed has been found.
				for (; RowsIt < RowsEnd; ++RowsIt)
				{
					// Catch delete pointer up, in case it encountered a row that wasn't in the array
					// or a duplicate entry.
					if (*RowsToRemoveIt < *RowsIt)
					{
						while (*RowsToRemoveIt < *RowsIt)
						{
							++RowsToRemoveIt;
						}
						if (RowsToRemoveIt == RowsToRemoveEnd)	
						{
							break;
						}
					}

					// Skip if there's a match with in the list of rows to delete.
					if (*RowsToRemoveIt == *RowsIt)
					{
						RowsToRemoveIt++;
						DeletedCount++;

						if (RowsToRemoveIt == RowsToRemoveEnd)
						{
							++RowsIt;
							break;
						}
					}
					else
					{
						*RowsInsert++ = *RowsIt;
					}
				}

				// Copy the remainder down without any checks as there are no more rows to remove.
				for (; RowsIt < RowsEnd; ++RowsIt)
				{
					*RowsInsert++ = *RowsIt;
				}

				// Remove the remaining tail of empty slots.
				int32 RemoveStartIndex = static_cast<int32>(RowsInsert - Rows.GetData());
				Rows.RemoveAt(RemoveStartIndex, DeletedCount, EAllowShrinking::No);
			}
		}
		else
		{
			Remove(TConstArrayView<RowHandle>(RowsToRemove.GetData(), RowsToRemove.Num()));
		}
	}

	bool FRowHandleArray::Contains(RowHandle Row) const
	{
		return Find(Row) >= 0;
	}

	void FRowHandleArray::Reset(int32 NewSize)
	{
		Rows.Reset(NewSize);
	}

	void FRowHandleArray::Empty(int32 Slack)
	{
		Rows.Empty(Slack);
	}

	void FRowHandleArray::Sort()
	{
		if (!bIsSorted)
		{
			RadixSort64(Rows.GetData(), Rows.Num());
			bIsSorted = true;
		}
	}

	void FRowHandleArray::Sort(TArrayView<RowHandle> ScratchBuffer)
	{
		
		if (!bIsSorted)
		{
			checkf(Rows.Num() <= ScratchBuffer.Num(), 
				TEXT("The scratch buffer used for sorting a row handle array needs to contain at least %i entries."), Rows.Num());
			// The row handle array holds 64-bit integers so the values will always be copied, so it doesn't matter whether or not
			// the buffer is initialized, so use the more straightforward code path.
			RadixSort64<ERadixSortBufferState::IsInitialized>(Rows.GetData(), ScratchBuffer.GetData(), Rows.Num());
			bIsSorted = true;
		}
	}

	void FRowHandleArray::MakeUnique()
	{
		if (!bIsUnique)
		{
			Sort();

			RowHandle* Begin = Rows.GetData();
			RowHandle* End = Rows.GetData() + Rows.Num();
			RowHandle* RemoveBegin = std::unique(Begin, End);
			Rows.RemoveAt(static_cast<int32>(std::distance(Rows.GetData(), RemoveBegin)), static_cast<int32>(std::distance(RemoveBegin, End)), EAllowShrinking::No);

			bIsUnique = true;
		}
	}

	void FRowHandleArray::ReduceToDuplicates()
	{
		static_assert(InvalidRowHandle != std::numeric_limits<RowHandle>::max(), 
			"FRowHandleArray::MakeIntersection() needs an unused value for the row handle and currently uses the max value of a RowHandle."
			"If InvalidRowHandle is ever changed to be this value, the unused value needs to be set to another value.");

		if (!bIsUnique && Rows.Num() > 1)
		{
			Sort();

			RowHandle* Insert = Rows.GetData();
			RowHandle* Front = Rows.GetData();
			RowHandle* Next = Rows.GetData() + 1;
			RowHandle* End = Insert + Rows.Num();
			// Can't use InvalidRowHandle as that might be a value in the array, so pick the largest row handle value as it's highly
			// unlikely that it'll be used.
			RowHandle LastInsert = std::numeric_limits<RowHandle>::max();

			for (; Next != End; ++Next, ++Front)
			{
				if (*Front == *Next && LastInsert != *Next)
				{
					LastInsert = *Next;
					*Insert++ = *Next;
				}
			}

			if (Front != Insert)
			{
				Rows.RemoveAt(static_cast<int32>(std::distance(Rows.GetData(), Insert)), static_cast<int32>(std::distance(Insert, End)), EAllowShrinking::No);
			}

			bIsUnique = true;
		}
		else
		{
			Rows.Reset();
			bIsUnique = true;
			bIsSorted = true;
		}
	}

	void FRowHandleArray::SortedMerge(FRowHandleArrayView AddedRows)
	{
		if (Rows.IsEmpty())
		{
			Rows.AddUninitialized(AddedRows.Num());
			FMemory::Memcpy(Rows.GetData(), AddedRows.GetData(), AddedRows.NumBytes());
			return;
		}

		Sort();

		if (!AddedRows.IsEmpty())
		{
			// Check if it's a quick append at the start or end.
			if (bIsSorted && AddedRows.IsSorted())
			{
				if (Rows.Last() <= *AddedRows.begin())
				{
					bIsUnique = bIsUnique && AddedRows.IsUnique() && Rows.Last() < *AddedRows.begin();
					int32 StartIndex = Rows.AddUninitialized(AddedRows.Num());
					FMemory::Memcpy(Rows.GetData() + StartIndex, AddedRows.GetData(), AddedRows.NumBytes());
					return;
				}
				else if (AddedRows.Last() <= *Rows.begin())
				{
					bIsUnique = bIsUnique && AddedRows.IsUnique() && AddedRows.Last() < *Rows.begin();
					Rows.Insert(AddedRows.GetData(), AddedRows.Num(), 0);
					return;
				}
			}

			SortedMergeInternal(AddedRows);
		}
	}

	void FRowHandleArray::SortedMerge(const FRowHandleArray& AddedRows)
	{
		checkf(AddedRows.IsSorted(), TEXT("Row handle array provided for sorted merge was not sorted."));

		SortedMerge(AddedRows.GetRows());
	}

	void FRowHandleArray::SortedMerge(FRowHandleArray&& AddedRows)
	{
		if (Rows.IsEmpty())
		{
			*this = MoveTemp(AddedRows);
			return;
		}

		Sort();

		if (!AddedRows.IsEmpty())
		{
			AddedRows.Sort();

			if (Rows.Last() <= *AddedRows.Rows.begin())
			{
				bIsUnique = bIsUnique && AddedRows.IsUnique() && Rows.Last() < *AddedRows.Rows.begin();
				
				int32 StartIndex = Rows.AddUninitialized(AddedRows.Num());
				FMemory::Memcpy(Rows.GetData() + StartIndex, AddedRows.Rows.GetData(), AddedRows.Rows.NumBytes());
			}
			else if (AddedRows.Rows.Last() <= *Rows.begin())
			{
				AddedRows.bIsUnique = bIsUnique && AddedRows.IsUnique() && AddedRows.Rows.Last() < *Rows.begin();

				int32 StartIndex = AddedRows.Rows.AddUninitialized(Rows.Num());
				FMemory::Memcpy(AddedRows.Rows.GetData() + StartIndex, Rows.GetData(), Rows.NumBytes());

				*this = MoveTemp(AddedRows);
			}
			else if (AddedRows.Rows.Num() < Rows.Num())
			{
				SortedMergeInternal(AddedRows.GetRows());
			}
			else
			{
				AddedRows.SortedMergeInternal(GetRows());
				*this = MoveTemp(AddedRows);
			}
		}
	}

	void FRowHandleArray::InvertOrder()
	{
		if (!Rows.IsEmpty())
		{
			RowHandle* Front = Rows.GetData();
			RowHandle* End = Front + Rows.Num() - 1;
			while (Front < End)
			{
				Swap(*Front++, *End--);
			}
			bIsSorted = false;
		}
	}

	FRowHandleArrayView FRowHandleArray::GetRows() const
	{
		FRowHandleArrayView::EFlags Flags = FRowHandleArrayView::EFlags::None;
		Flags |= bIsSorted ? FRowHandleArrayView::EFlags::IsSorted : FRowHandleArrayView::EFlags::None;
		Flags |= bIsUnique ? FRowHandleArrayView::EFlags::IsUnique : FRowHandleArrayView::EFlags::None;
		return FRowHandleArrayView(Rows, Flags);
	}

	TArrayView<RowHandle> FRowHandleArray::GetMutableRows(EFlags Guarantee)
	{
		bIsSorted = bIsSorted && EnumHasAllFlags(Guarantee, EFlags::IsSorted);
		bIsUnique = bIsUnique && EnumHasAllFlags(Guarantee, EFlags::IsUnique);
		return TArrayView<RowHandle>(Rows);
	}

	bool FRowHandleArray::IsSorted() const
	{
		return bIsSorted;
	}

	bool FRowHandleArray::IsUnique() const
	{
		return bIsUnique;
	}

	bool FRowHandleArray::IsEmpty() const
	{
		return Rows.IsEmpty();
	}

	bool FRowHandleArray::IsSame(const FRowHandleArray& Rhs) const
	{
		if (this == &Rhs)
		{
			return true;
		}

		if (Rows.Num() == Rhs.Num())
		{
			const bool bRhsIsSorted = Rhs.IsSorted();
			if (bIsSorted && bRhsIsSorted)
			{
				// If both lists are sorted do a fast memory compare.
				const SIZE_T CompareByteCount = static_cast<SIZE_T>(Rows.NumBytes());
				int32 Result = FMemory::Memcmp(Rows.GetData(), Rhs.GetRows().GetData(), CompareByteCount); 
				return Result == 0;
			}
			else if (bRhsIsSorted)
			{
				// If the right side is sorted (meaning the left side isn't) prefer to search in the right side
				// as it allows a binary search.
				for (RowHandle Row : Rows)
				{
					if (Algo::BinarySearch(Rhs.Rows, Row) < 0)
					{
						return false;
					}
				}
				return true;
			}
			else if (bIsSorted)
			{
				// Same as the previous but now prefer the left as that's the sorted list.
				for (RowHandle RhsRow : Rhs.GetRows())
				{
					if (Algo::BinarySearch(Rows, RhsRow) < 0)
					{
						return false;
					}
				}
				return true;
			}
			else
			{
				// Both lists are unsorted so do a linear search through one of them.
				for (RowHandle RhsRow : Rhs.GetRows())
				{
					if (Algo::IndexOf(Rows, RhsRow) < 0)
					{
						return false;
					}
				}
				return true;
			}
		}
		return false;
	}

	void FRowHandleArray::SortedMergeInternal(FRowHandleArrayView AddedRows)
	{
		// There are two paths, one is an in-place addition if the added rows fit within the existing array
		// and one to create a new array and merge-copy the values over. The latter is done to avoid an additional copy
		// that would happen during a resize.

		int32 NewSize = Rows.Num() + AddedRows.Num();
		if (Rows.Max() < NewSize)
		{
			TArray<RowHandle> NewArray;
			NewArray.AddUninitialized(NewSize);
			RowHandle* RowsInsert = NewArray.GetData();

			const RowHandle* AddedRowsIt = AddedRows.GetData();
			const RowHandle* AddedRowsEnd = AddedRows.GetData() + AddedRows.Num();
			const RowHandle* RowsIt = Rows.GetData();
			const RowHandle* RowsEnd = Rows.GetData() + Rows.Num();

			// Experimented with a few different ranges and landed on 8. 4 and 8 were pretty close, but
			// 8 edging out just a lit bit. 16 was slightly worse than 4. This will be hardware specific though.
			while (RowsEnd - RowsIt >= 8 && AddedRowsEnd - AddedRowsIt >= 8)
			{
				*RowsInsert++ = *RowsIt < *AddedRowsIt ? *RowsIt++ : *AddedRowsIt++;
				*RowsInsert++ = *RowsIt < *AddedRowsIt ? *RowsIt++ : *AddedRowsIt++;
				*RowsInsert++ = *RowsIt < *AddedRowsIt ? *RowsIt++ : *AddedRowsIt++;
				*RowsInsert++ = *RowsIt < *AddedRowsIt ? *RowsIt++ : *AddedRowsIt++;

				*RowsInsert++ = *RowsIt < *AddedRowsIt ? *RowsIt++ : *AddedRowsIt++;
				*RowsInsert++ = *RowsIt < *AddedRowsIt ? *RowsIt++ : *AddedRowsIt++;
				*RowsInsert++ = *RowsIt < *AddedRowsIt ? *RowsIt++ : *AddedRowsIt++;
				*RowsInsert++ = *RowsIt < *AddedRowsIt ? *RowsIt++ : *AddedRowsIt++;
			}
			while (RowsIt != RowsEnd && AddedRowsIt != AddedRowsEnd)
			{
				*RowsInsert++ = *RowsIt < *AddedRowsIt ? *RowsIt++ : *AddedRowsIt++;
			}

			if (RowsIt != RowsEnd)
			{
				FMemory::Memcpy(RowsInsert, RowsIt, (RowsEnd - RowsIt) * sizeof(RowHandle));
			}
			else if (AddedRowsIt != AddedRowsEnd)
			{
				FMemory::Memcpy(RowsInsert, AddedRowsIt, (AddedRowsEnd - AddedRowsIt) * sizeof(RowHandle));
			}

			Rows = MoveTemp(NewArray);
		}
		else
		{
			const RowHandle* AddedRowsIt = AddedRows.GetData() + AddedRows.Num() - 1;
			const RowHandle* AddedRowsEnd = AddedRows.GetData() - 1;
			const RowHandle* RowsIt = Rows.GetData() + Rows.Num() - 1;
			const RowHandle* RowsEnd = Rows.GetData() - 1;

			Rows.AddUninitialized(AddedRows.Num());
			RowHandle* RowsInsert = Rows.GetData() + Rows.Num() - 1;

			// Experimented with a few different ranges and landed on 8. 4 and 8 were pretty close, but
			// 8 edging out just a lit bit. 16 was slightly worse than 4. This will be very hardware specific though.
			while (RowsIt - RowsEnd >= 8 && AddedRowsIt - AddedRowsEnd >= 8)
			{
				*RowsInsert-- = *RowsIt > *AddedRowsIt ? *RowsIt-- : *AddedRowsIt--;
				*RowsInsert-- = *RowsIt > *AddedRowsIt ? *RowsIt-- : *AddedRowsIt--;
				*RowsInsert-- = *RowsIt > *AddedRowsIt ? *RowsIt-- : *AddedRowsIt--;
				*RowsInsert-- = *RowsIt > *AddedRowsIt ? *RowsIt-- : *AddedRowsIt--;

				*RowsInsert-- = *RowsIt > *AddedRowsIt ? *RowsIt-- : *AddedRowsIt--;
				*RowsInsert-- = *RowsIt > *AddedRowsIt ? *RowsIt-- : *AddedRowsIt--;
				*RowsInsert-- = *RowsIt > *AddedRowsIt ? *RowsIt-- : *AddedRowsIt--;
				*RowsInsert-- = *RowsIt > *AddedRowsIt ? *RowsIt-- : *AddedRowsIt--;
			}
			while (RowsIt != RowsEnd && AddedRowsIt != AddedRowsEnd)
			{
				*RowsInsert-- = *RowsIt > *AddedRowsIt ? *RowsIt-- : *AddedRowsIt--;
			}

			if (AddedRowsIt != AddedRowsEnd)
			{
				// Insert at the front as all original values have been moved out.
				FMemory::Memcpy(Rows.GetData(), AddedRows.GetData(), (AddedRowsIt - AddedRowsEnd) * sizeof(RowHandle));
			}
		}
		bIsUnique = false; // Assume there was overlap between the existing and merged list.
	}

	int32 FRowHandleArray::Find(RowHandle Row) const
	{
		return bIsSorted
			? Algo::BinarySearch(Rows, Row)
			: Algo::IndexOf(Rows, Row);
	}
} // namespace UE::Editor::DataStorage
