// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/TypedElementRowHandleArrayView.h"

#include "Algo/BinarySearch.h"

namespace UE::Editor::DataStorage
{
	FRowHandleArrayView::FRowHandleArrayView(const RowHandle* Rows, int32 RowCount, EFlags Flags)
		: Rows(Rows)
		, RowCount(RowCount)
		, bIsSorted(EnumHasAllFlags(Flags, EFlags::IsSorted))
		, bIsUnique(EnumHasAllFlags(Flags, EFlags::IsUnique))
	{}

	FRowHandleArrayView::FRowHandleArrayView(TConstArrayView<RowHandle> Rows, EFlags Flags)
		: Rows(Rows.GetData())
		, RowCount(Rows.Num())
		, bIsSorted(EnumHasAllFlags(Flags, EFlags::IsSorted))
		, bIsUnique(EnumHasAllFlags(Flags, EFlags::IsUnique))
	{}

	FRowHandleArrayView::FRowHandleArrayView(std::initializer_list<RowHandle> Rows, EFlags Flags)
		: Rows(std::data(Rows))
		, RowCount(static_cast<int32>(Rows.size()))
		, bIsSorted(EnumHasAllFlags(Flags, EFlags::IsSorted))
		, bIsUnique(EnumHasAllFlags(Flags, EFlags::IsUnique))
	{}

	const RowHandle* FRowHandleArrayView::begin() const
	{
		return Rows;
	}

	const RowHandle* FRowHandleArrayView::end() const
	{
		return Rows + RowCount;
	}

	RowHandle FRowHandleArrayView::operator[](uint32 Index) const
	{
		checkf(static_cast<int32>(Index) < RowCount && RowCount > 0, TEXT("Index (%u) out of bounds (%i)."), Index, RowCount);
		return Rows[Index];
	}

	const RowHandle* FRowHandleArrayView::GetData() const
	{
		return Rows;
	}

	const RowHandle& FRowHandleArrayView::First() const
	{
		checkf(RowCount > 0, TEXT("Attempting to get the first element from an empty row handle array view."));
		return Rows[0];
	}

	const RowHandle& FRowHandleArrayView::Last() const
	{
		checkf(RowCount > 0, TEXT("Attempting to get the last element from an empty row handle array view."));
		return Rows[RowCount - 1];
	}

	int32 FRowHandleArrayView::Num() const
	{
		return RowCount;
	}

	int32 FRowHandleArrayView::NumBytes() const
	{
		return RowCount * sizeof(RowHandle);
	}

	FRowHandleArrayView::operator TConstArrayView<RowHandle>() const
	{
		return TConstArrayView<RowHandle>(Rows, RowCount);
	}

	bool FRowHandleArrayView::IsSorted() const
	{
		return bIsSorted;
	}

	bool FRowHandleArrayView::IsUnique() const
	{
		return bIsUnique;
	}

	bool FRowHandleArrayView::IsEmpty() const
	{
		return Rows == nullptr || RowCount <= 0;
	}

	bool FRowHandleArrayView::Contains(RowHandle Row) const
	{
		if (bIsSorted)
		{
			TConstArrayView<RowHandle> Range(Rows, RowCount);
			return Algo::BinarySearch(Range, Row) != INDEX_NONE;
		}
		else
		{
			for (RowHandle Current : *this)
			{
				if (Current == Row)
				{
					return true;
				}
			}
			return false;
		}
	}
} // namespace UE::Editor::DataStorage
