// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/TypedElementUtil.h"

#include "Elements/Framework/TypedElementList.h"
#include "Elements/Framework/TypedElementRegistry.h"

namespace TypedElementUtil
{

void BatchElementsByType(FTypedElementListConstRef InElementsToBatch, TMap<FTypedHandleTypeId, TArray<FTypedElementHandle>>& OutElementsByType)
{
	OutElementsByType.Reset();
	InElementsToBatch->ForEachElementHandle([&OutElementsByType](const FTypedElementHandle& InElementHandle)
	{
		TArray<FTypedElementHandle>& ElementsForType = OutElementsByType.FindOrAdd(InElementHandle.GetId().GetTypeId());
		ElementsForType.Add(InElementHandle);
		return true;
	});
}

void BatchElementsByType(TArrayView<const FTypedElementHandle> InElementsToBatch, TMap<FTypedHandleTypeId, TArray<FTypedElementHandle>>& OutElementsByType)
{
	OutElementsByType.Reset();
	for (const FTypedElementHandle& ElementHandle : InElementsToBatch)
	{
		TArray<FTypedElementHandle>& ElementsForType = OutElementsByType.FindOrAdd(ElementHandle.GetId().GetTypeId());
		ElementsForType.Add(ElementHandle);
	}
}

TArray<FScriptTypedElementHandle> ConvertToScriptElementArray(const TArray<FTypedElementHandle>& InNativeHandles, UTypedElementRegistry* Registry)
{
	TArray<FScriptTypedElementHandle> ScriptHandles;
	ScriptHandles.Reserve(InNativeHandles.Num());
	for (const FTypedElementHandle& NativeHandle : InNativeHandles)
	{
		ScriptHandles.Add(Registry->CreateScriptHandle(NativeHandle.GetId()));
	}
	
	return ScriptHandles;
}

TArray<FTypedElementHandle> ConvertToNativeElementArray(const TArray<FScriptTypedElementHandle>& InScriptHandles)
{
	TArray<FTypedElementHandle> NativeHandles;
	NativeHandles.Reserve(InScriptHandles.Num());

	for (const FScriptTypedElementHandle& ScriptHandle : InScriptHandles)
	{
		if (FTypedElementHandle NativeHandle = ScriptHandle.GetTypedElementHandle())
		{
			NativeHandles.Add(MoveTemp(NativeHandle));
		}
	}

	return NativeHandles;
}

} // namespace TypedElementUtil
