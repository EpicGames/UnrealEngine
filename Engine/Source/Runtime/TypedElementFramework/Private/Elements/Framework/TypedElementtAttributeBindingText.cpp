// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/TypedElementAttributeBindingText.h"

namespace UE::Editor::DataStorage
{
	FTextAttributeFormatted::FTextAttributeFormatted(
		FTextFormat InFormat, RowHandle InTargetRow, ICoreProvider* InDataStorage)
		: Format(MoveTemp(InFormat))
		, TargetRow(InTargetRow)
		, DataStorage(InDataStorage)
	{
	}
	
	FTextAttributeFormatted& FTextAttributeFormatted::Arg(FString Name, FFormatArgumentValue Value)
	{
		NamedProperties.Add(MoveTemp(Name), FPropertyInfo{ .Default = MoveTemp(Value) });
		return *this;
	}

	FTextAttributeFormatted::operator TAttribute<FText>()
	{
		using namespace Private; 
		
		return TAttribute<FText>::CreateLambda(
			[
				Format = MoveTemp(Format),
				NamedProperties = MoveTemp(NamedProperties),
				Storage = DataStorage,
				Row = TargetRow
			]()
			{
				FFormatNamedArguments NamedArguments;
				NamedArguments.Reserve(NamedProperties.Num());
				for (const TPair<FString, FPropertyInfo>& Property : NamedProperties)
				{
					if (Property.Value.Property.IsBound())
					{
						TWeakObjectPtr<const UScriptStruct> ColumnType = Property.Value.Property.GetObjectTypeInfo();

						if(Property.Value.DynamicColumnIdentifier != NAME_None)
						{
							ColumnType = Storage->GenerateDynamicColumn(FDynamicColumnDescription{
								.TemplateType = ColumnType.Get(),
								.Identifier = Property.Value.DynamicColumnIdentifier
							});
						}

						if(ColumnType.IsValid())
						{
							if (void* Column = Storage->GetColumnData(Row, ColumnType.Get()))
							{
								// We use the original column type (Property.GetObjectTypeInfo()) instead of the dynamic ColumnType
								// so that the type info in the property matches
								NamedArguments.Add(Property.Key, Property.Value.Property.Get(Column, Property.Value.Property.GetObjectTypeInfo()));
								continue;
							}
						}
					}
					NamedArguments.Add(Property.Key, Property.Value.Default);
				}
				return FText::Format(Format, NamedArguments);
			});
	}
}
