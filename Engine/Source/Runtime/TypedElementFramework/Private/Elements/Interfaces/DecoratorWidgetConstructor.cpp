// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Interfaces/DecoratorWidgetConstructor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DecoratorWidgetConstructor)

FTedsDecoratorWidgetConstructor::FTedsDecoratorWidgetConstructor(const UScriptStruct* InTypeInfo)
	: FTedsWidgetConstructorBase(InTypeInfo)
{
}

TSharedPtr<SWidget> FTedsDecoratorWidgetConstructor::CreateDecoratorWidget(TSharedPtr<SWidget> InChildWidget,
	UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle DataRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	return nullptr;
}
