// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Elements/Columns/TypedElementUIColumns.h"
#include "Elements/Columns/WidgetPurposeColumns.h"
#include "Elements/Framework/TypedElementDataStorageWidget.h"
#include "Math/NumericLimits.h"
#include "Widgets/SNullWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementDataStorageUiInterface)

#define LOCTEXT_NAMESPACE "FTypedElementWidgetConstructor"

//
// FTedsWidgetConstructorBase
//

FTedsWidgetConstructorBase::FTedsWidgetConstructorBase(const UScriptStruct* InTypeInfo)
	: TypeInfo(InTypeInfo)
{
}

bool FTedsWidgetConstructorBase::Initialize(const UE::Editor::DataStorage::FMetaDataView& InArguments,
	TArray<TWeakObjectPtr<const UScriptStruct>> InMatchedColumnTypes, UE::Editor::DataStorage::RowHandle InFactoryRowHandle)
{
	WidgetFactoryRow = InFactoryRowHandle;
	return Initialize_Internal(InArguments, MoveTemp(InMatchedColumnTypes));
}

bool FTedsWidgetConstructorBase::Initialize_Internal(const UE::Editor::DataStorage::FMetaDataView& InArguments,
	TArray<TWeakObjectPtr<const UScriptStruct>> InMatchedColumnTypes)
{
	MatchedColumnTypes = MoveTemp(InMatchedColumnTypes);
	
	// If we matched with any dynamic columns, store a mapping from the base template to the identifier so the widget can look it up later
	for (const TWeakObjectPtr<const UScriptStruct>& Column : MatchedColumnTypes)
	{
		FName Identifier = UE::Editor::DataStorage::ColumnUtils::GetDynamicColumnIdentifier(Column.Get());
		if (!Identifier.IsNone())
		{
			MatchedDynamicTemplates.Emplace(Cast<const UScriptStruct>(Column->GetSuperStruct()), Identifier);
		}
	}
	
	return true;
}

const UScriptStruct* FTedsWidgetConstructorBase::GetTypeInfo() const
{
	return TypeInfo;
}

const TArray<TWeakObjectPtr<const UScriptStruct>>& FTedsWidgetConstructorBase::GetMatchedColumns() const
{
	return MatchedColumnTypes;
}

const UE::Editor::DataStorage::Queries::FConditions* FTedsWidgetConstructorBase::GetQueryConditions(const UE::Editor::DataStorage::ICoreProvider* Storage) const
{
	if (const FWidgetFactoryConditionsColumn* ConditionsColumn = Storage->GetColumn<FWidgetFactoryConditionsColumn>(WidgetFactoryRow))
	{
		return &ConditionsColumn->Conditions;
	}
	else 
	{
		return QueryConditions;
	}
}

//
// FTypedElementWidgetConstructor
//

FTypedElementWidgetConstructor::FTypedElementWidgetConstructor(const UScriptStruct* InTypeInfo)
	: FTedsWidgetConstructorBase(InTypeInfo)
{
}

TConstArrayView<const UScriptStruct*> FTypedElementWidgetConstructor::GetAdditionalColumnsList() const
{
	return {};
}

FString FTypedElementWidgetConstructor::CreateWidgetDisplayName(
	UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle Row) const
{
	return CreateWidgetDisplayNameText(DataStorage, Row).ToString();
}

FText FTypedElementWidgetConstructor::CreateWidgetDisplayNameText(
	UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle Row) const
{
	switch (MatchedColumnTypes.Num())
	{
	case 0:
		return LOCTEXT("TEDSColumn", "TEDS Column");
	case 1:
		return DescribeColumnType(MatchedColumnTypes[0].Get());
	default:
	{
		FString LongestMatchString = DescribeColumnType(MatchedColumnTypes[0].Get()).ToString();
		FStringView LongestMatch = LongestMatchString;
		const TWeakObjectPtr<const UScriptStruct>* It = MatchedColumnTypes.GetData();
		const TWeakObjectPtr<const UScriptStruct>* ItEnd = It + MatchedColumnTypes.Num();
		++It; // Skip the first entry as that's already set.
		for (; It != ItEnd; ++It)
		{
			FString NextMatchText = DescribeColumnType(It->Get()).ToString();
			FStringView NextMatch = NextMatchText;

			int32 MatchSize = 0;
			auto ItLeft = LongestMatch.begin();
			auto ItLeftEnd = LongestMatch.end();
			auto ItRight = NextMatch.begin();
			auto ItRightEnd = NextMatch.end();
			while (
				ItLeft != ItLeftEnd &&
				ItRight != ItRightEnd &&
				*ItLeft == *ItRight)
			{
				++MatchSize;
				++ItLeft;
				++ItRight;
			}

			// At least 3 letters have to match to avoid single or double letter names which typically mean nothing.
			if (MatchSize > 2)
			{
				LongestMatch.LeftInline(MatchSize);
			}
			else
			{
				// There are not enough characters in the string that match. Just return the name of the first column
				return FText::FromString(LongestMatchString);
			}
		}
		return FText::FromString(FString(LongestMatch));
	}
	};
}

TSharedPtr<SWidget> FTypedElementWidgetConstructor::ConstructFinalWidget(
	UE::Editor::DataStorage::RowHandle Row,
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	// Add the additional columns to the UI row
	TSharedPtr<SWidget> Widget = SNullWidget::NullWidget;
	
	if (const FTypedElementRowReferenceColumn* RowReference = DataStorage->GetColumn<FTypedElementRowReferenceColumn>(Row))
	{
		bool bConstructWidget = DataStorage->IsRowAssigned(RowReference->Row);

		// If the original row matches this widgets query conditions currently, create the actual internal widget
		if (const UE::Editor::DataStorage::Queries::FConditions* MatchedQueryConditions = GetQueryConditions(DataStorage);
			MatchedQueryConditions != nullptr && bConstructWidget)
		{
			bConstructWidget &= DataStorage->MatchesColumns(RowReference->Row, *MatchedQueryConditions);
		}
		
		if (bConstructWidget)
		{
			DataStorage->AddColumns(Row, GetAdditionalColumnsList());
			Widget = Construct(Row, DataStorage, DataStorageUi, Arguments);
		}
	}
	// If we don't have an original row, simply construct the widget
	else
	{
		DataStorage->AddColumns(Row, GetAdditionalColumnsList());
		Widget = Construct(Row, DataStorage, DataStorageUi, Arguments);
	}
	
	return Widget;
}

TSharedPtr<SWidget> FTypedElementWidgetConstructor::Construct(
	UE::Editor::DataStorage::RowHandle Row,
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	UE::Editor::DataStorage::RowHandle TargetRow = GetTargetRow(DataStorage, Row);

	TSharedPtr<SWidget> Widget = CreateWidget(DataStorage, DataStorageUi, TargetRow, Row, Arguments);
	if (Widget)
	{
		DataStorage->GetColumn<FTypedElementSlateWidgetReferenceColumn>(Row)->Widget = Widget;
		if (SetColumns(DataStorage, Row))
		{
			if (FinalizeWidget(DataStorage, DataStorageUi, Row, Widget))
			{
				AddDefaultWidgetColumns(Row, DataStorage);
				return Widget;
			}
		}
	}
	return nullptr;
}

TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> FTypedElementWidgetConstructor::ConstructColumnSorters(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	
	TArray<TWeakObjectPtr<const UScriptStruct>> ColumnTypes;
	ColumnTypes.Reserve(MatchedColumnTypes.Num());
	auto GetAllColumns = [&ColumnTypes](TNotNull<const UScriptStruct*> Column)
	{
		ColumnTypes.Add(Column);
	};
	
	ForAllColumnTypes(GetAllColumns);

	TArray<TSharedPtr<const FColumnSorterInterface>> Results;
	Results.Reserve(ColumnTypes.Num());
	DataStorageUi->GeneratePropertySorters(Results, ColumnTypes);
	
	return Results;
}

TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSearcherInterface>> FTypedElementWidgetConstructor::ConstructColumnSearchers(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;

	TArray<TWeakObjectPtr<const UScriptStruct>> ColumnTypes;
	ColumnTypes.Reserve(MatchedColumnTypes.Num());
	auto GetAllColumns = [&ColumnTypes](TNotNull<const UScriptStruct*> Column)
		{
			ColumnTypes.Add(Column);
		};
	
	ForAllColumnTypes(GetAllColumns);

	TArray<TSharedPtr<const FColumnSearcherInterface>> Results;
	Results.Reserve(ColumnTypes.Num());
	DataStorageUi->GeneratePropertySearchers(Results, ColumnTypes);
	
	return Results;
}

TSharedPtr<SWidget> FTypedElementWidgetConstructor::CreateWidget(const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	return nullptr;
}

TSharedPtr<SWidget> FTypedElementWidgetConstructor::CreateWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle UiRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	return CreateWidget(Arguments);
}

bool FTypedElementWidgetConstructor::SetColumns(UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle Row)
{
	return true;
}

FText FTypedElementWidgetConstructor::DescribeColumnType(const UScriptStruct* ColumnType) const
{
#if WITH_EDITOR
	if (ColumnType)
	{
		return ColumnType->GetDisplayNameText();
	}
	else
#endif
	{
		return LOCTEXT("Invalid", "<Invalid>");
	}
}

bool FTypedElementWidgetConstructor::FinalizeWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle Row,
	const TSharedPtr<SWidget>& Widget)
{
	return true;
}

void FTypedElementWidgetConstructor::AddDefaultWidgetColumns(UE::Editor::DataStorage::RowHandle Row, UE::Editor::DataStorage::ICoreProvider* DataStorage) const
{
	const FString WidgetLabel(CreateWidgetDisplayNameText(DataStorage, Row).ToString());
	DataStorage->AddColumn(Row, FTypedElementLabelColumn{.Label = WidgetLabel} );

	UE::Editor::DataStorage::RowHandle FactoryRow = GetWidgetFactoryRow();

	// Add a reference to the widget purpose this widget is being created for
	if (FWidgetFactoryColumn* FactoryColumn = DataStorage->GetColumn<FWidgetFactoryColumn>(FactoryRow))
	{
		UE::Editor::DataStorage::RowHandle PurposeRow = FactoryColumn->PurposeRowHandle;
		DataStorage->AddColumn(Row, FWidgetPurposeReferenceColumn{.PurposeRowHandle = PurposeRow});
	}

	// We don't want to display any second level widgets (widgets for widgets and so on...) in UI because they will cause the table viewer to
	// infinitely grow as you keep scrolling (which creates new widgets)
	if(DataStorage->HasColumns<FTypedElementSlateWidgetReferenceColumn>(Row))
	{
		if(const FTypedElementRowReferenceColumn* RowReferenceColumn = DataStorage->GetColumn<FTypedElementRowReferenceColumn>(Row))
		{
			if(DataStorage->HasColumns<FTypedElementSlateWidgetReferenceColumn>(RowReferenceColumn->Row))
			{
				DataStorage->AddColumn(Row, FHideRowFromUITag::StaticStruct());
			}
		}
	}
}

UE::Editor::DataStorage::RowHandle FTypedElementWidgetConstructor::GetTargetRow(
	UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle WidgetRow) const
{
	using namespace UE::Editor::DataStorage;
	RowHandle TargetRow = InvalidRowHandle;

	if(const FTypedElementRowReferenceColumn* RowReferenceColumn = DataStorage->GetColumn<FTypedElementRowReferenceColumn>(WidgetRow))
	{
		TargetRow = RowReferenceColumn->Row;
	}

	return TargetRow;
}

void FTypedElementWidgetConstructor::ForAllColumnTypes(TFunctionRef<void(TNotNull<const UScriptStruct*>)> Function) const
{
	using namespace UE::Editor::DataStorage;
	for (const TWeakObjectPtr<const UScriptStruct>& MatchedColumn : MatchedColumnTypes)
	{
		if (const UScriptStruct* MatchedColumnStruct = MatchedColumn.Get())
		{
			Function(TNotNull<const UScriptStruct*>(MatchedColumnStruct));
		}
	}
	
	TArray<TWeakObjectPtr<const UScriptStruct>, TInlineAllocator<32>> DynamicKeys;
	MatchedDynamicTemplates.GetKeys(DynamicKeys);

	for (const TWeakObjectPtr<const UScriptStruct>& DynamicKey : DynamicKeys)
	{
		if (const UScriptStruct* DynamicKeyStruct = DynamicKey.Get())
		{
			Function(TNotNull<const UScriptStruct*>(DynamicKeyStruct));
		}
	}
}

UE::Editor::DataStorage::RowHandle FTypedElementWidgetConstructor::GetWidgetFactoryRow() const
{
	return WidgetFactoryRow;
}

bool FTypedElementWidgetConstructor::Initialize(const UE::Editor::DataStorage::FMetaDataView& InArguments,
	TArray<TWeakObjectPtr<const UScriptStruct>> InMatchedColumnTypes, const UE::Editor::DataStorage::Queries::FConditions& InQueryConditions)
{
	QueryConditions = &InQueryConditions;
	return Initialize_Internal(InArguments, MoveTemp(InMatchedColumnTypes));
}

const UE::Editor::DataStorage::Queries::FConditions* FTypedElementWidgetConstructor::GetQueryConditions() const
{
	const UE::Editor::DataStorage::ICoreProvider* Storage = UE::Editor::DataStorage::GetDataStorageFeature<UE::Editor::DataStorage::ICoreProvider>(UE::Editor::DataStorage::StorageFeatureName);
	return FTedsWidgetConstructorBase::GetQueryConditions(Storage);
}


// FSimpleWidgetConstructor

FSimpleWidgetConstructor::FSimpleWidgetConstructor(const UScriptStruct* InTypeInfo)
	: FTypedElementWidgetConstructor(InTypeInfo)
{
}

TSharedPtr<SWidget> FSimpleWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	return nullptr;
}

bool FSimpleWidgetConstructor::SetColumns(UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle Row)
{
	return FTypedElementWidgetConstructor::SetColumns(DataStorage, Row);
}

TSharedPtr<SWidget> FSimpleWidgetConstructor::CreateWidget(const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	// This function is not needed anymore and only exists so derived classes cannot derive from it anymore
	return nullptr;
}

bool FSimpleWidgetConstructor::FinalizeWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle Row, const TSharedPtr<SWidget>& Widget)
{
	// This function is not needed anymore and only exists so derived classes cannot derive from it anymore
	return true;
}

TSharedPtr<SWidget> FSimpleWidgetConstructor::Construct(UE::Editor::DataStorage::RowHandle WidgetRow, UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	const UE::Editor::DataStorage::RowHandle TargetRow = GetTargetRow(DataStorage, WidgetRow);

	// Set any required columns on the widget row first
	SetColumns(DataStorage, WidgetRow);

	// Create the actual widget
	TSharedPtr<SWidget> Widget = CreateWidget(DataStorage, DataStorageUi, TargetRow, WidgetRow, Arguments);

	// If the widget was created, add the default columns we want all widget rows to have (e.g Label)
	if (Widget)
	{
		AddDefaultWidgetColumns(WidgetRow, DataStorage);
	}
	
	return Widget;
}

UE::Editor::DataStorage::IUiProvider::FPurposeInfo::FPurposeInfo(const FName& InNamespace, const FName& InName, const FName& InFrame,
	EPurposeType InPurposeType, const FText& InDescription, const FPurposeID& InParent)
	: Namespace(InNamespace)
	, Name(InName)
	, Frame(InFrame)
	, Type(InPurposeType)
	, Description(InDescription)
	, ParentPurposeID(InParent)
{
}

UE::Editor::DataStorage::IUiProvider::FPurposeInfo::FPurposeInfo(const FName& InLegacyPurposeName, EPurposeType InPurposeType,
	const FText& InDescription, const FPurposeID& InParent)
{
	FString NamespaceSplit, NameSplit, FrameSplit;

	// We want to split the legacy purpose separated by dots into 3 unique names. E.g "General.Cell.Large" -> "General", "Cell", "Large"
	InLegacyPurposeName.ToString().Split(".", &NamespaceSplit, &NameSplit);
	NameSplit.Split(".", &NameSplit, &FrameSplit);

	Namespace = FName(NamespaceSplit);
	Name = FName(NameSplit);
	Frame = FName(FrameSplit);
	
	Type = InPurposeType;
	Description = InDescription;
	ParentPurposeID = InParent;
}

UE::Editor::DataStorage::IUiProvider::FPurposeInfo::FPurposeInfo(const FWidgetPurposeNameColumn& WidgetPurposeNameColumn)
	: FPurposeInfo(WidgetPurposeNameColumn.Namespace, WidgetPurposeNameColumn.Name, WidgetPurposeNameColumn.Frame)
{
	
}

UE::Editor::DataStorage::IUiProvider::FPurposeID UE::Editor::DataStorage::IUiProvider::FPurposeInfo::GeneratePurposeID() const
{
	return FMapKey(ToString());
}

FName UE::Editor::DataStorage::IUiProvider::FPurposeInfo::ToString() const
{
	// Create the unique ID from the purpose by combining the namespace, name and frame
	TStringBuilder<256> PurposeIDBuilder;

	if (!Namespace.IsNone())
	{
		Namespace.AppendString(PurposeIDBuilder);
	}

	// We want the dots even if the namespace/frame is missing since it signifies there is something missing (".Name.Frame")
	// instead of being ambigous ("Namespace.Name", which is identical to "Name.Frame")
	PurposeIDBuilder.Append(TEXT("."));
	Name.AppendString(PurposeIDBuilder);
	PurposeIDBuilder.Append(TEXT("."));

	if (!Frame.IsNone())
	{
		Frame.AppendString(PurposeIDBuilder);
	}

	return FName(PurposeIDBuilder);
}


#undef LOCTEXT_NAMESPACE
