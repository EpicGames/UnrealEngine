// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsRevisionControlStateHelper.h"

#include "Elements/Columns/TypedElementPackageColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Misc/Paths.h"

namespace UE::Editor::RevisionControl
{
	using namespace UE::Editor::DataStorage;

	RowHandle FindOrAddPackageRow(ICoreProvider& DataStorage, const FString& InFilename)
	{
		const TableHandle PackageTable = DataStorage.FindTable(PackageTableName);
		if (PackageTable == InvalidTableHandle)
		{
			return InvalidRowHandle;
		}

		FString Filename = InFilename;
		FPaths::NormalizeFilename(Filename);
		Filename = FPaths::ConvertRelativePathToFull(Filename);

		FMapKey Key = FMapKey(MoveTemp(Filename));
		RowHandle Row = DataStorage.LookupMappedRow(MappingDomain, Key);

		if (!DataStorage.IsRowAvailable(Row))
		{
			Row = DataStorage.AddRow(PackageTable);
			DataStorage.MapRow(MappingDomain, MoveTemp(Key), Row);
		}
		return Row;
	}

	void ApplySccStateUpdate(ICoreProvider& DataStorage, RowHandle Row, const FSccStateColumn& NewState)
	{
		FSccStateColumn StateColumn = NewState;
		DataStorage.AddColumn(Row, MoveTemp(StateColumn));
		DataStorage.AddColumn(Row, FSccStateDirtyTag::StaticStruct());

		// Signal that an update pass has completed regardless of modification status —
		// drives the StopFetchUpdates observer that tears down file-status monitoring.
		DataStorage.AddColumns<FSCCStatusUpdateEndedTag>(Row);

		// The backreference via FTypedElementPackageReference isn't 1:1 (multiple actors
		// can reference a single SCC row), but it at least filters out rows with no actors.
		if (DataStorage.HasColumns<FTypedElementPackageReference>(Row))
		{
			DataStorage.ActivateQueries(TEXT("UpdateSCCForActors"));
		}
	}
}
