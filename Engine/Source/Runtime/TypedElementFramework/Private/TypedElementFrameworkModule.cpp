// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/TypedElementRegistry.h"
#include "Modules/ModuleManager.h"

class FTypedElementFrameworkModule : public FDefaultModuleImpl
{
public:
	virtual void StartupModule() override
	{
		UTypedElementRegistry::Private_InitializeInstance();
	}

	virtual void ShutdownModule() override
	{
		UTypedElementRegistry::Private_ShutdownInstance();
	}
};

IMPLEMENT_MODULE(FTypedElementFrameworkModule, TypedElementFramework)
