// Copyright Epic Games, Inc. All Rights Reserved.

#include "TypedElementFrameworkTests.h"
#include "Misc/AutomationTest.h"
#include "ProfilingDebugging/ScopedTimers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementFrameworkTests)

UE_DEFINE_TYPED_ELEMENT_DATA_RTTI(FTestTypedElementData);

FText UTestTypedElementInterfaceA_ImplTyped::GetDisplayName(const FTypedElementHandle& InElementHandle)
{
	const FTestTypedElementData* ElementData = InElementHandle.GetData<FTestTypedElementData>();
	if (!ElementData)
	{
		return FText();
	}

	return FText();
}

bool UTestTypedElementInterfaceA_ImplTyped::SetDisplayName(const FTypedElementHandle& InElementHandle, FText InNewName, bool bNotify)
{
	const FTestTypedElementData* ElementData = InElementHandle.GetData<FTestTypedElementData>();
	if (!ElementData)
	{
		return false;
	}

	return false;
}

FText UTestTypedElementInterfaceA_ImplUntyped::GetDisplayName(const FTypedElementHandle& InElementHandle)
{
	return FText();
}

bool UTestTypedElementInterfaceA_ImplUntyped::SetDisplayName(const FTypedElementHandle& InElementHandle, FText InNewName, bool bNotify)
{
	return false;
}

bool UTestTypedElementInterfaceBAndC_Typed::MarkAsTested(const FTypedElementHandle& InElementHandle)
{
	const FTestTypedElementData* ElementData = InElementHandle.GetData<FTestTypedElementData>();
	if (ElementData)
	{
		return false;
	}

	// this shouldn't happen during the test
	check(false);
	return false;
}

bool UTestTypedElementInterfaceBAndC_Typed::GetIsTested(const FTypedElementHandle& InElementHandle) const
{
	const FTestTypedElementData* ElementData = InElementHandle.GetData<FTestTypedElementData>();
	if (ElementData)
	{
		return false;
	}

	// this shouldn't happen during the test
	check(false);
	return false;
}

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTypedElementRegistrySmokeTest, "System.Runtime.TypedElementRegistry.SmokeTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::SmokeFilter)
bool FTypedElementRegistrySmokeTest::RunTest(const FString& Parameters)
{
	UTypedElementRegistry* Registry = NewObject<UTypedElementRegistry>();
	
	auto TestInterfaceAHandle = [](const TTypedElement<ITestTypedElementInterfaceA>& InElementDataInterfaceHandle)
	{
		// Proxy API added via specialization
		InElementDataInterfaceHandle.SetDisplayName(FText());
		InElementDataInterfaceHandle.GetDisplayName();

		// Verbose API
		InElementDataInterfaceHandle.GetInterfaceChecked().SetDisplayName(InElementDataInterfaceHandle, FText());
		InElementDataInterfaceHandle.GetInterfaceChecked().GetDisplayName(InElementDataInterfaceHandle);
	};

	auto TestInterfaceBHandle = [](TTypedElement<ITestTypedElementInterfaceB>& InElementDataInterfaceHandle)
	{
		// Proxy API added via specialization
		InElementDataInterfaceHandle.MarkAsTested();


		// Verbose API
		InElementDataInterfaceHandle.GetInterfaceChecked().MarkAsTested(InElementDataInterfaceHandle);
	};

	auto TestInterfaceCHandle = [](const TTypedElement<ITestTypedElementInterfaceC>& InElementDataInterfaceHandle)
	{
		// Proxy API added via specialization
		InElementDataInterfaceHandle.GetIsTested();

		// Verbose API
		InElementDataInterfaceHandle.GetInterfaceChecked().GetIsTested(InElementDataInterfaceHandle);
	};

	auto TestInterfaceAccess = [&Registry, &TestInterfaceAHandle, &TestInterfaceBHandle, &TestInterfaceCHandle](const FTypedElementHandle& InElementHandle)
	{
		// Get the interface and the element handle in two calls - this is how scripting might work
		if (ITestTypedElementInterfaceA* Interface = Registry->GetElementInterface<ITestTypedElementInterfaceA>(InElementHandle))
		{
			Interface->SetDisplayName(InElementHandle, FText());
			Interface->GetDisplayName(InElementHandle);
		}

		// Get the interface and the element handle in a single call - this is how C++ might work
		if (TTypedElement<ITestTypedElementInterfaceA> Element = Registry->GetElement<ITestTypedElementInterfaceA>(InElementHandle))
		{
			TestInterfaceAHandle(Element);
		}

		// Get the interface and the element handle in two calls - this is how scripting might work
		if (ITestTypedElementInterfaceB* Interface = Registry->GetElementInterface<ITestTypedElementInterfaceB>(InElementHandle))
		{
			Interface->MarkAsTested(InElementHandle);
		}

		// Get the interface and the element handle in a single call - this is how C++ might work
		if (TTypedElement<ITestTypedElementInterfaceB> Element = Registry->GetElement<ITestTypedElementInterfaceB>(InElementHandle))
		{
			TestInterfaceBHandle(Element);
		}

		// Get the interface and the element handle in two calls - this is how scripting might work
		if (ITestTypedElementInterfaceC* Interface = Registry->GetElementInterface<ITestTypedElementInterfaceC>(InElementHandle))
		{
			Interface->GetIsTested(InElementHandle);
		}

		// Get the interface and the element handle in a single call - this is how C++ might work
		if (TTypedElement<ITestTypedElementInterfaceC> Element = Registry->GetElement<ITestTypedElementInterfaceC>(InElementHandle))
		{
			TestInterfaceCHandle(Element);
		}
	};

	const FName DummyElementType_Typed = "DummyElementType_Typed";
	Registry->RegisterElementType<FTestTypedElementData, true>(DummyElementType_Typed);
	Registry->RegisterElementInterface<ITestTypedElementInterfaceA>(DummyElementType_Typed, NewObject<UTestTypedElementInterfaceA_ImplTyped>());

	UObject* TypedElementInterfaceBAndC =  NewObject<UTestTypedElementInterfaceBAndC_Typed>();
	Registry->RegisterElementInterface<ITestTypedElementInterfaceB>(DummyElementType_Typed, TypedElementInterfaceBAndC);
	Registry->RegisterElementInterface<ITestTypedElementInterfaceC>(DummyElementType_Typed, TypedElementInterfaceBAndC);


	const FName DummyElementType_Untyped = "DummyElementType_Untyped";
	Registry->RegisterElementType(DummyElementType_Untyped, true);
	Registry->RegisterElementInterface<ITestTypedElementInterfaceA>(DummyElementType_Untyped, NewObject<UTestTypedElementInterfaceA_ImplUntyped>());

	TTypedElementOwner<FTestTypedElementData> TypedElement1 = Registry->CreateElement<FTestTypedElementData>(DummyElementType_Typed);
	TypedElement1.GetDataChecked().InternalElementId = "TypedElement1";
	TTypedElementOwner<FTestTypedElementData> TypedElement2 = Registry->CreateElement<FTestTypedElementData>(DummyElementType_Typed);
	TypedElement2.GetDataChecked().InternalElementId = "TypedElement2";
	TTypedElementOwner<FTestTypedElementData> TypedElement3 = Registry->CreateElement<FTestTypedElementData>(DummyElementType_Typed);
	TypedElement3.GetDataChecked().InternalElementId = "TypedElement3";

	FTypedElementOwner UntypedElement1 = Registry->CreateElement(DummyElementType_Untyped, 0);
	FTypedElementOwner UntypedElement2 = Registry->CreateElement(DummyElementType_Untyped, 1);
	FTypedElementOwner UntypedElement3 = Registry->CreateElement(DummyElementType_Untyped, 2);

	TestInterfaceAccess(TypedElement1.AcquireHandle());
	TestInterfaceAccess(UntypedElement1.AcquireHandle());

	FTypedElementListPtr ElementList = Registry->CreateElementList();
	ElementList->Reserve(6);
	ElementList->Add(TypedElement1);
	ElementList->Add(TypedElement2);
	ElementList->Add(TypedElement3);
	ElementList->Add(UntypedElement1);
	ElementList->Add(UntypedElement2);
	ElementList->Add(UntypedElement3);
	
	ElementList->ForEachElementHandle([&TestInterfaceAccess](const FTypedElementHandle& InElementHandle)
	{
		TestInterfaceAccess(InElementHandle);
		return true;
	});
	
	ElementList->ForEachElement<ITestTypedElementInterfaceA>([&TestInterfaceAHandle](const TTypedElement<ITestTypedElementInterfaceA>& InElementHandle)
	{
		TestInterfaceAHandle(InElementHandle);
		return true;
	});

	// Test some elements with an interface they don't have
	ElementList->ForEachElement<ITestTypedElementInterfaceC>([&TestInterfaceCHandle](const TTypedElement<ITestTypedElementInterfaceC>& InElementHandle)
	{
		TestInterfaceCHandle(InElementHandle);
		return true;
	});

	ElementList.Reset();


	// Test the weak handles 
	FScriptTypedElementHandle ScriptHandleTyped = Registry->CreateScriptHandle(TypedElement1.GetId());
	check(ScriptHandleTyped.IsSet());

	FScriptTypedElementHandle ScriptHandleUntyped = Registry->CreateScriptHandle(UntypedElement1.GetId());
	check(ScriptHandleUntyped.IsSet());

	Registry->DestroyElement(TypedElement1);
	Registry->DestroyElement(TypedElement2);
	Registry->DestroyElement(TypedElement3);

	Registry->DestroyElement(UntypedElement1);
	Registry->DestroyElement(UntypedElement2);
	Registry->DestroyElement(UntypedElement3);

	check(!ScriptHandleTyped.IsSet());
	check(!ScriptHandleUntyped.IsSet());

	// Verify that there were no leaks
	Registry->ProcessDeferredElementsToDestroy();

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTypedElementRegistryPerfTest, "System.Runtime.TypedElementRegistry.PerfTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::PerfFilter)
bool FTypedElementRegistryPerfTest::RunTest(const FString& Parameters)
{
	constexpr int32 NumHandlesToTest = 1000000;

	UTypedElementRegistry* Registry = NewObject<UTypedElementRegistry>();

	const FName DummyElementType_Typed = "DummyElementType_Typed";
	Registry->RegisterElementType<FTestTypedElementData, true>(DummyElementType_Typed);
	Registry->RegisterElementInterface<ITestTypedElementInterfaceA>(DummyElementType_Typed, NewObject<UTestTypedElementInterfaceA_ImplTyped>());

	const FName DummyElementType_Untyped = "DummyElementType_Untyped";
	Registry->RegisterElementType(DummyElementType_Untyped, true);
	Registry->RegisterElementInterface<ITestTypedElementInterfaceA>(DummyElementType_Untyped, NewObject<UTestTypedElementInterfaceA_ImplUntyped>());

	TArray<TTypedElementOwner<FTestTypedElementData>> TypedOwnerHandles;
	TArray<FTypedElementOwner> UntypedOwnerHandles;
	FTypedElementListPtr ElementList = Registry->CreateElementList();

	// Create typed handles
	{
		TypedOwnerHandles.Reserve(NumHandlesToTest);
		{
			FScopedDurationTimeLogger Timer(FString::Printf(TEXT("Creating %d typed handles"), NumHandlesToTest));
			for (int32 Index = 0; Index < NumHandlesToTest; ++Index)
			{
				TypedOwnerHandles.Emplace(Registry->CreateElement<FTestTypedElementData>(DummyElementType_Typed));
			}
		}
	}

	// Create untyped handles
	{
		UntypedOwnerHandles.Reserve(NumHandlesToTest);
		{
			FScopedDurationTimeLogger Timer(FString::Printf(TEXT("Creating %d untyped handles"), NumHandlesToTest));
			for (int32 Index = 0; Index < NumHandlesToTest; ++Index)
			{
				UntypedOwnerHandles.Emplace(Registry->CreateElement(DummyElementType_Untyped, Index));
			}
		}
	}

	// Populate an element list with all handles
	{
		FScopedDurationTimeLogger Timer(FString::Printf(TEXT("Appending %d typed handles to list"), TypedOwnerHandles.Num() + UntypedOwnerHandles.Num()));
		ElementList->Reserve(TypedOwnerHandles.Num() + UntypedOwnerHandles.Num());
		ElementList->Append(TypedOwnerHandles);
		ElementList->Append(UntypedOwnerHandles);
	}

	// Find an interface from each handle
	{
		FScopedDurationTimeLogger Timer(FString::Printf(TEXT("Finding %d interfaces from list"), ElementList->Num()));
		ElementList->ForEachElementHandle([Registry](const FTypedElementHandle& InElementHandle)
		{
			Registry->GetElementInterface<ITestTypedElementInterfaceA>(InElementHandle);
			return true;
		});
	}

	// Find an element from each handle
	{
		FScopedDurationTimeLogger Timer(FString::Printf(TEXT("Finding %d elements from list"), ElementList->Num()));
		ElementList->ForEachElementHandle([Registry](const FTypedElementHandle& InElementHandle)
		{
			Registry->GetElement<ITestTypedElementInterfaceA>(InElementHandle);
			return true;
		});
	}

	// Enumerate all elements that implement an interface
	{
		FScopedDurationTimeLogger Timer(FString::Printf(TEXT("Enumerating %d elements in list"), ElementList->Num()));
		ElementList->ForEachElement<ITestTypedElementInterfaceA>([Registry](const TTypedElement<ITestTypedElementInterfaceA>& InElementHandle)
		{
			return true;
		});
	}

	// Clear the element list
	{
		FScopedDurationTimeLogger Timer(FString::Printf(TEXT("Reset %d elements in list"), ElementList->Num()));
		ElementList.Reset();
	}

	// Destroy typed handles
	{
		{
			FScopedDurationTimeLogger Timer(FString::Printf(TEXT("Destroying %d typed handles"), TypedOwnerHandles.Num()));
			for (TTypedElementOwner<FTestTypedElementData>& TypedOwnerHandle : TypedOwnerHandles)
			{
				Registry->DestroyElement(TypedOwnerHandle);
			}
		}
		TypedOwnerHandles.Reset();
	}

	// Destroy untyped handles
	{
		{
			FScopedDurationTimeLogger Timer(FString::Printf(TEXT("Destroying %d untyped handles"), UntypedOwnerHandles.Num()));
			for (FTypedElementOwner& UntypedOwnerHandle : UntypedOwnerHandles)
			{
				Registry->DestroyElement(UntypedOwnerHandle);
			}
		}
		UntypedOwnerHandles.Reset();
	}

	// Verify that there were no leaks
	Registry->ProcessDeferredElementsToDestroy();

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS

