// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementRegistry.h"
#include "UObject/Interface.h"

#include "TypedElementFrameworkTests.generated.h"

UINTERFACE(MinimalAPI, BlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UTestTypedElementInterfaceA : public UInterface
{
	GENERATED_BODY()
};

/**
 * Test interfaces
 */
class ITestTypedElementInterfaceA
{
	GENERATED_BODY()

public:
	virtual FText GetDisplayName(const FTypedElementHandle& InElementHandle) { return FText(); }

	virtual bool SetDisplayName(const FTypedElementHandle& InElementHandle, FText InNewName, bool bNotify = true) { return false; }

	UFUNCTION(BlueprintCallable, Category = "TypedElementFramework|Testing")
	virtual FText GetDisplayName(const FScriptTypedElementHandle& InElementHandle) { return FText(); }

	UFUNCTION(BlueprintCallable, Category = "TypedElementFramework|Testing")
	virtual bool SetDisplayName(const FScriptTypedElementHandle& InElementHandle, FText InNewName, bool bNotify = true) { return false; }
};

UINTERFACE(MinimalAPI, BlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UTestTypedElementInterfaceB : public UInterface
{
	GENERATED_BODY()
};

class ITestTypedElementInterfaceB
{
	GENERATED_BODY()

public:
	virtual bool MarkAsTested(const FTypedElementHandle& InElementHandle) { return false; }

	UFUNCTION(BlueprintCallable, Category = "TypedElementFramework|Testing")
	virtual bool MarkAsTested(const FScriptTypedElementHandle& InElementHandle) { return false; }
};

UINTERFACE(MinimalAPI, BlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UTestTypedElementInterfaceC : public UInterface
{
	GENERATED_BODY()
};

class ITestTypedElementInterfaceC
{
	GENERATED_BODY()

public:
	virtual bool GetIsTested(const FTypedElementHandle& InElementHandle) const { return false; }

	UFUNCTION(BlueprintCallable, Category = "TypedElementFramework|Testing")
	virtual bool GetIsTested(const FScriptTypedElementHandle& InElementHandle) const { return false; }
};

template <>
struct TTypedElement<ITestTypedElementInterfaceA> : public TTypedElementBase<ITestTypedElementInterfaceA>
{
	FText GetDisplayName() const { return InterfacePtr->GetDisplayName(*this); }
	bool SetDisplayName(FText InNewName, bool bNotify = true) const { return InterfacePtr->SetDisplayName(*this, InNewName, bNotify); }
};

template <>
struct TTypedElement<ITestTypedElementInterfaceB> : public TTypedElementBase<ITestTypedElementInterfaceB>
{
	bool MarkAsTested() { return InterfacePtr->MarkAsTested(*this); }
};

template <>
struct TTypedElement<ITestTypedElementInterfaceC> : public TTypedElementBase<ITestTypedElementInterfaceC>
{
	bool GetIsTested() const { return InterfacePtr->GetIsTested(*this); }
};

/**
 * Test dummy type
 */
struct FTestTypedElementData
{
	UE_DECLARE_TYPED_ELEMENT_DATA_RTTI(FTestTypedElementData);
	
	FName InternalElementId;
};

UCLASS()
class UTestTypedElementInterfaceA_ImplTyped : public UObject, public ITestTypedElementInterfaceA
{
	GENERATED_BODY()

public:
	virtual FText GetDisplayName(const FTypedElementHandle& InElementHandle) override;
	virtual bool SetDisplayName(const FTypedElementHandle& InElementHandle, FText InNewName, bool bNotify) override;
};

/**
 * Test untyped
 */
UCLASS()
class UTestTypedElementInterfaceA_ImplUntyped : public UObject, public ITestTypedElementInterfaceA
{
	GENERATED_BODY()

public:
	virtual FText GetDisplayName(const FTypedElementHandle& InElementHandle) override;
	virtual bool SetDisplayName(const FTypedElementHandle& InElementHandle, FText InNewName, bool bNotify) override;
};

/**
 * Test to two implementation in one object
 */
UCLASS()
class UTestTypedElementInterfaceBAndC_Typed : public UObject, public ITestTypedElementInterfaceB, public ITestTypedElementInterfaceC
{
	GENERATED_BODY()

public:
	virtual bool MarkAsTested(const FTypedElementHandle& InElementHandle) override;
	virtual bool GetIsTested(const FTypedElementHandle& InElementHandle) const override;
};

