// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "DataStorage/MapKey.h"

#include <algorithm>
#include "CommonTypes.generated.h"

/**
 * Base for the data structures for a column.
 */
USTRUCT()
struct FEditorDataStorageColumn
{
	GENERATED_BODY()
};

/**
 * Base for the data structures that act as tags to rows. Tags should not have any data.
 */
USTRUCT()
struct FEditorDataStorageTag
{
	GENERATED_BODY()
};

namespace UE
{
	// Work around missing header/implementations on some platforms
	namespace detail
	{
		template<typename T, typename U>
		concept SameHelper = std::is_same_v<T, U>;
	}
	template<typename T, typename U>
	concept same_as = detail::SameHelper<T, U> && detail::SameHelper<U, T>;

	template<typename From, typename To>
	concept convertible_to = std::is_convertible_v<From, To> && requires { static_cast<To>(std::declval<From>()); };

	template<typename Derived, typename Base>
	concept derived_from = std::is_base_of_v<Base, Derived> && std::is_convertible_v<const volatile Derived*, const volatile Base*>;

	template<class T>
	struct type_identity { using type = T; };

	template< class T >
	using type_identity_t = type_identity<T>::type;

	template <class T>
	constexpr T MakeDefault()
	{
		if constexpr (std::is_void_v<T>)
		{
			return;
		}
		else
		{
			return T{};
		}
	}

	namespace Editor::DataStorage
	{
		using FColumn = FEditorDataStorageColumn;
		using FTag = FEditorDataStorageTag;

		template<typename T>
		concept FunctionType =
			requires { &std::remove_reference_t<T>::operator(); }			// non-generic lambdas & single-operator() functors
			|| std::is_function_v<std::remove_pointer_t<std::decay_t<T>>>	// functions, function pointers & function references
			|| std::is_member_function_pointer_v<std::decay_t<T>>;			// member function pointers

		/**
		 * Defines a dynamic type for a value tag
		 * Example:
		 *   FValueTag ColorTagType(TEXT("Color"));
		 *   FValueTag DirectionTagType(TEXT("Direction"));
		 * A value tag can take on different values for each type.  This is set up when a tag is added to a row.
		 */
		class FValueTag
		{
		public:
			TYPEDELEMENTFRAMEWORK_API explicit FValueTag(const FName& InName);
			
			TYPEDELEMENTFRAMEWORK_API const FName& GetName() const;
			bool operator==(const FValueTag& Other) const = default;
		private:
			TYPEDELEMENTFRAMEWORK_API friend uint32 GetTypeHash(const FValueTag& InName);
			FName Name;
		};

		template<typename T>
		concept TValueTagType = std::is_same_v<T, FValueTag>;

		struct FDynamicColumnDescription
		{
			const UScriptStruct* TemplateType;
			FName Identifier;

			TYPEDELEMENTFRAMEWORK_API friend uint32 GetTypeHash(const FDynamicColumnDescription& Descriptor);
			bool operator==(const FDynamicColumnDescription&) const = default;
		};

		enum class ETableType : uint8
		{
			Invalid, //< Not a valid table.
			Declared, //< Table declared by user.
			Derivative, //< Table declared by users, based on another table.
			Variant, //< A dynamically generated variant of a declared or derivative table.
		};

		/** 
		 * Foreign keys provide a mechanic to generate a key that unique identifies a given row so it can be
		 * referenced by external sources.
		 */
		using FForeignKey = FMapKey;
		
		/**
		 * Short lived view into information about a table.
		 * Note: This struct is meant for short term usage only and becomes invalid the moment there's any change to a table.
		 */
		struct FTableInfoView
		{
			/** List of all dynamically created variants of this table. */
			TConstArrayView<TableHandle> Variants;
			/** All declared tables that use this table as a template. */
			TConstArrayView<TableHandle> Derivatives;
			/** The name of the table. */
			FName Name;
			/** The type of table. */
			ETableType Type = ETableType::Invalid;
			/**
			 * If this is a variant then the Parent represents the table used as the basis for the variant. If this is a derived table then
			 * the Parent represents the table that was passed in as the template for this table.
			 */
			TableHandle Parent = InvalidTableHandle;
		};

		struct FTableRegistrationOptions
		{
			/** If set, use the provided table as a source to copy from. Additional provided rows will be added to the copied layout. */
			TableHandle SourceTable = InvalidTableHandle;
			/** 
			 * Indicates the amount of memory a shard (a.k.a. chunk) of the table will take. Increasing this number means less memory 
			 * fragmentation and longer continuous time spend in a query callback while smaller sizes will decrease wasted memory. Making
			 * this too large yields diminishing returns as cache efficiency will plateau while distribution across threads will suffer as
			 * there are less shards to distribute. Making this too small though will cause increased overhead in managing tables and the
			 * overhead of calling query callbacks will relatively increase. Consider leaving this to default and only update if profiling
			 * indicates a benefit. Benefits may include going larger for very large tables since distribution among processors will be high
			 * but the cost of table management will go down. Decrease for tables that on average only have a small number of entries to
			 * save on memory.
			 * Set to 0 will indicate to use a default size or the editor specified override. Any other value will override any default or
			 * user setting.
			 */
			int32 MemorySize = 0;
		};

		// Standard callbacks.

		using RowCreationCallbackRef = TFunctionRef<void(RowHandle Row)>;
		using ColumnCreationCallbackRef = TFunctionRef<void(void* Column, const UScriptStruct& ColumnType)>;
		using ColumnListCallbackRef = TFunctionRef<void(const UScriptStruct& ColumnType)>;
		using ColumnListWithDataCallbackRef = TFunctionRef<void(void* Column, const UScriptStruct& ColumnType)>;
		using ColumnCopyOrMoveCallback = void (*)(const UScriptStruct& ColumnType, void* Destination, void* Source);

		template<typename T>
		concept THasDynamicColumnTemplateSpecifier = std::is_empty_v<typename T::EditorDataStorage_DynamicColumnTemplate>;
		
		template<typename T>
		concept TDynamicColumnTemplate = (UE::derived_from<T, FColumn> || UE::derived_from<T, FTag>) && THasDynamicColumnTemplateSpecifier<T>;
		
		// Template concepts to enforce type correctness.
		template<typename T>
		concept TDataColumnType = UE::derived_from<T, FColumn> && !THasDynamicColumnTemplateSpecifier<T>;

		template<typename T>
		concept TTagColumnType = UE::derived_from<T, FTag> && !THasDynamicColumnTemplateSpecifier<T>;
		
		template<typename T>
		concept TColumnType = TDataColumnType<T> || TTagColumnType<T>;

		template<typename T>
		concept TEnumType = std::is_enum_v<T>;

		/** Capture for variant names, e.g. for dynamic columns or hierarchies, so the name can be passed in as a template argument. */
		template <int N>
		struct TVariantName
		{
			static constexpr int Size = N;
			char Name[N > 1 ? (N - 1) : 1];

			constexpr TVariantName() requires (N == 0) = default;
			constexpr TVariantName(const char (&InName)[N]) requires (N > 0)
			{
				std::copy_n(InName, N - 1, Name);
			}

			FName GetName() const
			{
				return N > 1 ? FName(N - 1, Name) : NAME_None;
			}
		};
	} // namespace Editor::DataStorage
} // namespace UE
