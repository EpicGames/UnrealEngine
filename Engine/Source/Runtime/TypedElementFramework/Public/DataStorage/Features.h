// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Features/IModularFeatures.h"
#include "UObject/NameTypes.h"
#include "GenericPlatform/GenericPlatform.h"

namespace UE::Editor::DataStorage
{
	TYPEDELEMENTFRAMEWORK_API const extern FName StorageFeatureName;
	TYPEDELEMENTFRAMEWORK_API const extern FName CompatibilityFeatureName;
	TYPEDELEMENTFRAMEWORK_API const extern FName UiFeatureName;

	TYPEDELEMENTFRAMEWORK_API FSimpleMulticastDelegate& OnEditorDataStorageFeaturesEnabled();
	TYPEDELEMENTFRAMEWORK_API bool AreEditorDataStorageFeaturesEnabled();

	template<typename T>
	const T* GetDataStorageFeature(const FName InName)
	{
		if (IModularFeatures::Get().IsModularFeatureAvailable(InName))
		{
			return &IModularFeatures::Get().GetModularFeature<T>(InName);
		}
		return nullptr;
	}

	template<typename T>
	T* GetMutableDataStorageFeature(const FName InName)
	{
		if (IModularFeatures::Get().IsModularFeatureAvailable(InName))
		{
			return &IModularFeatures::Get().GetModularFeature<T>(InName);
		}
		return nullptr;
	}
} // namespace UE::Editor::DataStorage
