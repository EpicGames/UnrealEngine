// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/NumericLimits.h"
#include "Templates/TypeHash.h"

#include "Handles.generated.h"

namespace UE::Editor::DataStorage
{
	using TableHandle = uint64;
	static constexpr TableHandle InvalidTableHandle = TNumericLimits<TableHandle>::Max();

	using RowHandle = uint64;
	static constexpr RowHandle InvalidRowHandle = 0;

	using QueryHandle = uint64;
	static constexpr QueryHandle InvalidQueryHandle = TNumericLimits<QueryHandle>::Max();

	/**
	 * Handle to a registered relation type.
	 * Relation types define how two rows can be connected (e.g., SubclassOf, ImplementsInterface).
	 */
	using RelationTypeHandle = uint64;
	static constexpr RelationTypeHandle InvalidRelationTypeHandle = TNumericLimits<RelationTypeHandle>::Max();

	struct FHierarchyHandle
	{
		uint64 Reserved = 0;
		bool operator==(FHierarchyHandle Other) const
		{
			return Reserved == Other.Reserved;
		}
	};
} // namespace UE::Editor::DataStorage

/*
 * FTedsRowHandle is a strongly typed wrapper around UE::Editor::DataStorage::RowHandle and should only be used in cases where you need the extra info.
 * E.g for reflection/UHT or for template specializing something that needs to know the semantics of the row handle.
 * For all other cases, you should use the regular UE::Editor::DataStorage::RowHandle
 */
USTRUCT()
struct FTedsRowHandle
{
	GENERATED_BODY()

	UPROPERTY()
	uint64 RowHandle = UE::Editor::DataStorage::InvalidRowHandle;

	operator UE::Editor::DataStorage::RowHandle () const
	{
		return RowHandle;
	}

	FTedsRowHandle& operator=(UE::Editor::DataStorage::RowHandle InRowHandle)
	{
		RowHandle = InRowHandle;
		return *this;
	}

	bool IsValid() const
	{
		return RowHandle != UE::Editor::DataStorage::InvalidRowHandle;
	}
		
	friend uint32 GetTypeHash(const FTedsRowHandle& Key)
	{
		return GetTypeHash(Key.RowHandle);
	}
};

static_assert(sizeof(FTedsRowHandle::RowHandle) == sizeof(UE::Editor::DataStorage::RowHandle), "RowHandle and RowHandle wrapper sizes should match");

/*
 * FTedsTableHandle is a strongly typed wrapper around UE::Editor::DataStorage::TableHandle and should only be used in cases where you need the extra info.
 * E.g for reflection/UHT or for template specializing something that needs to know the semantics of the table handle.
 * For all other cases, you should use the regular UE::Editor::DataStorage::TableHandle
 */
USTRUCT()
struct FTedsTableHandle
{
	GENERATED_BODY()

	UPROPERTY()
	uint64 TableHandle = UE::Editor::DataStorage::InvalidTableHandle;

	operator UE::Editor::DataStorage::TableHandle() const
	{
		return TableHandle;
	}

	FTedsTableHandle& operator=(UE::Editor::DataStorage::TableHandle InTableHandle)
	{
		TableHandle = InTableHandle;
		return *this;
	}

	bool IsValid() const
	{
		return TableHandle != UE::Editor::DataStorage::InvalidTableHandle;
	}

	friend uint32 GetTypeHash(const FTedsTableHandle& Key)
	{
		return GetTypeHash(Key.TableHandle);
	}
};

static_assert(sizeof(FTedsTableHandle::TableHandle) == sizeof(UE::Editor::DataStorage::TableHandle), "TableHandle and TableHandle wrapper sizes should match");