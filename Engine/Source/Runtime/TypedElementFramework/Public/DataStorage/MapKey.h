// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "Misc/TVariant.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "UObject/SoftObjectPath.h"

class UObject;

namespace UE::Editor::DataStorage
{
	class FMapKeyView;
	struct FKeyIssueView;

	/**
	 * If the key encountered a problem the type will be switched to FKeyIssue and the description contains an explanation of the problem.
	 */
	struct FKeyIssue
	{
		FKeyIssue() = default;
		TYPEDELEMENTFRAMEWORK_API FKeyIssue(const FString::ElementType* Description);
		TYPEDELEMENTFRAMEWORK_API FKeyIssue(FString Description);
		TYPEDELEMENTFRAMEWORK_API FKeyIssue(FStringView Description);
		TYPEDELEMENTFRAMEWORK_API FKeyIssue(const FKeyIssueView& Description);

		bool operator==(const FKeyIssue& Rhs) const = default;
		bool operator!=(const FKeyIssue& Rhs) const = default;
		TYPEDELEMENTFRAMEWORK_API bool operator==(const FKeyIssueView& Rhs) const;
		TYPEDELEMENTFRAMEWORK_API bool operator!=(const FKeyIssueView& Rhs) const;

		FString Description;
	};

	struct FKeyIssueView
	{
		FKeyIssueView() = default;
		TYPEDELEMENTFRAMEWORK_API FKeyIssueView(FStringView Description);
		TYPEDELEMENTFRAMEWORK_API FKeyIssueView(const FKeyIssue& Description);

		bool operator==(const FKeyIssueView& Rhs) const = default;
		bool operator!=(const FKeyIssueView& Rhs) const = default;
		TYPEDELEMENTFRAMEWORK_API bool operator==(const FKeyIssue& Rhs) const;
		TYPEDELEMENTFRAMEWORK_API bool operator!=(const FKeyIssue& Rhs) const;

		FStringView Description;
	};

	class FMapKey
	{
		friend class FMapKeyView;
		friend struct FKeyComparer;
		friend struct FKeyCopy;
		friend struct FKeyMove;

	public:
		FMapKey() = default;
		
		TYPEDELEMENTFRAMEWORK_API FMapKey(const FMapKey& Key);
		TYPEDELEMENTFRAMEWORK_API FMapKey(FMapKey&& Rhs);
		TYPEDELEMENTFRAMEWORK_API explicit FMapKey(FKeyIssue Key);
		TYPEDELEMENTFRAMEWORK_API explicit FMapKey(const void* Key);
		TYPEDELEMENTFRAMEWORK_API explicit FMapKey(const UObject* Key);
		TYPEDELEMENTFRAMEWORK_API explicit FMapKey(int64 Key);
		TYPEDELEMENTFRAMEWORK_API explicit FMapKey(uint64 Key);
		TYPEDELEMENTFRAMEWORK_API explicit FMapKey(float Key);
		TYPEDELEMENTFRAMEWORK_API explicit FMapKey(double Key);
		TYPEDELEMENTFRAMEWORK_API explicit FMapKey(FString Key);
		TYPEDELEMENTFRAMEWORK_API explicit FMapKey(FName Key);
		TYPEDELEMENTFRAMEWORK_API explicit FMapKey(FSoftObjectPath Key);

		TYPEDELEMENTFRAMEWORK_API FMapKey& operator=(const FMapKey& Rhs);
		TYPEDELEMENTFRAMEWORK_API FMapKey& operator=(FMapKey&& Rhs);

		TYPEDELEMENTFRAMEWORK_API bool operator==(const FMapKey& Rhs) const;
		TYPEDELEMENTFRAMEWORK_API bool operator==(const FMapKeyView& Rhs) const;
		TYPEDELEMENTFRAMEWORK_API bool operator!=(const FMapKey& Rhs) const;
		TYPEDELEMENTFRAMEWORK_API bool operator!=(const FMapKeyView& Rhs) const;

		TYPEDELEMENTFRAMEWORK_API uint64 CalculateHash() const;
		TYPEDELEMENTFRAMEWORK_API bool IsSet() const;
		TYPEDELEMENTFRAMEWORK_API void Clear();
		TYPEDELEMENTFRAMEWORK_API FString ToString() const;

		template<typename T>
		bool IsType() const { return Key.IsType<T>(); }
		template<typename T>
		const T& GetAs() const { return Key.Get<T>(); }

	private:
		using KeyType = TVariant<FEmptyVariantState, FKeyIssue, const void*, const UObject*, int64, uint64, float, double, FString, FName, TUniquePtr<FSoftObjectPath>>;

		KeyType Key;
	};

	class FMapKeyView
	{
		friend struct FKeyToKeyViewConverter;
		friend struct FKeyViewComparer;

	public:
		FMapKeyView() = default;
		TYPEDELEMENTFRAMEWORK_API FMapKeyView(const FMapKey& InKey);
		TYPEDELEMENTFRAMEWORK_API explicit FMapKeyView(const FKeyIssue& Key);
		TYPEDELEMENTFRAMEWORK_API explicit FMapKeyView(FKeyIssueView Key);
		TYPEDELEMENTFRAMEWORK_API explicit FMapKeyView(const void* Key);
		TYPEDELEMENTFRAMEWORK_API explicit FMapKeyView(const UObject* Key);
		TYPEDELEMENTFRAMEWORK_API explicit FMapKeyView(int64 Key);
		TYPEDELEMENTFRAMEWORK_API explicit FMapKeyView(uint64 Key);
		TYPEDELEMENTFRAMEWORK_API explicit FMapKeyView(float Key);
		TYPEDELEMENTFRAMEWORK_API explicit FMapKeyView(double Key);
		TYPEDELEMENTFRAMEWORK_API explicit FMapKeyView(const FString& Key);
		TYPEDELEMENTFRAMEWORK_API explicit FMapKeyView(FStringView Key);
		TYPEDELEMENTFRAMEWORK_API explicit FMapKeyView(const FName& Key);
		TYPEDELEMENTFRAMEWORK_API explicit FMapKeyView(const FSoftObjectPath& Key);

		TYPEDELEMENTFRAMEWORK_API FMapKeyView& operator=(const FMapKey& InKey);

		TYPEDELEMENTFRAMEWORK_API FMapKey CreateKey() const;

		TYPEDELEMENTFRAMEWORK_API bool operator==(const FMapKey& Rhs) const;
		TYPEDELEMENTFRAMEWORK_API bool operator==(const FMapKeyView& Rhs) const;
		TYPEDELEMENTFRAMEWORK_API bool operator!=(const FMapKey& Rhs) const;
		TYPEDELEMENTFRAMEWORK_API bool operator!=(const FMapKeyView& Rhs) const;

		TYPEDELEMENTFRAMEWORK_API uint64 CalculateHash() const;
		TYPEDELEMENTFRAMEWORK_API FString ToString() const;

		template<typename T>
		bool IsType() const { return Key.IsType<T>(); }
		template<typename T>
		const T& GetAs() const { return Key.Get<T>(); }

	private:
		using KeyViewType = TVariant<FEmptyVariantState, FKeyIssueView, const void*, const UObject*, int64, uint64, float, double, FStringView, const FName*, const FSoftObjectPath*>;

		KeyViewType Key;
	};
}
// namespace UE::Editor::DataStorage
