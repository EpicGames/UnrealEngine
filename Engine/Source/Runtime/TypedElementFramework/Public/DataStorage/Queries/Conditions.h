// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <algorithm>
#include "DataStorage/CommonTypes.h"
#include "Templates/SubclassOf.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UScriptStruct;

namespace UE::Editor::DataStorage
{
	class ICoreProvider;

namespace Queries
{
	struct FColumnBase
	{
		TWeakObjectPtr<const UScriptStruct> TypeInfo = nullptr;
		FName Identifier = NAME_None;

	protected:
		FColumnBase() = default;
		explicit FColumnBase(TWeakObjectPtr<const UScriptStruct> ColumnTypeInfo, const FName& InIdentifier)
			: TypeInfo(ColumnTypeInfo)
			, Identifier(InIdentifier)
		{}
	};

	template<typename T, TVariantName Id = TVariantName<0>()>
	struct TColumn final : public FColumnBase
	{
		explicit TColumn() : FColumnBase(T::StaticStruct(), NAME_None) {}
		explicit TColumn(const FName& Identifier) : FColumnBase(T::StaticStruct(), Identifier) {}
	};

	template<typename T, TVariantName Id> requires (decltype(Id)::Size > 0)
	struct TColumn<T, Id> final : public FColumnBase
	{
		explicit TColumn() : FColumnBase(T::StaticStruct(), []()
			{
				static FName Name = Id.GetName();
				return Name;
			}()) {}
	};

	template<>
	struct TColumn<void> final : public FColumnBase
	{
		explicit TColumn(TWeakObjectPtr<const UScriptStruct> ColumnTypeInfo) : FColumnBase(ColumnTypeInfo, NAME_None) {}
		explicit TColumn(TWeakObjectPtr<const UScriptStruct> ColumnTypeInfo, const FName& Identifier)
			: FColumnBase(ColumnTypeInfo, Identifier)
		{
		}
	};
	TColumn(TWeakObjectPtr<const UScriptStruct>) -> TColumn<void>;
	TColumn(TWeakObjectPtr<const UScriptStruct>, const FName&) -> TColumn<void>;

	template<typename T>
	concept TColumnDescType = std::is_base_of_v<FColumnBase, T>;

	/*
	 * Compile Context used to resolve dynamic columns when FConditions are compiled
	 */
	class IQueryConditionCompileContext 
	{
	public:
		virtual ~IQueryConditionCompileContext() = default;
		virtual const UScriptStruct* GenerateDynamicColumn(const FDynamicColumnDescription&) const = 0;
	};

	/*
	 * Specialized compile context that accepts an ICoreProvider
	 */
	class FEditorStorageQueryConditionCompileContext : public IQueryConditionCompileContext 
	{
	public:
		TYPEDELEMENTFRAMEWORK_API explicit FEditorStorageQueryConditionCompileContext(ICoreProvider* InDataStorage);
		TYPEDELEMENTFRAMEWORK_API virtual const UScriptStruct* GenerateDynamicColumn(const FDynamicColumnDescription& Description) const override;
		
	private:
		ICoreProvider* DataStorage;
	};


	/**
	 * Product of boolean combination of multiple columns. This can be used to verify if a collection of columns match
	 * the stored columns.
	 * NOTE: You must call Compile() before you call any members accessing the conditions
	 */
	class FConditions final
	{
	public:
		using ContainsCallback = TFunctionRef<bool(uint8_t ColumnIndex, TWeakObjectPtr<const UScriptStruct> Column)>;

		TYPEDELEMENTFRAMEWORK_API friend FConditions operator&&(const FConditions& Lhs, FColumnBase Rhs);
		TYPEDELEMENTFRAMEWORK_API friend FConditions operator&&(const FConditions& Lhs, const FConditions& Rhs);
		TYPEDELEMENTFRAMEWORK_API friend FConditions operator&&(FColumnBase Lhs, FColumnBase Rhs);
		TYPEDELEMENTFRAMEWORK_API friend FConditions operator&&(FColumnBase Lhs, const FConditions& Rhs);

		TYPEDELEMENTFRAMEWORK_API friend FConditions operator||(const FConditions& Lhs, FColumnBase Rhs);
		TYPEDELEMENTFRAMEWORK_API friend FConditions operator||(const FConditions& Lhs, const FConditions& Rhs);
		TYPEDELEMENTFRAMEWORK_API friend FConditions operator||(FColumnBase Lhs, FColumnBase Rhs);
		TYPEDELEMENTFRAMEWORK_API friend FConditions operator||(FColumnBase Lhs, const FConditions& Rhs);

		TYPEDELEMENTFRAMEWORK_API FConditions();
		// Not marked as "explicit" to allow conversion from a column. This means that conditions with a single
		// argument can be written in the same way as ones that use combinations.
		TYPEDELEMENTFRAMEWORK_API FConditions(FColumnBase Column);

		// Compile must be called before using any functions that access the columns
		TYPEDELEMENTFRAMEWORK_API FConditions& Compile(const IQueryConditionCompileContext& CompileContext);

		// Check whether these query conditions have been compiled
		TYPEDELEMENTFRAMEWORK_API bool IsCompiled() const;

		/** Convert the conditions into a string and append them to the provided string. */
		TYPEDELEMENTFRAMEWORK_API void AppendToString(FString& Output) const;

		/** Runs the provided list of columns through the conditions and returns true if a valid combination of columns is found. */
		TYPEDELEMENTFRAMEWORK_API bool Verify(TConstArrayView<FColumnBase> AvailableColumns) const;
		/**
		 * Runs the provided list of columns through the conditions and returns true if a valid combination of columns is found.
		 * This version returns a list of the columns that were used to match the condition.
		 */
		TYPEDELEMENTFRAMEWORK_API bool Verify(
			TArray<TWeakObjectPtr<const UScriptStruct>>& MatchedColumns,
			TConstArrayView<FColumnBase> AvailableColumns,
			bool AvailableColumnsAreSorted = false) const;
		/** Runs the provided list of columns through the conditions and returns true if a valid combination of columns is found. */
		TYPEDELEMENTFRAMEWORK_API bool Verify(TConstArrayView<TWeakObjectPtr<const UScriptStruct>> AvailableColumns,
			bool AvailableColumnsAreSorted = false) const;
		/**
		 * Runs the provided list of columns through the conditions and returns true if a valid combination of columns is found.
		 * This version returns a list of the columns that were used to match the condition.
		 */
		TYPEDELEMENTFRAMEWORK_API bool Verify(
			TArray<TWeakObjectPtr<const UScriptStruct>>& MatchedColumns,
			TConstArrayView<TWeakObjectPtr<const UScriptStruct>> AvailableColumns,
			bool AvailableColumnsAreSorted = false) const;
		/** Runs the provided list of columns through the conditions and returns true if a valid combination of columns is found. */
		TYPEDELEMENTFRAMEWORK_API bool Verify(TSet<TWeakObjectPtr<const UScriptStruct>> AvailableColumns) const;
		/** Runs through the list of query conditions and uses the callback to verify if a column is available. */
		TYPEDELEMENTFRAMEWORK_API bool Verify(ContainsCallback Callback) const;

		/** Returns the minimum number of columns needed for a successful match. */
		TYPEDELEMENTFRAMEWORK_API uint8_t MinimumColumnMatchRequired() const;
		/** Returns a list of all columns used. This can include duplicate columns. */
		TYPEDELEMENTFRAMEWORK_API TConstArrayView<TWeakObjectPtr<const UScriptStruct>> GetColumns() const;
		/** Whether or not there are any columns registered for operation.*/
		TYPEDELEMENTFRAMEWORK_API bool IsEmpty() const;
		/** Whether the conditions contain any columns that are dynamic templates. */
		TYPEDELEMENTFRAMEWORK_API bool UsesDynamicTemplates() const;

	private:
		template<typename AvailableColumnType, typename ProjectionFunction>
		bool VerifyWithDynamicColumn(TConstArrayView<AvailableColumnType> AvailableColumns, uint64& OutMatches, ProjectionFunction Projection) const;
		
		void AppendName(FString& Output, TWeakObjectPtr<const UScriptStruct> TypeInfo) const;

		bool EntersScopeNext(uint8_t Index) const;
		bool EntersScope(uint8_t Index) const;
		bool Contains(TWeakObjectPtr<const UScriptStruct> ColumnType, const TArray<FColumnBase>& AvailableColumns) const;

		bool VerifyBootstrap(ContainsCallback Contains) const;
		bool VerifyRange(uint8_t& TokenIndex, uint8_t& ColumnIndex, ContainsCallback Contains) const;

		void ConvertColumnBitToArray(TArray<TWeakObjectPtr<const UScriptStruct>>& MatchedColumns, uint64 ColumnBits) const;

		uint8_t MinimumColumnMatchRequiredRange(uint8_t& Front) const;

		static void AppendQuery(FConditions& Target, const FConditions& Source);

		static constexpr SIZE_T MaxColumnCount = 32;
		static constexpr SIZE_T MaxTokenCount = 64;

		enum class Token : uint8_t
		{
			None,
			And,
			Or,
			ScopeOpen,
			ScopeClose
		};

		enum class EColumnFlags : uint8_t
		{
			DynamicColumnTemplate = 1 << 0
		};
		FRIEND_ENUM_CLASS_FLAGS(EColumnFlags);
		
		using ColumnArray = TArray<TWeakObjectPtr<const UScriptStruct>, TInlineAllocator<MaxColumnCount>>;
		using TokenArray = TArray<Token, TInlineAllocator<MaxTokenCount>>;
		using IdentifierArray = TArray<FName, TInlineAllocator<MaxColumnCount>>;
		using ColumnFlagArray = TArray<EColumnFlags, TInlineAllocator<MaxColumnCount>>;

		ColumnArray Columns;
		ColumnFlagArray ColumnFlags;
		TokenArray Tokens;
		IdentifierArray Identifiers;
		uint8_t ColumnCount = 0;
		uint8_t TokenCount = 0;
		bool bIsCompiled = false;
	};

	TYPEDELEMENTFRAMEWORK_API FConditions operator&&(const FConditions& Lhs, FColumnBase Rhs);
	TYPEDELEMENTFRAMEWORK_API FConditions operator&&(const FConditions& Lhs, const FConditions& Rhs);
	TYPEDELEMENTFRAMEWORK_API FConditions operator&&(FColumnBase Lhs, FColumnBase Rhs);
	TYPEDELEMENTFRAMEWORK_API FConditions operator&&(FColumnBase Lhs, const FConditions& Rhs);

	TYPEDELEMENTFRAMEWORK_API FConditions operator||(const FConditions& Lhs, FColumnBase Rhs);
	TYPEDELEMENTFRAMEWORK_API FConditions operator||(const FConditions& Lhs, const FConditions& Rhs);
	TYPEDELEMENTFRAMEWORK_API FConditions operator||(FColumnBase Lhs, FColumnBase Rhs);
	TYPEDELEMENTFRAMEWORK_API FConditions operator||(FColumnBase Lhs, const FConditions& Rhs);
} // namespace Queries
} // namespace UE::Editor::DataStorage

