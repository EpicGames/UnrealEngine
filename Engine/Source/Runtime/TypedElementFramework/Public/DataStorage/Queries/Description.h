// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "DataStorage/CommonTypes.h"
#include "DataStorage/Handles.h"
#include "DataStorage/Queries/Types.h"
#include "DataStorage/Queries/Conditions.h"
#include "Elements/Interfaces/TypedElementQueryStorageInterfaces.h"
#include "Elements/Framework/TypedElementMetaData.h"
#include "Misc/Optional.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UScriptStruct;

namespace UE::Editor::DataStorage
{
	struct FQueryDescription;
	struct IQueryContext;
	
	using QueryCallback = TFunction<void(const FQueryDescription&, IQueryContext&)>;
	using QueryCallbackRef = TFunctionRef<void(const FQueryDescription&, IQueryContext&)>;
	using DirectQueryCallback = TFunction<void(const FQueryDescription&, IDirectQueryContext&)>;
	using DirectQueryCallbackRef = TFunctionRef<void(const FQueryDescription&, IDirectQueryContext&)>;
	
	struct FQueryDescription final
	{
		static constexpr int32 NumInlineSelections = 8;
		static constexpr int32 NumInlineConditions = 8;
		static constexpr int32 NumInlineDependencies = 2;
		static constexpr int32 NumInlineGroups = 2;

		enum class EActionType : uint8
		{
			None,	//< Do nothing.
			Select,	//< Selects a set of columns for further processing.
			Count,	//< Counts the number of entries that match the filter condition.

			Max //< Value indicating the maximum value in this enum. Not to be used as an enum value.
		};

		enum class EOperatorType : uint16
		{
			SimpleAll,			//< Unary: Type
			SimpleAny,			//< Unary: Type
			SimpleNone,			//< Unary: Type
			SimpleOptional,		//< Unary: Type
		
			Max //< Value indicating the maximum value in this enum. Not to be used as an enum value.
		};

		union FOperator
		{
			TWeakObjectPtr<const UScriptStruct> Type;
		};

		struct FValueTagData
		{
			// The Tag maps to a Mass ConstSharedFragment object
			FValueTag Tag;

			// The MatchValue specifies the value that the fragment must have to be matched
			// If MatchValue is NAME_None, then TEDS will match all values
			FName MatchValue;
		};

		/**
		 * Specifies the role a row plays in a relation.
		 */
		enum class ERelationRole : uint8
		{
			/** The "from" / "child" side of a relation */
			Subject,
			/** The "to" / "parent" side of a relation */
			Object
		};

		/**
		 * Condition for filtering rows by their participation in relations.
		 * If TargetRow is set, only matches relations with that specific counterpart.
		 */
		struct FRelationCondition
		{
			/** The relation type to filter by */
			RelationTypeHandle RelationType = InvalidRelationTypeHandle;

			/** Whether we're looking for rows as subjects or objects of the relation */
			ERelationRole Role = ERelationRole::Subject;

			/** Optional: If set, only match relations where the counterpart is this specific row */
			RowHandle TargetRow = InvalidRowHandle;
		};

		/**
		 * Condition for filtering rows by their position in a hierarchical relation.
		 * Used for IsDescendantOf/IsAncestorOf queries.
		 */
		struct FHierarchyCondition
		{
			/** The hierarchical relation type to traverse */
			RelationTypeHandle RelationType = InvalidRelationTypeHandle;

			/** The reference row to check ancestry/descendancy against */
			RowHandle ReferenceRow = InvalidRowHandle;

			/** Whether to include the reference row itself in the results */
			bool bIncludeSelf = false;
		};

		/**
		 * A single atomic relation predicate. Multiple predicates are AND-composed.
		 * Unifies the four existing condition types into one value type.
		 */
		struct FRelationPredicate
		{
			enum class EType : uint8
			{
				SubjectOf,      //< Row must be a subject of the relation (optionally with a specific object)
				ObjectOf,       //< Row must be an object of the relation (optionally with a specific subject)
				DescendantOf,   //< Row must be a descendant of ReferenceRow in a hierarchical relation
				AncestorOf,     //< Row must be an ancestor of ReferenceRow in a hierarchical relation
			};

			EType              Type         = EType::SubjectOf;
			RelationTypeHandle RelationType = InvalidRelationTypeHandle;

			/** For SubjectOf/ObjectOf: the specific counterpart row, or InvalidRowHandle to match any.
			 *  For DescendantOf: the ancestor to test against.
			 *  For AncestorOf: the descendant to test against. */
			RowHandle          ReferenceRow = InvalidRowHandle;

			/** For DescendantOf/AncestorOf: if true, the reference row itself also passes. */
			bool               bIncludeSelf = false;
		};

		struct FCallbackData
		{
			// Necessary to avoid deprecation warnings in clang when using compiler generated copy and move ctors. Can
			// be removed when ActivationCount is removed. Also causes spurious warnings in msvc, hence the conditional
			// compilation.
			#ifdef __clang__
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FCallbackData() = default;
			FCallbackData(const FCallbackData&) = default;
			FCallbackData& operator=(const FCallbackData&) = default;
			FCallbackData(FCallbackData&&) = default;
			FCallbackData& operator=(FCallbackData&&) = default;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			#endif

			TArray<FName, TInlineAllocator<NumInlineGroups>> BeforeGroups;
			TArray<FName, TInlineAllocator<NumInlineGroups>> AfterGroups;
			QueryCallback Function;
			FName Name;
			FName Group;
			/** If a name is set, it indicates the query callback will not be run unless explicitly activated with ActivateQueries. */
			FName ActivationName;
			const UScriptStruct* MonitoredType{ nullptr };
			EQueryCallbackType Type{ EQueryCallbackType::None };
			EQueryTickPhase Phase{ EQueryTickPhase::FrameEnd };
			bool Active = true;

			UE_DEPRECATED("5.7", "Activation counts are no longer supported.")
			uint8 ActivationCount;

			EExecutionMode ExecutionMode = EExecutionMode::Default;
		};
		FCallbackData Callback;

		// The list of arrays below are required to remain in the same order as they're added as the function binding expects certain entries
		// to be in a specific location.

		TArray<TWeakObjectPtr<const UScriptStruct>, TInlineAllocator<NumInlineSelections>> SelectionTypes;
		TArray<EQueryAccessType, TInlineAllocator<NumInlineSelections>> SelectionAccessTypes;
		TArray<FColumnMetaData, TInlineAllocator<NumInlineSelections>> SelectionMetaData;

		TArray<EOperatorType, TInlineAllocator<NumInlineConditions>> ConditionTypes;
		TArray<FOperator, TInlineAllocator<NumInlineConditions>> ConditionOperators;

		TArray<FDynamicColumnDescription, TInlineAllocator<NumInlineSelections>> DynamicSelectionTypes;
		TArray<EQueryAccessType, TInlineAllocator<NumInlineSelections>> DynamicSelectionAccessTypes;
		TArray<FColumnMetaData::EFlags, TInlineAllocator<NumInlineSelections>> DynamicSelectionMetaData;
		
		TArray<EOperatorType, TInlineAllocator<NumInlineConditions>> DynamicConditionOperations;
		TArray<FDynamicColumnDescription, TInlineAllocator<NumInlineConditions>> DynamicConditionDescriptions;

		TArray<FValueTagData> ValueTags;

		TOptional<UE::Editor::DataStorage::Queries::FConditions> Conditions;

		TArray<TWeakObjectPtr<const UClass>, TInlineAllocator<NumInlineDependencies>> DependencyTypes;
		TArray<EQueryDependencyFlags, TInlineAllocator<NumInlineDependencies>> DependencyFlags;
		/** Cached instances of the dependencies. This will always match the count of the other Dependency*Types, but may contain null pointers. */
		TArray<TWeakObjectPtr<UObject>, TInlineAllocator<NumInlineDependencies>> CachedDependencies;
		TArray<QueryHandle> Subqueries;
		FMetaData MetaData;

		FName Hierarchy;

		/** Relation conditions for filtering rows by relation participation */
		TArray<FRelationCondition, TInlineAllocator<2>> RelationConditions;

		/** Descendant-of conditions for hierarchical filtering */
		TArray<FHierarchyCondition, TInlineAllocator<1>> DescendantOfConditions;

		/** Ancestor-of conditions for hierarchical filtering */
		TArray<FHierarchyCondition, TInlineAllocator<1>> AncestorOfConditions;

		EActionType Action;
		bool bShouldBatchModifications = false;
	};
} // namespace UE::Editor::DataStorage
