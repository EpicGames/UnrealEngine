// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"
#include <atomic>

namespace UE::Editor::DataStorage
{
	// UE_DEPRECATED(5.6, "Use FMapKey(View) with the new mapping functions.") // Disabled as it's used by the deprecated functions.
	using IndexHash = uint64;

	enum class EQueryTickPhase : uint8
	{
		PrePhysics, //< Queries are executed before physics simulation starts.
		DuringPhysics, //< Queries that can be run in parallel with physics simulation work.
		PostPhysics, //< Queries that need rigid body and cloth simulation to be completed before being executed.
		FrameEnd, //< Catchall for queries demoted to the last possible moment.

		Max //< Value indicating the maximum value in this enum. Not to be used as an enum value.
	};

	enum class EQueryTickGroups : uint8
	{
		/** The standard group to run work in. */
		Default,

		/** General group that prepares for updating. */
		PreUpdate,

		/** General group that's used for updating data. */
		Update,

		/** General group that completes for updating. */
		PostUpdate,

		/**
		 * The group for queries that need to sync data from external sources such as subsystems or the world into
		 * the Data Storage. These typically run early in a phase.
		 */
		SyncExternalToDataStorage,
		/**
		 * The group for queries that need to sync data from the Data Storage to external sources such as subsystems
		 * or the world into. These typically run late in a phase.
		 */
		SyncDataStorageToExternal,
		/**
		 * Queries grouped under this name will sync data to/from widgets.
		 */
		SyncWidgets,

		Max //< Value indicating the maximum value in this enum. Not to be used as an enum value.
	};

	enum class EQueryCallbackType : uint8
	{
		/** No callback provided. */
		None,
		/** The query will be run every tick if at least one row matches. */
		Processor,
		/** The query will be run when a row is added that matches the query. The first recorded column will be actively monitored for changes. */
		ObserveAdd,
		/** The query will be run when a row is removed that matches the query. The first recorded column will be actively monitored for changes. */
		ObserveRemove,
		/**
		 * At the start of the assigned phase this query will run if there are any matches. These queries will have any deferred operations such as
		 * adding/removing rows/columns executed before the phase starts. This introduces sync points that hinder performance and are therefore
		 * recommended only for queries that save on work later in the phase such as repeated checks for validity.
		 */
		PhasePreparation,
		/**
		 * At the end of the assigned phase this query will run if there are any matches. These queries will have any deferred operations such as
		 * adding/removing rows/columns executed before the phase ends. This introduces sync points that hinder performance and are therefore
		 * recommended only cases where delaying deferred operations is not possible e.g. when tables are known to be referenced outside the update
		 * cycle.
		 */
		PhaseFinalization,

		Max //< Value indicating the maximum value in this enum. Not to be used as an enum value.
	};

	enum class EExecutionMode : uint8
	{
		Default, //< Use the default for the query callback type or the group default.
		GameThread, //< The query callback will be executed sequentially on the game thread.
		Threaded, //< The entire query callback is called on a separate thread, while respecting dependencies.
		ThreadedChunks //< The query callback is called on a separate thread for each chunk. This can not be set if there are sub-queries.
	};

	enum class EQueryAccessType : uint8
	{
		/** Provide read-only access. */
		ReadOnly,
		/** Provide read-only access, but do not include when finding matching tables. */
		OptionalReadOnly,
		/** Provide read and write access. */
		ReadWrite,
		/**/
		NUM
	};

	enum class EQueryDependencyFlags : uint8
	{
		None = 0,
		/** If set the dependency is accessed as read-only. If not set the dependency requires Read/Write access. */
		ReadOnly = 1 << 0,
		/** If set the dependency can only be used from the game thread, otherwise it can be accessed from any thread. */
		GameThreadBound = 1 << 1,
		/** If set the dependency will be re-fetched every iteration, otherwise only if not fetched before. */
		AlwaysRefresh = 1 << 2
	};
	ENUM_CLASS_FLAGS(EQueryDependencyFlags);

	struct FQueryResult final
	{
		enum class ECompletion
		{
			/** Query could be fully executed. */
			Fully,
			/** Only portions of the query were executed. This is caused by a problem that was encountered partway through processing. */
			Partially,
			/**
			 * The back-end doesn't support the particular query. This may be a limitation in how/where the query is run or because
			 * the query contains actions and/or operations that are not supported.
			 */
			Unsupported,
			/** The provided query is no longer available. */
			Unavailable,
			/** 
			 * The provided callback is not compatible with the query. This can be due to missing columns in the selection or the access
			 * to a column is more restricted than the callback requests.
			 */
			Incompatible,
			/** One or more dependencies declared on the query could not be retrieved. */
			MissingDependency
		};

		FQueryResult() = default;
		FQueryResult(const FQueryResult& Other)
			: Count(Other.Count.load())
			, Completed(Other.Completed)
		{}
		
		FQueryResult(FQueryResult&& Other) noexcept
			: Count(Other.Count.load())
			, Completed(Other.Completed)
		{}
		
		FQueryResult& operator=(const FQueryResult& Other)
		{
			Count.store(Other.Count.load());
			Completed = Other.Completed;
			return *this;
		}
		
		FQueryResult& operator=(FQueryResult&& Other) noexcept
		{
			Count.store(Other.Count.load());
			Completed = Other.Completed;
			return *this;
		}

		/** The number of rows that were processed. */
		std::atomic<uint32> Count{ 0 };
		ECompletion Completed{ ECompletion::Unavailable };
	};
} // namespace UE::Editor::DataStorage
