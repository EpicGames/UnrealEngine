// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "DataStorage/Scope/EditorDataScopeTypes.h"
#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

/**
 * Editor Scope Tree
 * ====================================
 *
 * Provides a hierarchy-based context propagation mechanism built on TEDS.
 * Editor components publish contextual data (selection, active object, etc.) at their level in a
 * named hierarchy. Downstream consumers read contextual data by walking up the hierarchy from their
 * own scope row until a matching column is found.
 *
 * Key concepts:
 *   - Scope Row: A row representing an editor component's scope, created via ICoreProvider::AddScopeRow
 *   - Thread-Local Current Context: An ambient scope row set per-thread, used as the default for lookups
 *   - Hierarchy Walk: GetScopeData walks parent links to find the nearest ancestor with data
 *   - Versioned: SetScopeData/RemoveScopeData bump a per-row version counter for change detection
 *
 * === Example: Panel Publishing Context ===
 *   using namespace UE::Editor::DataStorage;
 *   using namespace UE::Editor::DataStorage::Scope;
 *   ICoreProvider& Storage = *GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
 *   RowHandle PanelCtx = Storage.AddScopeRow();
 *   Storage.SetParentScope(PanelCtx, ParentCtx);
 *   Storage.SetScopeData<FSelectionColumn>(PanelCtx, FSelectionColumn{.Object = MyObj});
 *
 * === Example: Downstream Consumer ===
 *   FPushScopeGuard Guard(PanelCtx);
 *   // The widget's Construct would have CurrentContext
 *   // be PanelCtx without knowing its parent.
 *   SNew(<SomeSubWidget>);
 *   // Works for non slate too.  ContextData is slate agnostic.
 *
 * === Example: Change Detection ===
 *   FScopeDataVersion Cached;
 *   void Tick() {
 *       FScopeDataVersion V = Storage.GetScopeDataVersion<FSelectionColumn>(MyCtx);
 *       if (V != Cached) { Cached = V; RefreshUI(); }
 *   }
 *   // No SlateAttribute bindings yet, but above _could_ be done within an attribute to check if
 *   // dependent data changed and invalidate widget if so
 */

namespace UE::Editor::DataStorage::Scope
{

// ============================================================================
// Thread-Local Current Context
// ============================================================================

/** Returns the thread-local current scope row handle. InvalidRowHandle if none set. */
TYPEDELEMENTFRAMEWORK_API RowHandle GetCurrentScope();

/** Sets the thread-local current scope row handle. Returns the previous value.
 *  Prefer FPushScopeGuard for RAII usage. */
TYPEDELEMENTFRAMEWORK_API RowHandle SetCurrentScope(RowHandle NewContext);

/**
 * RAII guard that pushes a new scope row as the thread-local current context.
 * On destruction, restores the previous context. Supports nesting.
 */
class FPushScopeGuard
{
public:
	explicit FPushScopeGuard(RowHandle NewContext)
		: PreviousScope(SetCurrentScope(NewContext))
	{
	}

	~FPushScopeGuard()
	{
		SetCurrentScope(PreviousScope);
	}

	// Non-copyable, non-movable
	FPushScopeGuard(const FPushScopeGuard&) = delete;
	FPushScopeGuard& operator=(const FPushScopeGuard&) = delete;
	FPushScopeGuard(FPushScopeGuard&&) = delete;
	FPushScopeGuard& operator=(FPushScopeGuard&&) = delete;

private:
	RowHandle PreviousScope;
};

// ============================================================================
// TLS convenience overloads
// ============================================================================
// These free functions wrap ICoreProvider scope methods using the thread-local
// current scope (GetCurrentScope()) as the row argument. They exist so callers
// that operate in the ambient scope don't need to pass GetCurrentScope()
// explicitly at every call site.

/** Typed hierarchy-walking lookup using the thread-local current scope. */
template<TColumnType T>
const T* GetScopeData(ICoreProvider& Storage)
{
	return Storage.GetScopeData<T>(GetCurrentScope());
}

/** Hierarchy-walking version query using thread-local current scope. */
template<TColumnType T>
FScopeDataVersion GetScopeDataVersion(ICoreProvider& Storage)
{
	return Storage.GetScopeDataVersion<T>(GetCurrentScope());
}

/** Set scope data using thread-local current scope. */
template<TColumnType T>
FScopeDataVersion SetScopeData(ICoreProvider& Storage, T&& Data)
{
	return Storage.SetScopeData<T>(GetCurrentScope(), Forward<T>(Data));
}

/** Remove scope data using thread-local current scope. */
template<TColumnType T>
bool RemoveScopeData(ICoreProvider& Storage)
{
	return Storage.RemoveScopeData<T>(GetCurrentScope());
}

} // namespace UE::Editor::DataStorage::Scope
