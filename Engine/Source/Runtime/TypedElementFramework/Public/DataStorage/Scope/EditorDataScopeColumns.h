// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "DataStorage/CommonTypes.h"

#include "EditorDataScopeColumns.generated.h"

/**
 * Tag applied to all scope rows created via AddScopeRow.
 * Used to identify Editor Scope Tree rows in the TEDS debugger.
 */
USTRUCT(meta = (DisplayName = "Data Storage Scope Data"))
struct FDataStorageScopeDataTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * The VersePath of the asset/scope currently being edited by a tool.
 * Published on scope rows by editors (Level Editor, Prefab Editor, etc.)
 * so that downstream consumers can determine access-specifier visibility
 * via FVersePathRegistry::CanAccess(ContextPath, TargetPath).
 */
USTRUCT(meta = (DisplayName = "Editing Verse Scope"))
struct FEditingVerseScope : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY()
	FString EditingScopeVersePath;
};

/**
 * Captures the callstack at the point where a scope row was created via AddScopeRow.
 * Used by the Scope Hierarchy debugger tab to show where each scope row originated.
 */
USTRUCT(meta = (DisplayName = "Scope Row Source Info"))
struct FScopeRowSourceInfo : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	static constexpr int32 MaxCallstackDepth = 64;

	uint64 Callstack[MaxCallstackDepth] = {};
	int32 CallstackDepth = 0;
};

namespace UE::Editor::DataStorage
{
	using FScopeDataTag = FDataStorageScopeDataTag;
	using FEditingScopeColumn = FEditingVerseScope;
	using FScopeSourceInfoColumn = FScopeRowSourceInfo;
}
