// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"

class UScriptStruct;

namespace UE::Editor::DataStorage::Scope
{

/**
 * Represents a versioned snapshot of scope data for a specific column type on a row.
 * Used for change detection: compare two FScopeDataVersion values to detect if scope
 * data has changed (including removal).
 *
 * The version is opaque -- callers should only use operator==, operator!=, and IsValid().
 */
struct FScopeDataVersion
{
	TYPEDELEMENTFRAMEWORK_API FScopeDataVersion();

	TYPEDELEMENTFRAMEWORK_API bool operator==(const FScopeDataVersion& Other) const;
	TYPEDELEMENTFRAMEWORK_API bool operator!=(const FScopeDataVersion& Other) const;

	/** Returns true if this version refers to a valid row (i.e., was obtained from a real operation). */
	TYPEDELEMENTFRAMEWORK_API bool IsValid() const;

	/** Factory for internal use by ICoreProvider implementations. */
	TYPEDELEMENTFRAMEWORK_API static FScopeDataVersion Make(RowHandle InSourceRow, uint64 InVersion);

private:
	RowHandle SourceRow = InvalidRowHandle;
	uint64 Version = 0;
};

} // namespace UE::Editor::DataStorage::Scope
