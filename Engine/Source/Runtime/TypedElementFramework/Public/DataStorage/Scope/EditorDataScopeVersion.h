// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"

#include "EditorDataScopeVersion.generated.h"

/**
 * Monotonic version counter for a scope row.
 * Bumped on every SetScopeData or RemoveScopeData call targeting this row.
 * Part of the scope table schema — every scope row has one automatically.
 */
USTRUCT(meta = (DisplayName = "ContextDataVersion"))
struct FScopeDataVersionColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	uint64 Version = 0;
};
