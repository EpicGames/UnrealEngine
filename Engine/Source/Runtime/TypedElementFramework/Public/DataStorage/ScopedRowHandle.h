// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#include <utility>

namespace UE::Editor::DataStorage
{
	/** RAII class that removes its row from storage when going out of scope. */
	class FScopedRow
	{
	public:
		FScopedRow(ICoreProvider* InStorage, RowHandle InRow)
			: Storage(InStorage)
			, Row(InRow)
		{
		}

		FScopedRow(FScopedRow&& Other)
			: Storage(std::exchange(Other.Storage, nullptr))
			, Row(std::exchange(Other.Row, InvalidRowHandle))
		{
		}

		FScopedRow& operator=(FScopedRow&& Other)
		{
			std::swap(Storage, Other.Storage);
			std::swap(Row, Other.Row);
			return *this;
		}

		FScopedRow(const FScopedRow&) = delete;
		FScopedRow& operator=(const FScopedRow&) = delete;

		~FScopedRow()
		{
			if (Storage && Row != InvalidRowHandle)
			{
				Storage->RemoveRow(Row);
			}
		}

		ICoreProvider* GetStorage() const { return Storage; }
		RowHandle      GetRow() const { return Row; }

		explicit operator RowHandle() const { return GetRow(); }

	private:
		ICoreProvider* Storage;
		RowHandle      Row;
	};
}
