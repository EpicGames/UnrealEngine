// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "DecoratorWidgetColumns.generated.h"

/**
 * Tag used to uniquely identify widget factories for decorator widgets
 */
USTRUCT(meta = (DisplayName = "Decorator Widget Factory"))
struct FEditorDataStorageUiDecoratorWidgetFactoryTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

namespace UE::Editor::DataStorage::Ui
{
	using FDecoratorWidgetFactoryTag = FEditorDataStorageUiDecoratorWidgetFactoryTag;
}