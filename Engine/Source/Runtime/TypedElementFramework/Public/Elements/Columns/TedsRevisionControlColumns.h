// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"

#include "TedsRevisionControlColumns.generated.h"

/**
 * Row-mapping-domain identifier for TEDS source-control package rows. Lives here
 * (TypedElementFramework) rather than in the TedsRevisionControl plugin because
 * non-plugin engine modules (e.g. UnsavedAssetsTracker) and engine plugins that
 * ship to all licensees (e.g. PerforceSourceControl) need to consume these columns
 * without taking a hard dependency on the experimental EditorDataStorageFeatures
 * plugin. Behaviour (helper, processors, queries, table registration) still lives
 * in TedsRevisionControl; only the type definitions are framework-level.
 */
namespace UE::Editor::RevisionControl
{
	inline static const FName MappingDomain = "SourceControl";
}

UENUM()
enum class ESCCModification
{
	None,
	Conflicted,
	Modified,
	Added,
	Removed,
	NotInDepot
};

/**
 * Aggregated revision control state for a package row.
 * Modification == ESCCModification::None means the file is source-controlled and has no local change.
 *
 * IconStyleName / IconOverlayName / DisplayTooltip are populated authoritatively by the
 * active source-control provider (in its UpdateEditorDataStorage) from
 * FSourceControlState::GetIcon() and GetDisplayTooltip(). Consumers that only need to
 * render the row should use those directly without reinterpreting the booleans below.
 */
USTRUCT(meta = (DisplayName = "Revision control state"))
struct FSccStateColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	// modification axis
	//
	ESCCModification Modification = ESCCModification::None;

	// state axis
	// A file can be modified (modification) and bIsLocked and bIsNotCurrent
	// The state is viewed as an additional bit flag on top of the modification
	//
	bool bIsExternallyLocked = false;
	bool bIsLocked = false;
	bool bIsExternallyEdited = false;
	bool bIsNotCurrent = false;

	FName IconStyleSetName;
	FName IconStyleName;
	FName IconOverlayName;
	FText DisplayTooltip;
};

// tag used to trigger a file monitoring start
USTRUCT(meta = (DisplayName = "SCC status update required"))
struct FSCCStatusUpdateRequiredTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

// tag used to trigger a file monitoring stop
USTRUCT(meta = (DisplayName = "SCC status update ended"))
struct FSCCStatusUpdateEndedTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

// tag to indicate the Scc state has changed.
USTRUCT(meta = (DisplayName = "SCC state dirty"))
struct FSccStateDirtyTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};
