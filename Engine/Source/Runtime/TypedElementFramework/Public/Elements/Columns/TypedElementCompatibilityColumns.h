// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "TypedElementCompatibilityColumns.generated.h"

class UObject;
class UWorld;

/**
 * Column containing a non-owning reference to a UObject.
 */
USTRUCT(meta = (DisplayName = "UObject reference"))
struct FTypedElementUObjectColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	// If the UObject is accessed during an OnRemove event that is triggered by the garbage collection, the UObject will
	// already be marked as unreachable and regular functions to retrieve the UObject will return an nullptr. In these cases
	// use the unreachable versions such as Get(/*bEvenIfPendingKill*/ true) or GetEvenIfUnreachable().
	TWeakObjectPtr<UObject> Object;
};

/**
 * Column containing information to uniquely identify the UObject, e.g. for use by the garbage collection.
 */
USTRUCT(meta = (DisplayName = "UObject ID"))
struct FTypedElementUObjectIdColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	uint32 Id;
	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	int32 SerialNumber;
};

/**
 * Column containing a non-owning reference to an arbitrary object. It's strongly recommended
 * to also add a FTypedElementScriptStructTypeInfoColumn to make sure the type can be safely
 * recovered.
 */
USTRUCT(meta = (DisplayName = "External object reference"))
struct FTypedElementExternalObjectColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	void* Object;
};

/**
 * Tag to identify a row with a Class Default Object (CDO).
 * If there's a FTypedElementUObjectColumn, the stored object will be a CDO if this tag is present.
 */
USTRUCT(meta = (DisplayName = "Class Default Object"))
struct FTypedElementClassDefaultObjectTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * Tag to identify a row with an actor. If there's a FTypedElementUObjectColumn, the stored object will be
 * an actor if this tag is present.
 */
USTRUCT(meta = (DisplayName = "Actor"))
struct FTypedElementActorTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * Column that stores a reference to the world.
 */
USTRUCT(meta = (DisplayName = "World"))
struct FTypedElementWorldColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	TWeakObjectPtr<UWorld> World;
};

/**
* Tag to identify a row with a UWorld. If there's a FTypedElementUObjectColumn, the stored object will be
 * a UWorld if this tag is present.
 */
USTRUCT(meta = (DisplayName = "World Tag"))
struct FEditorDataStorageWorldTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * Column that stores a reference to a level.
 */
USTRUCT(meta = (DisplayName = "Level"))
struct FTypedElementLevelColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	TWeakObjectPtr<ULevel> Level;
};

/**
* Tag to identify a row with a ULevel. If there's a FTypedElementUObjectColumn, the stored object will be
 * a ULevel if this tag is present.
 */
USTRUCT(meta = (DisplayName = "Level Tag"))
struct FEditorDataStorageLevelTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * Tag to signal that a row represents a property bag "placeholder-typed" object reference.
 * This object has an unknown base type and will generally be associated with a "property
 * bag" containing any serialized data that was loaded for it as a set of "loose" properties.
 */
USTRUCT(meta = (DisplayName = "Property bag placeholder"))
struct FTypedElementPropertyBagPlaceholderTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * Tag to signal that a row contains an object with at least one loose property associated with it.
 */
USTRUCT(meta = (DisplayName = "Loose property"))
struct FTypedElementLoosePropertyTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * A column that stores an inferred base type for placeholder-typed object references. Note that
 * this will not be the same as the placeholder object type (stored in the class type info column).
 * This column can be used to query for a "base type" determined from the serialization context,
 * for systems that need to look/behave differently based on an inferred base type context (e.g. UI).
 */
USTRUCT(meta = (DisplayName = "Placeholder type info"))
struct FTypedElementPropertyBagPlaceholderTypeInfoColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	TWeakObjectPtr<const UStruct> InferredBaseType;
};

/**
 * Column containing the ID Name of the UObject.
 */
USTRUCT(meta = (DisplayName = "Name"))
struct FEditorDataStorageUObjectIdNameColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable, Sortable))
	FName IdName;
};

/**
 * Tag on a row that to denote this is a PIE specific row (e.g PIE actors, PIE folders)
 */
USTRUCT(meta = (DisplayName = "PIE Object"))
struct FEditorDataStoragePieObjectTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * Tag on a row that stores the editor only guid of the row
 */
USTRUCT(meta = (DisplayName = "Guid"))
struct FEditorDataStorageGuidColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	FGuid Guid;
};

namespace UE::Editor::DataStorage
{
	using FUObjectIdNameColumn = FEditorDataStorageUObjectIdNameColumn;
	using FLevelColumn = FTypedElementLevelColumn;
	using FLevelTag = FEditorDataStorageLevelTag;
	using FWorldTag = FEditorDataStorageWorldTag;
	using FPieObjectTag = FEditorDataStoragePieObjectTag;
	using FGuidColumn = FEditorDataStorageGuidColumn;
}  // namespace UE::Editor::DataStorage