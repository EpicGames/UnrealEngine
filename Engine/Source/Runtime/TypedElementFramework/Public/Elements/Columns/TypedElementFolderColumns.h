// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#include "TypedElementFolderColumns.generated.h"

/**
 * Column that signifies this row is for a folder
 */
USTRUCT(meta = (DisplayName = "Folder"))
struct FFolderTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};