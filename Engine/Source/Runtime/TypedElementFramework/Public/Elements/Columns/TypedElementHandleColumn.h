// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "Elements/Framework/TypedElementHandle.h"

#include "TypedElementHandleColumn.generated.h"

// A column which contains a TypedElementHandle from the TypedElementFramework
// Prefer to use the namespaced alias when referencing in code - UE::Editor::DataStorage::Compatibility::FTypedElementColumn
USTRUCT()
struct FTedsTypedElementColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	FTypedElementHandle Handle;
};

namespace UE::Editor::DataStorage::Compatibility
{
	using FTypedElementColumn = FTedsTypedElementColumn;
}  // namespace UE::Editor::DataStorage::Compatibility