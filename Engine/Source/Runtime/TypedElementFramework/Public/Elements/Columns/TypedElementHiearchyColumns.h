// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "DataStorage/MapKey.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "UObject/ObjectMacros.h"

#include "TypedElementHiearchyColumns.generated.h"

/**
 * A reference to the direct hierarchical parent of this row.
 */
USTRUCT(meta = (DisplayName = "Parent"))
struct FTableRowParentColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UE::Editor::DataStorage::RowHandle Parent;
};

/**
 * A reference to the direct hierarchical parent of this row which has not been resolved yet. The stored value will
 * be used to attempt to find the indexed row. This column can not be used to find rows that are not indexed.
 */
USTRUCT(meta = (DisplayName = "Parent (Unresolved)"))
struct FUnresolvedTableRowParentColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UE::Editor::DataStorage::FMapKey ParentIdKey;
};
