// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "UObject/ObjectMacros.h"

#include "TypedElementIconOverrideColumns.generated.h"

/**
 * When present this column overrides any icon that would normally be used to represent the row.
 */
USTRUCT(meta = (DisplayName = "Icon override"))
struct FTypedElementIconOverrideColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	/** Name of the replacement icon. */
	UPROPERTY()
	FName IconName;
};