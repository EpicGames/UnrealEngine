// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "UObject/ObjectMacros.h"

#include "TypedElementLabelColumns.generated.h"

/**
 * Column that stores a label.
 */
USTRUCT(meta = (DisplayName = "Label"))
struct FTypedElementLabelColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable, Sortable))
	FString Label;
};

/**
 * Column that stores the hash of a label. This is typically paired with FTypedElementLabelColumn, but 
 * kept separate in order to iterate quickly over all hash values.
 */
USTRUCT(meta = (DisplayName = "Label hash"))
struct FTypedElementLabelHashColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	uint64 LabelHash;
};