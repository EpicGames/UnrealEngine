// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "UObject/ObjectMacros.h"

#include "TypedElementMiscColumns.generated.h"


/**
 * Tag to indicate that there are one or more bits of information in the row that
 * need to be copied out the Data Storage and into the original object. This tag
 * will automatically be removed at the end of a tick.
 */
USTRUCT(meta = (DisplayName = "Sync back to world"))
struct FTypedElementSyncBackToWorldTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * Tag to signal that data a processor copies out of the world must be synced to the data storage.
 * Useful for when an Actor was recently spawned or reloaded in the world.
 * Currently used if any property changes since there is no mechanism to selectively run
 * queries for specific changed properties.
 */
USTRUCT(meta = (DisplayName = "Sync from world"))
struct FTypedElementSyncFromWorldTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * Tag to signal that data a processor copies out of the world must be synced to the data storage.
 * Useful for when an Actor was recently spawned or reloaded in the world. This version is not
 * automatically removed and intended for interactive operations that will take a few frames
 * to complete such as dragging.
 * Currently used if any property changes since there is no mechanism to selectively run
 * queries for specific changed properties.
 */
USTRUCT(meta = (DisplayName = "Sync from world (interactive)"))
struct FTypedElementSyncFromWorldInteractiveTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * A general reference to another row. 
 */
USTRUCT(meta = (DisplayName = "Row reference"))
struct FTypedElementRowReferenceColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UE::Editor::DataStorage::RowHandle Row;
};

/**
 * A name for this row.
 */
USTRUCT(meta = (DisplayName = "Name"))
struct UE_DEPRECATED(5.7, "Use of the FNameColumn has been deprecated, please create a new column for your purpose.") FNameColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable, DeprecatedProperty))
	FName Name;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FNameColumn() = default;
	~FNameColumn() = default;
	FNameColumn(const FNameColumn&) = default;
	FNameColumn(FNameColumn&&) = default;
	FNameColumn& operator=(const FNameColumn&) = default;
	FNameColumn& operator=(FNameColumn&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

/**
 * Tag used to mark a row as "favorite" in TEDS - which can then be used to filter in table views.
 */
USTRUCT(meta = (DisplayName = "Favorite"))
struct FFavoriteTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};