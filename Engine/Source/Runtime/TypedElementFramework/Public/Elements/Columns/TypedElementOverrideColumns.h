// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "UObject/OverridableManager.h"

#include "TypedElementOverrideColumns.generated.h"

// Whether the object has an override on the base
USTRUCT(meta = (DisplayName = "Override"))
struct FObjectOverrideColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	EOverriddenState OverriddenState;
};