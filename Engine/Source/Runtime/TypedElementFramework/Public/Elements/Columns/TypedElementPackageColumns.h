// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "DataStorage/Handles.h"
#include "DataStorage/MapKey.h"
#include "DataStorage/Queries/Types.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "IO/PackageId.h"
#include "Misc/PackagePath.h"
#include "UObject/ObjectMacros.h"

#include "TypedElementPackageColumns.generated.h"

namespace UE::Editor::DataStorage
{
	inline static const FName PackageTableName = TEXT("Editor_PackageTable");
	inline static const FName PackageHierarchyName = TEXT("PackageHierarchy");
}

/**
 * A package reference column that has not yet been resolved to reference a package.
 */
USTRUCT(meta = (DisplayName = "Unresolved package path reference"))
struct FTypedElementPackageUnresolvedReference final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	FString PathOnDisk;
};

/**
 * Column that references a row in the table that provides package and source control information.
 */
USTRUCT(meta = (DisplayName = "Package path reference"))
struct FTypedElementPackageReference final : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	
	UE::Editor::DataStorage::RowHandle Row;
};

/**
 * Tag that indicates some related package information has been modified.
 */
USTRUCT(meta = (DisplayName = "Package information has been updated"))
struct FTypedElementPackageUpdatedTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * Column that stores the path of a package.
 */
USTRUCT(meta = (DisplayName = "Package path"))
struct FTypedElementPackagePathColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable))
	FString Path;
};

/**
 * Column that stores the fully normalized file path of a package as seen by the operating system.
 */
USTRUCT(meta = (DisplayName = "Package file path on disk"))
struct FTypedElementPackageFilePathOnDiskColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable))
	FString FilePathOnDisk;
};

inline uint32 GetTypeHash(const FTypedElementPackagePathColumn& InStruct)
{
	return GetTypeHash(InStruct.Path);
}

/**
 * Column that stores the full loading path to a package.
 */
USTRUCT(meta = (DisplayName = "Package loaded path"))
struct FTypedElementPackageLoadedPathColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	FPackagePath LoadedPath;
};

/**
 * Tag to indicate that the data on the row is in a package that was marked unsaved/marked dirty
 */
USTRUCT(meta = (DisplayName = "In Dirty Package"))
struct FTedsPackageDirtyTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * Tag to indicate that the data on the row is the primary object in a package
 */
USTRUCT(meta = (DisplayName = "Primary Package Object"))
struct FTedsPrimaryPackageObjectTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};
