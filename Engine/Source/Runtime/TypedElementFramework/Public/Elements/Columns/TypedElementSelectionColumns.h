// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "UObject/ObjectMacros.h"

#include "TypedElementSelectionColumns.generated.h"

/**
 * Column to represent that a row is selected
 */
USTRUCT(meta = (DisplayName = "Selected"))
struct FTypedElementSelectionColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	// The selection set this row belongs to, empty = default selection set
	UPROPERTY()
	FName SelectionSet;
};
