// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"

#include "TypedElementSlateWidgetColumns.generated.h"

class SWidget;
struct FTypedElementWidgetConstructor;

namespace UE::Editor::DataStorage
{
	class ITedsWidget;
}

/**
 * Stores a widget reference in the data storage. At the start of processing any
 * columns that are not pointing to a valid widget will be removed. If the
 * FTypedElementSlateWidgetDeletesRowTag is found then the entire row will
 * be deleted.
 */
USTRUCT(meta = (DisplayName = "Slate widget reference"))
struct FTypedElementSlateWidgetReferenceColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	// The actual internal widget
	TWeakPtr<SWidget> Widget;

	// Reference to the container widget that holds the internal widget
	TWeakPtr<UE::Editor::DataStorage::ITedsWidget> TedsWidget;

	// Reference to the widget constructor that was used to create this widget (if applicable)
	TWeakPtr<FTypedElementWidgetConstructor> WidgetConstructor;
};

/**
 * Tag to indicate that the entire row needs to be deleted when the widget in
 * FTypedElementSlateWidgetReferenceColumn is no longer valid, otherwise only
 * the column will be removed.
 */
USTRUCT(meta = (DisplayName = "Slate widget reference deletes row"))
struct FTypedElementSlateWidgetReferenceDeletesRowTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * A localized display name for this row.
 */
USTRUCT(meta = (DisplayName = "Display Name"))
struct FDisplayNameColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable, Sortable))
	FText DisplayName;
};

/**
 * A localized description for this row.
 */
USTRUCT(meta = (DisplayName = "Description"))
struct FDescriptionColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable))
	FText Description;
};

/**
 * A highlight text for this row.  Can be used by widget rows to set the corresponding HighlightText on a widget (e.g. STextBlock)
 */
USTRUCT(meta = (DisplayName = "Highlight Text"))
struct FHighlightTextColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY()
	FText HighlightText;
};

/**
 * A color for this row. Can be used by widget rows to determine widget color or non-widget rows to attach a logical color to their data
 * (e.g asset colors)
 */
USTRUCT(meta = (DisplayName = "Color"))
struct FSlateColorColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY()
	FSlateColor Color = FSlateColor::UseForeground();
};

/**
 * Tag added onto widget rows if they are currently in editing mode (e.g editable text boxes)
 */
USTRUCT(meta = (DisplayName = "Is Editing"))
struct FIsInEditingModeTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * A reference to the widget purpose this widget was created for
 */
USTRUCT(meta = (DisplayName = "Widget Purpose Reference"))
struct FWidgetPurposeReferenceColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UE::Editor::DataStorage::RowHandle PurposeRowHandle;
};
