// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "Math/MathFwd.h"

#include "TypedElementTransformColumns.generated.h"


/**
 * Column that stores a local transform. 
 */
USTRUCT(meta = (DisplayName = "Local Transform"))
struct FTypedElementLocalTransformColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	// Transform is not being initialized to avoid spending time on initialization when the
	// Transform will be updated the first and following ticks afr it's creation. If this
	// isn't initialized at the correct time, then the sync from source or the true initialization
	// need to be moved to an earlier phase or group.
	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	FTransform Transform { NoInit };
};

UE_EXPERIMENTAL(5.8, "This tag might change.")
USTRUCT(meta = (DisplayName = "Lock Transform"))
struct FTypedElementLockTransformTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};