// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "TypedElementTypeInfoColumns.generated.h"

class UClass;
class UScriptStruct;

/**
 * Column that stores type information for classes.
 */
USTRUCT(meta = (DisplayName = "Type"))
struct FTypedElementClassTypeInfoColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	TWeakObjectPtr<const UClass> TypeInfo;
};

/**
 * Column that stores type information for structs.
 */
USTRUCT(meta = (DisplayName = "ScriptStruct type info"))
struct FTypedElementScriptStructTypeInfoColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	TWeakObjectPtr<const UScriptStruct> TypeInfo;
};

/**
 * Column that stores a type display name that should override the main Class Type Info column
 * Note: Can be used to display types that aren't a UClass or UStruct (i.e. folders)
 */
UE_EXPERIMENTAL(5.8, "This column may be removed and replaced with some other mechanism.")
USTRUCT(meta = (DisplayName = "Type Display Override"))
struct FTypedElementTypeInfoDisplayOverrideColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	FText TypeDisplayName;
};
