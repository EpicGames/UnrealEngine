// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "UObject/ObjectMacros.h"

#include "TypedElementUIColumns.generated.h"

/**
 * Tag to indicate that this row should not show up in any TEDS UI that displays rows (e.g table viewer)
 */
USTRUCT(meta = (DisplayName = "Hide row from UI"))
struct FHideRowFromUITag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * Tag on to indicate that this row should be expanded in the UI
 */
USTRUCT(meta = (DisplayName = "Expanded item"))
struct FExpandedInUITag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};