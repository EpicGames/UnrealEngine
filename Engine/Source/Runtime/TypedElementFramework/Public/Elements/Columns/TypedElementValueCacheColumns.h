// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"

#include "TypedElementValueCacheColumns.generated.h"


/**
 * Column that can be used to cache an unsigned 32-bit value in.
 */
USTRUCT(meta = (DisplayName = "Uint32 value cache"))
struct FTypedElementU32IntValueCacheColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	uint32 Value;
};

/**
 * Column that can be used to cache a signed 32-bit value in.
 */
USTRUCT(meta = (DisplayName = "Int32 value cache"))
struct FTypedElementI32IntValueCacheColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	int32 Value;
};

/**
 * Column that can be used to cache an unsigned 64-bit value in.
 */
USTRUCT(meta = (DisplayName = "Uint64 value cache"))
struct FTypedElementU64IntValueCacheColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	uint64 Value;
};

/**
 * Column that can be used to cache a signed 64-bit value in.
 */
USTRUCT(meta = (DisplayName = "Int64 value cache"))
struct FTypedElementI64IntValueCacheColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	int64 Value;
};

/**
 * Column that can be used to cache a 32-bit floating point value in.
 */
USTRUCT(meta = (DisplayName = "float value cache"))
struct FTypedElementFloatValueCacheColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	float Value;
};