// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"

#include "TypedElementViewportColumns.generated.h"

/**
 * Column to hold the color that the object is outlined with in the viewport
 */
USTRUCT(meta = (DisplayName = "Viewport Outline Color"))
struct FTypedElementViewportOutlineColorColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (ClampMin = "0", ClampMax = "7"))
	uint8 SelectionOutlineColorIndex = 0;
};

/**
 * Column to hold the color that the object is overlaid with in the viewport
 */
USTRUCT(meta = (DisplayName = "Viewport Overlay Color"))
struct FTypedElementViewportOverlayColorColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY()
	FColor OverlayColor = FColor(EForceInit::ForceInitToZero);
};

/**
* Flag to mark current color dirty and trigger a rendering refresh
*/
USTRUCT(meta = (DisplayName = "Viewport Overlay Color Dirty Flag"))
struct FTypedElementViewportOverlayColorDirtyFlag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};
