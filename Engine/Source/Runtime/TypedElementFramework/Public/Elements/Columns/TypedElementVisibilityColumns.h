// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#if WITH_EDITOR
#include "Elements/Framework/TypedElementRowHandleArray.h"
#include "Misc/Change.h"
#endif

#include "TypedElementVisibilityColumns.generated.h"

/**
 * VisibleInEditor column that signifies whether or not this row's object should be visible in view ports
 */
USTRUCT(meta = (DisplayName = "Visibility"))
struct FVisibleInEditorColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

public:
	UPROPERTY(meta = (Sortable))
	bool bIsVisibleInEditor = true;
};

/**
 * Tag added to rows for a frame when their visibility value changes. 
 * This tag is required to be added for any changes you make to the TEDS column to be propagated back to the world, and is then removed at the end
 * of the frame.
 */
UE_EXPERIMENTAL(5.8, "This tag might be changed into a column if extra context is needed.")
USTRUCT(meta = (DisplayName = "Visibility Changed"))
struct FVisibilityChangedTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

#if WITH_EDITOR
namespace UE::Editor::DataStorage
{
	/** Change object for undo/redo of TEDS visibility column modifications. */
	UE_EXPERIMENTAL(5.8, "Undo handling of rows may change.")
	class FVisibilityColumnChange : public FCommandChange
	{
	public:
		/**
		 * Note that ChangedRowsIn should only include rows whose visibilty state actually changed. Including
		 *  rows whose visibility was already bVisibleOnApply before the transaction will cause undo to place
		 *  the rows in unmatching state on undo. 
		 */
		FVisibilityColumnChange(FRowHandleArray&& ChangedRowsIn, bool bVisibleOnApplyIn)
			: Rows(MoveTemp(ChangedRowsIn))
			, bVisibleOnApply(bVisibleOnApplyIn)
		{ }

		TYPEDELEMENTFRAMEWORK_API void Apply(UObject* Object) override;
		TYPEDELEMENTFRAMEWORK_API void Revert(UObject* Object) override;
		FString ToString() const override 
		{
			return TEXT("Change In-Editor Visibility");
		}

	private:
		FRowHandleArray Rows;
		bool bVisibleOnApply;
	};
}//end UE::Editor::DataStorage
#endif // WITH_EDITOR