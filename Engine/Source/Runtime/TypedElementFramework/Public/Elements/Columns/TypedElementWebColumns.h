// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "UObject/ObjectMacros.h"

#include "TypedElementWebColumns.generated.h"

/**
 * Information for retrieving a web page from the web.
 */
USTRUCT(meta = (DisplayName = "Url"))
struct FEditorDataStorageUrlColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY()
	FString UrlString;
};

/**
 * Information for retrieving an image from the web.
 * The width and height can optionally be set to non-zero indicating that the size should overwrite the size from the source image.
 */
USTRUCT(meta = (DisplayName = "WebImage", EditorDataStorage_DynamicColumnTemplate))
struct FEditorDataStorageWebImageColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	/** URL to the image in string form. Can be used with for instance with WebImage(Cache) to create a local image. */
	UPROPERTY()
	FString UrlString;

	/** If set, the target width for the image. A value of zero indicates that the width from the source image should be used. */
	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	uint16 Width;

	/** If set, the target height for the image. A value of zero indicates that the height from the source image should be used. */
	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	uint16 Height;
};

namespace UE::Editor::DataStorage
{
	using FUrlColumn = FEditorDataStorageUrlColumn;
	using FWebImageColumn = FEditorDataStorageWebImageColumn;
}