// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "WidgetPurposeColumns.generated.h"

/** Column used to store the type of a widget purpose */
USTRUCT()
struct FWidgetPurposeColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	/** The type of the purpose, used to determine how factories are registered for it */
	UE::Editor::DataStorage::IUiProvider::EPurposeType PurposeType;

	/** The unique ID of this purpose that can be used to reference it instead of the row handle */
	UE::Editor::DataStorage::IUiProvider::FPurposeID PurposeID;
};

/** Column used to store the name of a widget purpose split into 3 parts (E.g "SceneOutliner.Cell.Large") */
USTRUCT()
struct FWidgetPurposeNameColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable, Sortable))
	FName Namespace;

	UPROPERTY(meta = (Searchable, Sortable))
	FName Name;

	UPROPERTY(meta = (Searchable, Sortable))
	FName Frame;
};

/** Column to store info about a widget factory */
USTRUCT()
struct FWidgetFactoryColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	/** Row handle of the purpose the widget factory belongs to */
	UE::Editor::DataStorage::RowHandle PurposeRowHandle;
};

/** Column used to store the widget constructor used by a widget factory by its TypeInfo */
USTRUCT()
struct FWidgetFactoryConstructorTypeInfoColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	TWeakObjectPtr<const UScriptStruct> Constructor;
};

/** Column used to store the name of the widget constructor */
USTRUCT()
struct FWidgetConstructorNameColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable, Sortable))
	FName Name;
};

/** Column used to store the query conditions used by a widget factory
 *  If the column is not present the factory doesn't match against any conditions (i.e is a general purpose factory)
 */
USTRUCT()
struct FWidgetFactoryConditionsColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UE::Editor::DataStorage::Queries::FConditions Conditions;
};

// Deprecated columns

USTRUCT()
struct UE_DEPRECATED(5.7, "Use of the FWidgetFactoryConstructorColumn has been deprecated, use FWidgetFactoryConstructorTypeInfoColumn to store the constructor by type instead.")
FWidgetFactoryConstructorColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	TUniquePtr<FTypedElementWidgetConstructor> Constructor;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FWidgetFactoryConstructorColumn() = default;
	~FWidgetFactoryConstructorColumn() = default;
	FWidgetFactoryConstructorColumn(const FWidgetFactoryConstructorColumn&) = delete;
	FWidgetFactoryConstructorColumn(FWidgetFactoryConstructorColumn&&) = default;
	FWidgetFactoryConstructorColumn& operator=(const FWidgetFactoryConstructorColumn&) = delete;
	FWidgetFactoryConstructorColumn& operator=(FWidgetFactoryConstructorColumn&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
template<>
struct TStructOpsTypeTraits<FWidgetFactoryConstructorColumn> : public TStructOpsTypeTraitsBase2<FWidgetFactoryConstructorColumn>
{
	enum
	{
		WithCopy = false
	};
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS