// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

UE_DEPRECATED_HEADER(5.7, "Use DataStorage/Features.h instead of Elements/Common/EditorDataStorageFeatures.h")

#include "DataStorage/Features.h"
