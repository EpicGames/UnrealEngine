// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

UE_DEPRECATED_HEADER(5.7, "Use DataStorage/Debug/Log.h instead of Elements/Common/TypedElementDataStorageLog.h")

#include "DataStorage/Debug/Log.h"
