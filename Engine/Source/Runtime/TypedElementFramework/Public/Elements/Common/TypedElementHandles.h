// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

UE_DEPRECATED_HEADER(5.7, "Use DataStorage/Handles.h instead of Elements/Common/TypedElementHandles.h")

#include "DataStorage/Handles.h"
