// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

UE_DEPRECATED_HEADER(5.7, "Use DataStorage/Queries/Conditions.h instead of Elements/Common/TypedElementQueryConditions.h")

#include "DataStorage/Queries/Conditions.h"
