// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

UE_DEPRECATED_HEADER(5.7, "Use DataStorage/Queries/Types.h instead of Elements/Common/TypedElementQueryTypes.h")

#include "DataStorage/Queries/Types.h"
