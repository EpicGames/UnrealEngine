// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "TypedElementAttributeBindingText.h"

namespace UE::Editor::DataStorage
{
	class ICoreProvider;

	/**
	 * Builder class that can be used as a shorthand to bind data inside a TEDS row, column pair to a TAttribute so the attribute updates if the data
	 * in the column is changed.
	 * 
	 * Usage Example:
	 * 
	 * FAttributeBinder Binder(RowHandle);
	 * TAttribute<int> TestAttribute(Binder.BindData(&FTestColumnInt::TestInt))
	 */
	class FAttributeBinder
	{
	public:

		/* Create an attribute binder for a given row. */
		TYPEDELEMENTFRAMEWORK_API FAttributeBinder(RowHandle InTargetRow);
		
		/* Create an attribute binder for a given row. */
		TYPEDELEMENTFRAMEWORK_API FAttributeBinder(RowHandle InTargetRow, ICoreProvider* InDataStorage);

		/**
		 * Bind a specific data member inside a TEDS column to an attribute of the same type as the data
		 * 
		 * @param InVariable The data member inside a column to be bound
		 * @param InDefaultValue The default value to be used when the column isn't present on a row
		 *
		 * Example:
		 * FAttributeBinder Binder(RowHandle);
		 * TAttribute<FString> TestAttribute(Binder.BindData(&FTypedElementLabelColumn::Label))
		 */
		template <typename AttributeType, TDataColumnType ColumnType>
		TAttribute<AttributeType> BindData(AttributeType ColumnType::* InVariable, const AttributeType& InDefaultValue = AttributeType());

		/**
		 * Bind a specific data member inside a TEDS column to an attribute of the same type as the data
		 * 
		 * @param InVariable The data member inside a column to be bound
		 * @param InDefaultValue The default value to be used when the column isn't present on a row
		 * @param InIdentifier The identifier for this dynamic column
		 * @return A TAttribute bound to the row, column pair specified
		 *
		 * Example:
		 * FAttributeBinder Binder(RowHandle);
		 * TAttribute<FString> TestAttribute(Binder.BindData(&FTypedElementLabelColumn::Label, FName("Id")))
		 */
		template <typename AttributeType, TDynamicColumnTemplate ColumnType>
		TAttribute<AttributeType> BindData(const FName& InIdentifier, AttributeType ColumnType::* InVariable, const AttributeType& InDefaultValue = AttributeType());

		/**
		 * Bind a specific data member inside a TEDS column to an attribute of a different type than the data by providing a conversion function
		 * NOTE: the default value is not the actual attribute type but rather the data type in the column and it gets passed to the conversion function
		 *
		 * @param InVariable The data member inside a column to be bound
		 * @param InConverter Conversion function to convert from DataType -> AttributeType
		 * @param InDefaultValue The default value to be used when the column isn't present on a row
		 * @return A TAttribute bound to the row, column pair specified
		 *
		 * Example:
		 * UE::Editor::DataStorage::FAttributeBinder Binder(RowHandle);
		 * TAttribute<FText> TestAttribute(Binder.BindData(&FTypedElementLabelColumn::Label),
		 *                                 [](const FString& Data)
		 *                                   {
		 *                                      return FText::FromString(Data);
		 *                                   }
		 *                                 ));
		 */
		template <typename AttributeType, typename DataType, TDataColumnType ColumnType>
		TAttribute<AttributeType> BindData(DataType ColumnType::* InVariable, TFunction<AttributeType(const DataType&)> InConverter, const DataType& InDefaultValue = DataType());

		/**
		 * Bind a specific data member inside a TEDS column to an attribute of a different type than the data by providing a conversion function
		 * NOTE: the default value is not the actual attribute type but rather the data type in the column and it gets passed to the conversion function
		 *
		 * @param InVariable The data member inside a column to be bound
		 * @param InConverter Conversion function to convert from DataType -> AttributeType
		*  @param InIdentifier The identifier for this column if it is a dynamic column
		 * @param InDefaultValue The default value to be used when the column isn't present on a row
		 * @return A TAttribute bound to the row, column pair specified
		 *
		 * Example:
		 * UE::Editor::DataStorage::FAttributeBinder Binder(RowHandle);
		 * TAttribute<FText> TestAttribute(Binder.BindData(&FTypedElementLabelColumn::Label),
		 *                                 [](const FString& Data)
		 *                                   {
		 *                                      return FText::FromString(Data);
		 *                                   }
		 *                                 ));
		 */
		template <typename AttributeType, typename DataType, TDynamicColumnTemplate ColumnType>
		TAttribute<AttributeType> BindData(const FName& InIdentifier, DataType ColumnType::* InVariable, TFunction<AttributeType(const DataType&)> InConverter, const DataType& InDefaultValue = DataType());

		/**
		 * Overload for the conversion binder to accept lambdas instead of TFunctions
		 *
		 * @param InVariable The data member inside a column to be bound
		 * @param InConverter Conversion function to convert from DataType -> AttributeType
		 * @param InDefaultValue The default value to be used when the column isn't present on a row\
		 */
		template <typename DataType, TDataColumnType ColumnType, typename FunctionType>
			requires DataStorage::Private::AttributeBinderInvocable<FunctionType, DataType>
		auto BindData(DataType ColumnType::* InVariable, FunctionType&& InConverter, const DataType& InDefaultValue = DataType());

		/**
		 * Overload for the conversion binder to accept lambdas instead of TFunctions
		 *
		 * @param InVariable The data member inside a column to be bound
		 * @param InConverter Conversion function to convert from DataType -> AttributeType
		 * @param InDefaultValue The default value to be used when the column isn't present on a row\
		 * @param InIdentifier The identifier for this column if it is a dynamic column, NAME_None if it is not a dynamic column
		 * @return A TAttribute bound to the row, column pair specified
		 */
		template <typename DataType, TDynamicColumnTemplate ColumnType, typename FunctionType>
			requires DataStorage::Private::AttributeBinderInvocable<FunctionType, DataType>
		auto BindData(const FName& InIdentifier, DataType ColumnType::* InVariable, FunctionType&& InConverter, const DataType& InDefaultValue = DataType());

		/**
		 * Bind a whole TEDS column to a slate attribute (instead of a single member variable). This allows you to derive an attribute from multiple
		 * members of a TEDS column
		 *
		 * @param InConverter Conversion function to convert from ColumnType -> AttributeType
		 * @return A TAttribute bound to the row, column pair specified
		 *
		 * Example:
		 *			const TAttribute TestIntAttribute(Binder.BindColumn<FTestColumnInt>([](const FTestColumnInt& Column)
		 *			{
		 *				return Column.TestInt1 + Column.TestInt2;
		 *			}
		 */
		template <typename AttributeType, TDataColumnType ColumnType>
		TAttribute<AttributeType> BindColumn(TFunction<AttributeType(const ColumnType&)> InConverter);

		/**
		 * Overload for the BindColumn to accept lambdas instead of TFunctions
		 * @param InConverter Conversion function to convert from DataType -> AttributeType
		 * @return A TAttribute bound to the row, column pair specified
		 */
		template <TDataColumnType ColumnType, typename FunctionType>
			requires DataStorage::Private::AttributeBinderColumnInvocable<FunctionType, ColumnType>
		auto BindColumn(FunctionType&& InConverter);
		
		/**
         * Bind the presence of a TEDS tag or column to a slate attribute (instead of a single member variable). This allows you to derive an
         * attribute from the presence of a tag on a row
         *
         * @param InConverter Conversion function to convert from a bool (aka bHasColumn) -> AttributeType
         * @return A TAttribute bound to the row, column pair specified
         *
         * Example:
         *			const TAttribute TestHasIntAttribute(Binder.BindColumnsPresence<FTestHasIntTag>([](bool bHasColumn)
         *			{
         *				return bHasColumn ? Image1 : Image2;
         *			}));
         */
        template <typename AttributeType, TColumnType... ColumnTypes>
        TAttribute<AttributeType> BindColumnsPresence(TFunction<AttributeType(const bool)> InConverter);

		/**
		 * Overload for the BindColumnPresence to accept lambdas instead of TFunctions
		 * @param InConverter Conversion function to convert from a bool (aka bHasColumn) -> AttributeType
		 * @return A TAttribute bound to the row, column pair specified
		 */
		template <TColumnType... ColumnTypes, typename FunctionType>
			requires DataStorage::Private::AttributeBinderColumnPresenceInvocable<FunctionType>
		auto BindColumnsPresence(FunctionType&& InConverter);
		
		/**
		 * Overload for the BindColumnPresence to accept FConditions instead of TColumnTypes
		 * @param InConditions Conditions to check the row if it matches (can include dynamic columns)
		 * @param InConverter Conversion function to convert from a bool (aka bHasColumn) -> AttributeType
		 * @return A TAttribute bound to the row, column pair specified
		 */
		template <typename AttributeType>
		TAttribute<AttributeType> BindColumnsPresence(Queries::FConditions InConditions, TFunction<AttributeType(const bool)> InConverter);

		/**
		 * Overload for the BindColumnPresence(FConditions) to accept lambdas instead of TFunctions
		 * @param InConditions Conditions to check the row if it matches (can include dynamic columns)
		 * @param InConverter Conversion function to convert from a bool (aka bHasColumn) -> AttributeType
		 * @return A TAttribute bound to the row, column pair specified
		 */
		template <typename FunctionType>
			requires DataStorage::Private::AttributeBinderColumnPresenceInvocable<FunctionType>
		auto BindColumnsPresence(Queries::FConditions InConditions, FunctionType&& InConverter);
		
		/**
		 * Bind a whole TEDS column to a slate attribute (instead of a single member variable) using the column's typeinfo.
		 *
		 * @param InColumnType The typeinfo of the column (e.g FTestIntColumn::StaticStruct)
		 * @param InConverter Conversion function to convert from const void* ColumnData -> AttributeType
		 * @return A TAttribute bound to the row, column pair specified
		 *
		 * Example:
		 *			const TAttribute TestIntAttribute(Binder.BindColumnData(FTestColumnInt::StaticStruct(),
		 *				[](const TWeakObjectPtr<const UScriptStruct>& ColumnType, const void* ColumnData)
		 *			{
		 *				return static_cast<const FTestColumnInt*>(ColumnData)->TestInt;
		 *			}
		 */
		template <typename AttributeType>
		TAttribute<AttributeType> BindColumnData(const TWeakObjectPtr<const UScriptStruct>& InColumnType,
			const TFunction<AttributeType(const TWeakObjectPtr<const UScriptStruct>&, const void*)>& InConverter);

		/**
		 * Overload for the BindColumnData to accept lambdas instead of TFunctions
		 * @param InConverter Conversion function to convert from const void* ColumnData -> AttributeType
		 * @return A TAttribute bound to the row, column pair specified
		 */
		template <typename FunctionType>
			requires DataStorage::Private::AttributeBinderColumnDataInvocable<FunctionType>
		auto BindColumnData(const TWeakObjectPtr<const UScriptStruct>& InColumnType, FunctionType&& InConverter);

		/**
		 * Bind a delegate inside a TEDS column to a SLATE_EVENT macro on a widget
		 * @param InVariable The delegate inside the TEDS column
		 * @param InIdentifier The identifier for this column if it is a dynamic column, NAME_None if it is not a dynamic column
		 * @return A delegate that can be provided to an event on a slate widget
		 */
		template <typename InRetValType, typename... ParamTypes, TDataColumnType ColumnType>
		TDelegate<InRetValType(ParamTypes...)> BindEvent(TDelegate<InRetValType(ParamTypes...)> ColumnType::* InVariable);

		template <typename InRetValType, typename... ParamTypes, TDynamicColumnTemplate ColumnType>
		TDelegate<InRetValType(ParamTypes...)> BindEvent(const FName& InIdentifier, TDelegate<InRetValType(ParamTypes...)> ColumnType::* InVariable);

		// BindEvent overrides for TSharedPtr<TDelegate> since raw delegates inside of columns are unsafe because they are not trivially relocatable
		template <typename InRetValType, typename... ParamTypes, TDataColumnType ColumnType>
		TDelegate<InRetValType(ParamTypes...)> BindEvent(TSharedPtr<TDelegate<InRetValType(ParamTypes...)>> ColumnType::* InVariable);

		template <typename InRetValType, typename... ParamTypes, TDynamicColumnTemplate ColumnType>
		TDelegate<InRetValType(ParamTypes...)> BindEvent(const FName& InIdentifier, TSharedPtr<TDelegate<InRetValType(ParamTypes...)>> ColumnType::* InVariable);

		// Bind an event column to a Slate attribute directly instead of a TDelegate
		template <typename InRetValType, TDataColumnType ColumnType>
		TAttribute<InRetValType> BindEventAttribute(
			TSharedPtr<TDelegate<InRetValType()>> ColumnType::* InVariable,
			InRetValType DefaultValue = InRetValType{});

		template <typename InRetValType, TDynamicColumnTemplate ColumnType>
		TAttribute<InRetValType> BindEventAttribute(
			const FName& InIdentifier,
			TSharedPtr<TDelegate<InRetValType()>> ColumnType::* InVariable,
			InRetValType DefaultValue = InRetValType{});

		/**
		 * Directly bind an FString member in a TEDS column to an FText attribute as a shortcut
		 * @param InFStringVariable The FString variable
		 * @param InIdentifier The identifier for this column if it is a dynamic column, NAME_None if it is not a dynamic column
		 * @return A delegate that can be provided to a text widget in Slate (e.g STextBlock)
		 */
		template <TDataColumnType ColumnType>
		TAttribute<FText> BindText(FString ColumnType::* InFStringVariable);
		
		template <TDynamicColumnTemplate ColumnType>
		TAttribute<FText> BindText(const FName& InIdentifier, FString ColumnType::* InFStringVariable);

		/**
		 * Directly bind an FName member in a TEDS column to an FText attribute as a shortcut
		 * @param InFNameVariable The FName variable
		 * @param InIdentifier The identifier for this column if it is a dynamic column, NAME_None if it is not a dynamic column
		 * @return A delegate that can be provided to a text widget in Slate (e.g STextBlock)
		 */
		template <TDataColumnType ColumnType>
		TAttribute<FText> BindText(FName ColumnType::* InFNameVariable);
		
		template <TDynamicColumnTemplate ColumnType>
		TAttribute<FText> BindText(const FName& InIdentifier, FName ColumnType::* InFNameVariable);

		/**
		 * Composite a FText attribute using a format string and named arguments bound to TEDS columns.
		 * Arguments are passed in using the .Arg(...) function. Each argument starts with the name of the
		 * argument in the format string followed by one of the following options:
		 *		- A column variable that's a string (FText, FString or FName).
		 *		- A column variable with a converter to a string.
		 *		- A direct value that's supported by FFormatArgumentValue (FText and numbers).
		 * The final parameter for an argument is an optional default that optionally takes a FFormatArgumentValue value.
		 * @param Format The format to use composite a string together.
		 * @return A delegate that can be provided to a text widget in Slate (e.g STextBlock)
		 * 
		 * Example:
		 * TAttribute<FText> Attribute(Binder.BindTextFormat(
		 *		LOCTEXT("Format", "{Label}: {Value1}, {Value2}, {Value3}, {Value4}")
		 *		.Arg(TEXT("Label"), &FTypedElementLabelColumn::Label)
		 *		.Arg(TEXT("Value1"), &FValueColumn::Integer, [](const int32& Value) { return FText::AsNumber(Value); }, 42)
		 *		.Arg(TEXT("Value2"), &FTextColumn::Text, LOCTEXT("Default", "<no value>"))
		 *		.Arg(TEXT("Value3"), 42)
		 */
		TYPEDELEMENTFRAMEWORK_API FTextAttributeFormatted BindTextFormat(FTextFormat Format) const;
		
	private:

		// The target row for this binder
		RowHandle TargetRow;

		// A ptr to the data storage for quick access
		ICoreProvider* DataStorage;
	};
} // namespace UE::Editor::DataStorage

#include "TypedElementAttributeBinding.inl"