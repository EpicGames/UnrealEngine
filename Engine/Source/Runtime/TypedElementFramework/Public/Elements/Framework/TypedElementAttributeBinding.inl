// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TypedElementRegistry.h"
#include "TypedElementAttributeBindingProperty.h"

namespace UE::Editor::DataStorage
{
	//
	// FAttributeBinder
	//

	template <typename AttributeType, TDataColumnType ColumnType>
	TAttribute<AttributeType> FAttributeBinder::BindData(AttributeType ColumnType::* InVariable, const AttributeType& InDefaultValue)
	{
		using namespace DataStorage::Private;

		if(DataStorage)
		{
			// Create a direct property and bind it to the given variable
			TProperty<AttributeType> Prop;
			Prop.Bind(InVariable);
	
			// We don't want any references to this in the lambda because binders are designed to be used and destructed on the stack
			return TAttribute<AttributeType>::CreateLambda([Property = MoveTemp(Prop), Storage = DataStorage, Row = TargetRow,
				DefaultValue = InDefaultValue]()
			{
				// Get the column from the given row and use that to return the stored property
				if(ColumnType* Column = GetColumn<ColumnType>(Storage, Row))
				{
					return Property.Get(Column, ColumnType::StaticStruct());
				}
				return DefaultValue;
			});
		}
		return TAttribute<AttributeType>();
	}

	template <typename AttributeType, TDynamicColumnTemplate ColumnType>
	TAttribute<AttributeType> FAttributeBinder::BindData(const FName& InIdentifier, AttributeType ColumnType::* InVariable, const AttributeType& InDefaultValue)
	{
		using namespace DataStorage::Private;

		if(DataStorage)
		{
			// Create a direct property and bind it to the given variable
			TProperty<AttributeType> Prop;
			Prop.Bind(InVariable);

			// We don't want any references to this in the lambda because binders are designed to be used and destructed on the stack
			return TAttribute<AttributeType>::CreateLambda([Property = MoveTemp(Prop), Storage = DataStorage, Row = TargetRow,
				DefaultValue = InDefaultValue, Identifier = InIdentifier]()
			{
				// Get the column from the given row and use that to return the stored property
				if(ColumnType* Column = GetColumn<ColumnType>(Storage, Row, Identifier))
				{
					return Property.Get(Column, ColumnType::StaticStruct());
				}
				return DefaultValue;
			});
		}
		return TAttribute<AttributeType>();
	}

	template <typename AttributeType, typename DataType, TDataColumnType ColumnType>
	TAttribute<AttributeType> FAttributeBinder::BindData(DataType ColumnType::* InVariable, TFunction<AttributeType(const DataType&)> InConverter, const DataType& InDefaultValue)
	{
		using namespace DataStorage::Private;

		if(DataStorage)
		{
			AttributeType ConvertedDefault = InConverter(InDefaultValue);

			// Create a convertible property and bind it to the given variable
			TProperty<AttributeType> Prop;
			Prop.Bind(InVariable, MoveTemp(InConverter));
	
			return TAttribute<AttributeType>::CreateLambda(
				[
					Property = MoveTemp(Prop), 
					Storage = DataStorage, 
					Row = TargetRow, 
					DefaultValue = InDefaultValue, 
					ConvertedDefault = MoveTemp(ConvertedDefault)
				]()
				{
					if(ColumnType* Column = GetColumn<ColumnType>(Storage, Row))
					{
						return Property.Get(Column, ColumnType::StaticStruct());
					}

					return ConvertedDefault;
				});
		}
		return TAttribute<AttributeType>();
	}

	template <typename AttributeType, typename DataType, TDynamicColumnTemplate ColumnType>
	TAttribute<AttributeType> FAttributeBinder::BindData(const FName& InIdentifier, DataType ColumnType::* InVariable,
		TFunction<AttributeType(const DataType&)> InConverter, const DataType& InDefaultValue)
	{
		using namespace DataStorage::Private;

		if(DataStorage)
		{
			AttributeType ConvertedDefault = InConverter(InDefaultValue);

			// Create a convertible property and bind it to the given variable
			TProperty<AttributeType> Prop;
			Prop.Bind(InVariable, MoveTemp(InConverter));
	
			return TAttribute<AttributeType>::CreateLambda(
				[
					Property = MoveTemp(Prop), 
					Storage = DataStorage, 
					Row = TargetRow, 
					DefaultValue = InDefaultValue, 
					ConvertedDefault = MoveTemp(ConvertedDefault),
					Identifier = InIdentifier
				]()
				{
					if(ColumnType* Column = GetColumn<ColumnType>(Storage, Row, Identifier))
					{
						return Property.Get(Column, ColumnType::StaticStruct());
					}

					return ConvertedDefault;
				});
		}
		return TAttribute<AttributeType>();
	}
	
	template <typename DataType, TDataColumnType ColumnType, typename FunctionType>
		requires DataStorage::Private::AttributeBinderInvocable<FunctionType, DataType>
	auto FAttributeBinder::BindData(DataType ColumnType::* InVariable, FunctionType&& InConverter, const DataType& InDefaultValue)
	{
		// Deduce the attribute type from the return value of the converter function
		using AttributeType = decltype(InConverter(std::declval<DataType>()));
		
		return BindData<AttributeType, DataType>(InVariable, TFunction<AttributeType(const DataType&)>(MoveTemp(InConverter)), InDefaultValue);
	}
	
	template <typename DataType, TDynamicColumnTemplate ColumnType, typename FunctionType>
		requires DataStorage::Private::AttributeBinderInvocable<FunctionType, DataType>
	auto FAttributeBinder::BindData(const FName& InIdentifier, DataType ColumnType::* InVariable, FunctionType&& InConverter, const DataType& InDefaultValue)
	{
		// Deduce the attribute type from the return value of the converter function
		using AttributeType = decltype(InConverter(std::declval<DataType>()));
		
		return BindData<AttributeType, DataType>(InIdentifier, InVariable, TFunction<AttributeType(const DataType&)>(MoveTemp(InConverter)), InDefaultValue);
	}
	
	template <typename AttributeType, TDataColumnType ColumnType>
	TAttribute<AttributeType> FAttributeBinder::BindColumn(TFunction<AttributeType(const ColumnType&)> InConverter)
	{
		if(DataStorage)
		{
			return TAttribute<AttributeType>::CreateLambda(
				[
					Convertor = InConverter, 
					Storage = DataStorage, 
					Row = TargetRow 
				]()
				{
					if(ColumnType* Column = Storage->GetColumn<ColumnType>(Row))
					{
						return Convertor(*Column);
					}

					return AttributeType();
				});
		}
		return TAttribute<AttributeType>();
	}
		
	template <TDataColumnType ColumnType, typename FunctionType>
		requires DataStorage::Private::AttributeBinderColumnInvocable<FunctionType, ColumnType>
	auto FAttributeBinder::BindColumn(FunctionType&& InConverter)
	{
		// Deduce the attribute type from the return value of the converter function
		using AttributeType = decltype(InConverter(std::declval<const ColumnType&>()));
		
		return BindColumn<AttributeType, ColumnType>(TFunction<AttributeType(const ColumnType&)>(InConverter));
	}
	
	template <typename AttributeType, TColumnType... ColumnTypes>
    TAttribute<AttributeType> FAttributeBinder::BindColumnsPresence(TFunction<AttributeType(const bool)> InConverter)
    {
    	if(DataStorage)
    	{
    		return TAttribute<AttributeType>::CreateLambda(
    			[
    				Convertor = InConverter, 
    				Storage = DataStorage, 
    				Row = TargetRow 
    			]()
    			{
    				return Convertor(Storage->HasColumns<ColumnTypes...>(Row));
    			});
    	}
    	return TAttribute<AttributeType>();
    }
    	
	template <TColumnType... ColumnTypes, typename FunctionType>
		requires DataStorage::Private::AttributeBinderColumnPresenceInvocable<FunctionType>
	auto FAttributeBinder::BindColumnsPresence(FunctionType&& InConverter)
	{
		// Deduce the attribute type from the return value of the converter function
		using AttributeType = decltype(InConverter(std::declval<const bool>()));
    	
		return BindColumnsPresence<AttributeType, ColumnTypes...>(TFunction<AttributeType(const bool)>(MoveTemp(InConverter)));
	}

	template <typename AttributeType>
	TAttribute<AttributeType> FAttributeBinder::BindColumnsPresence(Queries::FConditions InConditions, TFunction<AttributeType(const bool)> InConverter)
	{
		if(DataStorage)
		{
			if (!InConditions.IsCompiled())
			{
				InConditions.Compile(Queries::FEditorStorageQueryConditionCompileContext(DataStorage));
			}
			return TAttribute<AttributeType>::CreateLambda(
				[
					Conditions = InConditions,
					Convertor = InConverter, 
					Storage = DataStorage, 
					Row = TargetRow 
				]()
				{
					return Convertor(Storage->MatchesColumns(Row, Conditions));
				});
		}
		return TAttribute<AttributeType>();
	}
    	
	template <typename FunctionType>
		requires DataStorage::Private::AttributeBinderColumnPresenceInvocable<FunctionType>
	auto FAttributeBinder::BindColumnsPresence(Queries::FConditions InConditions, FunctionType&& InConverter)
	{
		// Deduce the attribute type from the return value of the converter function
		using AttributeType = decltype(InConverter(std::declval<const bool>()));
    	
		return BindColumnsPresence<AttributeType>(InConditions, TFunction<AttributeType(const bool)>(MoveTemp(InConverter)));
	}
	
	template <typename AttributeType>
	TAttribute<AttributeType> FAttributeBinder::BindColumnData(const TWeakObjectPtr<const UScriptStruct>& InColumnType,
		const TFunction<AttributeType(const TWeakObjectPtr<const UScriptStruct>&, const void*)>& InConverter)
	{
		if(DataStorage)
		{
			return TAttribute<AttributeType>::CreateLambda(
				[
					ColumnType = InColumnType,
					Convertor = InConverter, 
					Storage = DataStorage, 
					Row = TargetRow 
				]()
				{
					if(const void* ColumnData = Storage->GetColumnData(Row, ColumnType.Get()))
					{
						return Convertor(ColumnType, ColumnData);
					}

					return AttributeType();
				});
		}
		return TAttribute<AttributeType>();
	}

	template <typename FunctionType>
			requires DataStorage::Private::AttributeBinderColumnDataInvocable<FunctionType>
	auto FAttributeBinder::BindColumnData(const TWeakObjectPtr<const UScriptStruct>& InColumnType, FunctionType&& InConverter)
	{
		// Deduce the attribute type from the return value of the converter function
		using AttributeType = decltype(InConverter(std::declval<const TWeakObjectPtr<const UScriptStruct>&>(), std::declval<const void*>()));
		
		return BindColumnData<AttributeType>(InColumnType,
			TFunction<AttributeType(const TWeakObjectPtr<const UScriptStruct>&, const void*)>(InConverter));
	}

	template <typename InRetValType, typename... ParamTypes, TDataColumnType ColumnType>
		TDelegate<InRetValType(ParamTypes...)> FAttributeBinder::BindEvent(TDelegate<InRetValType(ParamTypes...)> ColumnType::* InVariable)
	{
		using namespace DataStorage::Private;
		
		// Create a property for the delegate
		TProperty<TDelegate<InRetValType(ParamTypes...)>> Prop;
		Prop.Bind(InVariable);
	
		return TDelegate<InRetValType(ParamTypes...)>::CreateLambda([Property = MoveTemp(Prop), Storage = DataStorage,
			Row = TargetRow](ParamTypes&&... Params)
		{
			if(ColumnType* Column = GetColumn<ColumnType>(Storage, Row))
			{
				// Get the delegate in the bound column for the specified row
				TDelegate<InRetValType(ParamTypes...)> Delegate = Property.Get(Column, ColumnType::StaticStruct());

				// Execute the delegate if it is bound
				if(Delegate.IsBound())
				{
					return Delegate.Execute(Forward<ParamTypes>(Params)...);
				}
			}
			return InRetValType();
		});
	}

	template <typename InRetValType, typename... ParamTypes, TDynamicColumnTemplate ColumnType>
	TDelegate<InRetValType(ParamTypes...)> FAttributeBinder::BindEvent(const FName& InIdentifier, TDelegate<InRetValType(ParamTypes...)> ColumnType::* InVariable)
	{
		using namespace DataStorage::Private;

		// Create a property for the delegate
		TProperty<TDelegate<InRetValType(ParamTypes...)>> Prop;
		Prop.Bind(InVariable);

		return TDelegate<InRetValType(ParamTypes...)>::CreateLambda([Property = MoveTemp(Prop), Storage = DataStorage,
			Row = TargetRow, Identifier = InIdentifier](ParamTypes&&... Params)
		{
			if(ColumnType* Column = GetColumn<ColumnType>(Storage, Row, Identifier))
			{
				// Get the delegate in the bound column for the specified row
				TDelegate<InRetValType(ParamTypes...)> Delegate = Property.Get(Column, ColumnType::StaticStruct());

				// Execute the delegate if it is bound
				if(Delegate.IsBound())
				{
					return Delegate.Execute(Forward<ParamTypes>(Params)...);
				}
			}
			return InRetValType();
		});
	}

	template <typename InRetValType, typename... ParamTypes, TDataColumnType ColumnType>
	TDelegate<InRetValType(ParamTypes...)> FAttributeBinder::BindEvent(TSharedPtr<TDelegate<InRetValType(ParamTypes...)>> ColumnType::* InVariable)
	{
		using namespace DataStorage::Private;

		TProperty<TSharedPtr<TDelegate<InRetValType(ParamTypes...)>>> Prop;
		Prop.Bind(InVariable);

		return TDelegate<InRetValType(ParamTypes...)>::CreateLambda([Property = MoveTemp(Prop), Storage = DataStorage,
			Row = TargetRow](ParamTypes&&... Params)
		{
			if (ColumnType* Column = GetColumn<ColumnType>(Storage, Row))
			{
				TSharedPtr<TDelegate<InRetValType(ParamTypes...)>>& SharedDelegate = Property.Get(*Column);
				if (SharedDelegate && SharedDelegate->IsBound())
				{
					return SharedDelegate->Execute(Forward<ParamTypes>(Params)...);
				}
			}
			return InRetValType();
		});
	}

	template <typename InRetValType, typename... ParamTypes, TDynamicColumnTemplate ColumnType>
	TDelegate<InRetValType(ParamTypes...)> FAttributeBinder::BindEvent(const FName& InIdentifier, TSharedPtr<TDelegate<InRetValType(ParamTypes...)>> ColumnType::* InVariable)
	{
		using namespace DataStorage::Private;

		TProperty<TSharedPtr<TDelegate<InRetValType(ParamTypes...)>>> Prop;
		Prop.Bind(InVariable);

		return TDelegate<InRetValType(ParamTypes...)>::CreateLambda([Property = MoveTemp(Prop), Storage = DataStorage,
			Row = TargetRow, Identifier = InIdentifier](ParamTypes&&... Params)
		{
			if (ColumnType* Column = GetColumn<ColumnType>(Storage, Row, Identifier))
			{
				TSharedPtr<TDelegate<InRetValType(ParamTypes...)>>& SharedDelegate = Property.Get(*Column);
				if (SharedDelegate && SharedDelegate->IsBound())
				{
					return SharedDelegate->Execute(Forward<ParamTypes>(Params)...);
				}
			}
			return InRetValType();
		});
	}

	template <typename InRetValType, TDataColumnType ColumnType>
	TAttribute<InRetValType> FAttributeBinder::BindEventAttribute(
		TSharedPtr<TDelegate<InRetValType()>> ColumnType::* InVariable,
		InRetValType DefaultValue)
	{
		using namespace DataStorage::Private;

		TProperty<TSharedPtr<TDelegate<InRetValType()>>> Prop;
		Prop.Bind(InVariable);

		return TAttribute<InRetValType>::CreateLambda([Property = MoveTemp(Prop), Storage = DataStorage,
			Row = TargetRow, Default = MoveTemp(DefaultValue)]()
		{
			if (ColumnType* Column = GetColumn<ColumnType>(Storage, Row))
			{
				TSharedPtr<TDelegate<InRetValType()>>& SharedDelegate = Property.Get(*Column);
				if (SharedDelegate && SharedDelegate->IsBound())
				{
					return SharedDelegate->Execute();
				}
			}
			return Default;
		});
	}

	template <typename InRetValType, TDynamicColumnTemplate ColumnType>
	TAttribute<InRetValType> FAttributeBinder::BindEventAttribute(
		const FName& InIdentifier,
		TSharedPtr<TDelegate<InRetValType()>> ColumnType::* InVariable,
		InRetValType DefaultValue)
	{
		using namespace DataStorage::Private;

		TProperty<TSharedPtr<TDelegate<InRetValType()>>> Prop;
		Prop.Bind(InVariable);

		return TAttribute<InRetValType>::CreateLambda([Property = MoveTemp(Prop), Storage = DataStorage,
			Row = TargetRow, Identifier = InIdentifier, Default = MoveTemp(DefaultValue)]()
		{
			if (ColumnType* Column = GetColumn<ColumnType>(Storage, Row, Identifier))
			{
				TSharedPtr<TDelegate<InRetValType()>>& SharedDelegate = Property.Get(*Column);
				if (SharedDelegate && SharedDelegate->IsBound())
				{
					return SharedDelegate->Execute();
				}
			}
			return Default;
		});
	}

	template <TDataColumnType ColumnType>
	TAttribute<FText> FAttributeBinder::BindText(FString ColumnType::* InFStringVariable)
	{
		return BindData(InFStringVariable, [](const FString& InString)
		{
			return FText::FromString(InString);
		}, FString());
	}
	
	template <TDynamicColumnTemplate ColumnType>
	TAttribute<FText> FAttributeBinder::BindText(const FName& InIdentifier, FString ColumnType::* InFStringVariable)
	{
		return BindData(InIdentifier, InFStringVariable, [](const FString& InString)
		{
			return FText::FromString(InString);
		}, FString());
	}

	template <TDataColumnType ColumnType>
	TAttribute<FText> FAttributeBinder::BindText(FName ColumnType::* InFNameVariable)
	{
		return BindData(InFNameVariable, [](const FName& InName)
		{
			return FText::FromName(InName);
		}, FName());
	}
	
	template <TDynamicColumnTemplate ColumnType>
	TAttribute<FText> FAttributeBinder::BindText(const FName& InIdentifier, FName ColumnType::* InFNameVariable)
	{
		return BindData(InIdentifier, InFNameVariable, [](const FName& InName)
		{
			return FText::FromName(InName);
		}, FName());
	}

} // namespace UE::Editor::DataStorage
