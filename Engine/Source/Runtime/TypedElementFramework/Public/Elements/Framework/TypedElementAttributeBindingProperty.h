// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Containers/Map.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"

namespace UE::Editor::DataStorage
{
	namespace Private
	{
		// Concept for a conversion function used by the attribute binder to bind a column data member to an attribute of a different type
		template <typename FunctionType, typename ArgumentType>
		concept AttributeBinderInvocable = std::is_invocable_v<std::decay_t<FunctionType>, const ArgumentType&>;
		
		// Concept for a conversion function used by the attribute binder to bind a lambda to a TFunction accepting column arguments
		template <typename FunctionType, typename ColumnType>
		concept AttributeBinderColumnInvocable = std::is_invocable_v<std::decay_t<FunctionType>, const ColumnType&>;

		// Concept for a conversion function used by the attribute binder to bind a lambda to a TFunction accepting column data arguments
		template <typename FunctionType>
		concept AttributeBinderColumnDataInvocable = std::is_invocable_v<std::decay_t<FunctionType>, const TWeakObjectPtr<const UScriptStruct>&, const void*>;
		
		// Concept for a conversion function used by the attribute binder to bind a lambda to a TFunction accepting a bool argument representing if
		// the given column is present on the row
        template <typename FunctionType>
        concept AttributeBinderColumnPresenceInvocable = std::is_invocable_v<std::decay_t<FunctionType>, const bool>;

		template <TColumnType ColumnType>
		ColumnType* GetColumn(ICoreProvider* DataStorage, RowHandle Row)
		{
			return DataStorage->GetColumn<ColumnType>(Row);
		}
		
		template <TDynamicColumnTemplate ColumnType>
		ColumnType* GetColumn(ICoreProvider* DataStorage, RowHandle Row, const FName& InIdentifier)
		{
			if(ensureMsgf(InIdentifier != NAME_None, TEXT("None identifier passed to dynamic column version of GetColumn. Will always return nullptr")))
			{
				return DataStorage->GetColumn<ColumnType>(Row, InIdentifier);
			}
			
			return nullptr;
		}

		template <typename PropertyType>
		struct TEmptyProperty final
		{
			PropertyType* Get(void* Object) const;
			const PropertyType* Get(const void* Object) const;
		};

		template <typename PropertyType>
		struct TDirectProperty final
		{
			// Offset to a particular data member inside the object
			size_t Offset;

			PropertyType* Get(void* Object) const;
			const PropertyType* Get(const void* Object) const;
		};

		// A property that goes through a conversion function before being accessed from the object
		template <typename PropertyType>
		struct TConvertibleProperty final
		{
			// Conversion function
			TFunction<PropertyType(const void*)> Converter;

			// Cache to avoid a copy when returning if not necessary
			mutable PropertyType Cache;

			PropertyType* Get(void* Object) const;
			const PropertyType* Get(const void* Object) const;
		};

		template<typename T>
		concept TDataColumnOrDynamicDataColumnTemplate = requires(T t)
		{
			TDataColumnType<T> || THasDynamicColumnTemplateSpecifier<T>;
		};

		template <typename PropertyType>
		class TProperty
		{
		public:
			// Bind this property directly
			template <TDataColumnOrDynamicDataColumnTemplate ObjectType>
			void Bind(PropertyType ObjectType::* Variable);

			// Bind this property using a conversion function
			template <typename InputType, TDataColumnOrDynamicDataColumnTemplate ObjectType>
			void Bind(InputType ObjectType::* Variable, TFunction<PropertyType(const InputType&)> Converter);

			// Get the bound property for the specified object
			template <TDataColumnOrDynamicDataColumnTemplate ObjectType>
			PropertyType& Get(ObjectType& Object) const;

			// Get the bound property for the specified object
			template <TDataColumnOrDynamicDataColumnTemplate ObjectType>
			const PropertyType& Get(const ObjectType& Object) const;

			// Get the bound property for this specified object ptr by providing type information about the object
			PropertyType& Get(void* Object, TWeakObjectPtr<const UScriptStruct> Type) const;

			// Get the bound property for this specified object ptr by providing type information about the object
			const PropertyType& Get(const void* Object, TWeakObjectPtr<const UScriptStruct> Type) const;

			// Returns the stored type information or null if no type was set.
			TWeakObjectPtr<const UScriptStruct> GetObjectTypeInfo() const;

			// Whether or not this property has been bound.
			bool IsBound() const;

		private:
			// Internally we could be storing a direct property or a convertible property
			using InternalPropertyType = TVariant<
				TEmptyProperty<PropertyType>,
				TDirectProperty<PropertyType>,
				TConvertibleProperty<PropertyType>>;

			// The actual property
			InternalPropertyType InternalProperty;

			// The type of the object we are bound to
			TWeakObjectPtr<const UScriptStruct> ObjectTypeInfo = nullptr;
		};
	
		//
		// TEmptyProperty
		//
		template <typename PropertyType>
		PropertyType* TEmptyProperty<PropertyType>::Get(void* Object) const
		{
			checkf(false, TEXT("Empty properties shouldn't be accessed."));
			return nullptr;
		}
		
		template <typename PropertyType>
		const PropertyType* TEmptyProperty<PropertyType>::Get(const void* Object) const
		{
			checkf(false, TEXT("Empty properties shouldn't be accessed."));
			return nullptr;
		}
		
		
		//
		// TDirectProperty
		//

		template <typename PropertyType>
		PropertyType* TDirectProperty<PropertyType>::Get(void* Object) const
		{
			return reinterpret_cast<PropertyType*>((reinterpret_cast<char*>(Object) + Offset));
		}

		template <typename PropertyType>
		const PropertyType* TDirectProperty<PropertyType>::Get(const void* Object) const
		{
			return reinterpret_cast<const PropertyType*>((reinterpret_cast<const char*>(Object) + Offset));
		}


		//
		// TConvertibleProperty
		//

		template <typename PropertyType>
		PropertyType* TConvertibleProperty<PropertyType>::Get(void* Object) const
		{
			Cache = Converter(Object);
			return &Cache;
		}

		template <typename PropertyType>
		const PropertyType* TConvertibleProperty<PropertyType>::Get(const void* Object) const
		{
			Cache = Converter(Object);
			return &Cache;
		}


		//
		// TProperty
		//

		template <typename PropertyType>
		template <TDataColumnOrDynamicDataColumnTemplate ObjectType>
		void TProperty<PropertyType>::Bind(PropertyType ObjectType::* Variable)
		{
			TDirectProperty<PropertyType> Result;
			Result.Offset = reinterpret_cast<size_t>(&(reinterpret_cast<const ObjectType*>(0)->*Variable));

			ObjectTypeInfo = ObjectType::StaticStruct();
			InternalProperty = InternalPropertyType(TInPlaceType<TDirectProperty<PropertyType>>(), MoveTemp(Result));
		}

		template <typename PropertyType>
		template <typename InputType, TDataColumnOrDynamicDataColumnTemplate ObjectType>
		void TProperty<PropertyType>::Bind(InputType ObjectType::* Variable, TFunction<PropertyType(const InputType&)> Converter)
		{
			TConvertibleProperty<PropertyType> Result;
			Result.Converter = [Converter = MoveTemp(Converter), Variable](const void* Object)
				{
					return Converter(reinterpret_cast<const ObjectType*>(Object)->*Variable);
				};

			ObjectTypeInfo = ObjectType::StaticStruct();
			InternalProperty = InternalPropertyType(TInPlaceType<TConvertibleProperty<PropertyType>>(), MoveTemp(Result));
		}

		template <typename PropertyType>
		template <TDataColumnOrDynamicDataColumnTemplate ObjectType>
		PropertyType& TProperty<PropertyType>::Get(ObjectType& Object) const
		{
			return Get(&Object, ObjectType::StaticStruct());
		}

		template <typename PropertyType>
		template <TDataColumnOrDynamicDataColumnTemplate ObjectType>
		const PropertyType& TProperty<PropertyType>::Get(const ObjectType& Object) const
		{
			return Get(&Object, ObjectType::StaticStruct());
		}

		template <typename PropertyType>
		PropertyType& TProperty<PropertyType>::Get(void* Object, TWeakObjectPtr<const UScriptStruct> Type) const
		{
			checkf(Object, TEXT("Nullptr pointer provided to Object while trying to retrieve a property value."));
			checkf(Type.IsValid(), TEXT("Nullptr pointer provided to Type while trying to retrieve a property value."));
			checkf(IsBound(), TEXT("Attempting to retrieve the value of object type (%s) from a property that wasn't bound."),
				*Type->GetFName().ToString());
			ensureMsgf(Type == ObjectTypeInfo, TEXT("Provided object type (%s) did not match bound object type (%s)."),
				*Type->GetFName().ToString(), ObjectTypeInfo.IsValid() ? *ObjectTypeInfo->GetFName().ToString() : TEXT("<no type>"));

			return *Visit([Object](auto&& Prop) { return Prop.Get(Object); }, InternalProperty);
		}

		template <typename PropertyType>
		const PropertyType& TProperty<PropertyType>::Get(const void* Object, TWeakObjectPtr<const UScriptStruct> Type) const
		{
			checkf(Object, TEXT("Nullptr pointer provided to Object while trying to retrieve a property value."));
			checkf(Type.IsValid(), TEXT("Nullptr pointer provided to Type while trying to retrieve a property value."));
			checkf(IsBound(), TEXT("Attempting to retrieve the value of object type (%s) from a property that wasn't bound."),
				*Type->GetFName().ToString());
			ensureMsgf(Type == ObjectTypeInfo, TEXT("Provided object type (%s) did not match bound object type (%s)."),
				*Type->GetFName().ToString(), ObjectTypeInfo.IsValid() ? *ObjectTypeInfo->GetFName().ToString() : TEXT("<no type>"));
			
			return *Visit([Object](auto&& Prop) { return Prop.Get(Object); }, InternalProperty);
		}

		template <typename PropertyType>
		TWeakObjectPtr<const UScriptStruct> TProperty<PropertyType>::GetObjectTypeInfo() const
		{
			return ObjectTypeInfo;
		}

		template <typename PropertyType>
		bool TProperty<PropertyType>::IsBound() const
		{
			return !InternalProperty.template IsType<TEmptyProperty<PropertyType>>();
		}
	} // namespace Private
}