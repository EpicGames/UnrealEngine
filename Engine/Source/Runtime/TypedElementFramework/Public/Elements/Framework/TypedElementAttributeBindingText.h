// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Containers/Map.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "TypedElementAttributeBindingProperty.h"

namespace UE::Editor::DataStorage
{
	
	//
	// FTextAttributeFormatted
	//
	
	class FTextAttributeFormatted
	{
		friend class FAttributeBinder;
		
	public:
		TYPEDELEMENTFRAMEWORK_API FTextAttributeFormatted& Arg(FString Name, FFormatArgumentValue Value);
			
		template <TDataColumnType ColumnType>
		FTextAttributeFormatted& Arg(FString Name, FText ColumnType::* Variable, FFormatArgumentValue Default = {},
			FName ColumnIdentifier = NAME_None);

		template <TDataColumnType ColumnType>
		FTextAttributeFormatted& Arg(FString Name, FString ColumnType::* Variable, FFormatArgumentValue Default = {},
			FName ColumnIdentifier = NAME_None);

		template <TDataColumnType ColumnType>
		FTextAttributeFormatted& Arg(FString Name, FName ColumnType::* Variable, FFormatArgumentValue Default = {},
			FName ColumnIdentifier = NAME_None);

		template <typename DataType, TDataColumnType ColumnType>
		FTextAttributeFormatted& Arg(
			FString Name, 
			DataType ColumnType::* Variable, 
			const TFunction<FText(const DataType&)>& Converter, 
			FFormatArgumentValue Default = {},
			FName ColumnIdentifier = NAME_None);
			
		template <typename DataType, TDataColumnType ColumnType, typename FunctionType> 
			requires DataStorage::Private::AttributeBinderInvocable<FunctionType, DataType>
		FTextAttributeFormatted& Arg(FString Name, DataType ColumnType::* Variable, FunctionType Converter, FFormatArgumentValue Default = {},
			FName ColumnIdentifier = NAME_None);
	
		TYPEDELEMENTFRAMEWORK_API operator TAttribute<FText>();

	private:
		FTextAttributeFormatted(FTextFormat InFormat, RowHandle InTargetRow, ICoreProvider* InDataStorage);

		struct FPropertyInfo
		{
			DataStorage::Private::TProperty<FText> Property;
			FFormatArgumentValue Default;
			FName DynamicColumnIdentifier;
		};
		TMap<FString, FPropertyInfo> NamedProperties;
		FTextFormat Format;
			
		// The target row for this binder
		RowHandle TargetRow;
		// A ptr to the data storage for quick access
		ICoreProvider* DataStorage;
	};

	template <TDataColumnType ColumnType>
	FTextAttributeFormatted& FTextAttributeFormatted::Arg(
		FString Name, FText ColumnType::* Variable, FFormatArgumentValue Default, FName ColumnIdentifier)
	{
		DataStorage::Private::TProperty<FText> Prop;
		Prop.Bind(Variable);
		NamedProperties.Add(MoveTemp(Name), FPropertyInfo{ .Property = MoveTemp(Prop), .Default = MoveTemp(Default),
			.DynamicColumnIdentifier = ColumnIdentifier});

		return *this;
	}

	template <TDataColumnType ColumnType>
	FTextAttributeFormatted& FTextAttributeFormatted::Arg(
		FString Name, FString ColumnType::* Variable, FFormatArgumentValue Default, FName ColumnIdentifier)
	{
		return Arg(MoveTemp(Name), Variable, [](const FString& Value) { return FText::FromString(Value); }, MoveTemp(Default), ColumnIdentifier);
	}

	template <TDataColumnType ColumnType>
	FTextAttributeFormatted& FTextAttributeFormatted::Arg(
		FString Name, FName ColumnType::* Variable, FFormatArgumentValue Default, FName ColumnIdentifier)
	{
		return Arg(MoveTemp(Name), Variable, [](const FName& Value) { return FText::FromName(Value); }, MoveTemp(Default), ColumnIdentifier);
	}

	template <typename DataType, TDataColumnType ColumnType>
	FTextAttributeFormatted& FTextAttributeFormatted::Arg(
		FString Name,
		DataType ColumnType::* Variable,
		const TFunction<FText(const DataType&)>& Converter, 
		FFormatArgumentValue Default,
		FName ColumnIdentifier)
	{
		DataStorage::Private::TProperty<FText> Prop;
		Prop.Bind(Variable, Converter);
		NamedProperties.Add(MoveTemp(Name), FPropertyInfo{ .Property = MoveTemp(Prop), .Default = MoveTemp(Default),
			.DynamicColumnIdentifier = ColumnIdentifier });
		
		return *this;
	}

	template <typename DataType, TDataColumnType ColumnType, typename FunctionType>
		requires DataStorage::Private::AttributeBinderInvocable<FunctionType, DataType>
	FTextAttributeFormatted& FTextAttributeFormatted::Arg(
		FString Name,
		DataType ColumnType::* Variable,
		FunctionType Converter,
		FFormatArgumentValue Default,
		FName ColumnIdentifier)
	{
		return Arg(MoveTemp(Name), Variable, TFunction<FText(const DataType&)>(MoveTemp(Converter)), MoveTemp(Default), ColumnIdentifier);
	}

}