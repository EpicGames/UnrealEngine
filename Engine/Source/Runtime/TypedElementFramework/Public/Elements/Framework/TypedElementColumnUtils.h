// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ArrayView.h"

class UScriptStruct;

namespace UE::Editor::DataStorage::ColumnUtils
{
	struct Argument
	{
		FName Name;
		FString Value;
	};

	/** 
	 * If found in the column, the variable under the provided name will be updated to the target value. 
	 * The target variable must be convertable from string. Any incompatible or missing variables will
	 * be silently ignored.
	 */
	TYPEDELEMENTFRAMEWORK_API void SetColumnValue(
		void* ColumnData, const UScriptStruct* ColumnType, FName ArgumentName, const FString& ArgumentValue);
	/**
	 * If found in the column, the variables under the provided names will be updated to the target values.
	 * The target variables must be convertable from string. Any incompatible or missing variables will
	 * be silently ignored.
	 */
	TYPEDELEMENTFRAMEWORK_API void SetColumnValues(void* ColumnData, const UScriptStruct* ColumnType, TConstArrayView<Argument> Arguments);

	/** Utility function for SetColumnValue that automatically detects the type of the column. */
	template<typename ColumnType>
	void SetColumnValue(ColumnType& Column, FName ArgumentName, const FString& ArgumentValue)
	{
		SetColumnValue(&Column, ColumnType::StaticStruct(), ArgumentName, ArgumentValue);
	}

	/** Utility function for SetColumnValues that automatically detects the type of the column. */
	template<typename ColumnType>
	void SetColumnValues(ColumnType& Column, TConstArrayView<Argument> Arguments)
	{
		SetColumnValues(&Column, ColumnType::StaticStruct(), Arguments);
	}

	/** Utility function to check if a column type is a dynamic template. */
	TYPEDELEMENTFRAMEWORK_API bool IsDynamicTemplate(const UScriptStruct* InColumn);

	/** Utility function to check if a column type is derived from a dynamic template. */
	TYPEDELEMENTFRAMEWORK_API bool IsDerivedFromDynamicTemplate(const UScriptStruct* InColumn);

	/** Utility function to get the identifier for a dynamic column, NAME_None if invalid */
	TYPEDELEMENTFRAMEWORK_API FName GetDynamicColumnIdentifier(const UScriptStruct* InColumn);
}

// Deprecated old namespace with an alias to the new namespace
namespace TypedElement
{
	namespace UE_DEPRECATED(5.6, "Use UE::Editor::DataStorage::ColumnUtils instead") ColumnUtils
	{
		using namespace UE::Editor::DataStorage::ColumnUtils;
	}
}
