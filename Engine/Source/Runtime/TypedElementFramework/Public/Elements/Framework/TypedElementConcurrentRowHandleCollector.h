// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <atomic>
#include "Elements/Framework/TypedElementRowHandleArray.h"

namespace UE::Editor::DataStorage
{
	enum class EConcurrentRowHandleCollectorType
	{
		Array, // Collects all row handles in random order. Is allowed to have duplicates.
		Set // Collects only a unique instance of a row handles and orders them numerically.
	};

	/**
	 * A lock-free utility class to collect rows from multiple threads at the same time.
	 * Two options are provided:
	 *		- Array: Collects all provided row handles without order and can have duplicates.
	 *				This is usually faster than a set.
	 *		- Set: Merges the provided row handles in sorted order and removes duplicates. This is
	 *				generally slower than an array.
	 * Each thread needs to first fill a local row handle array. That list is then passed on to the
	 * collector that will combine it with the final list collected from multiple threads. The cost of
	 * merging is spread across multiple threads, but as more list contribute some threads will have to
	 * merge larger lists than other threads causing an imbalance in thread runtime. The worse case will
	 * be the last thread as that will need to merge two arrays that are roughly both half the size of
	 * the final array. It's therefor recommended to run with "AutoBalanceParallelChunkProcessing".
	 */
	template<EConcurrentRowHandleCollectorType Type>
	class FConcurrentRowHandleCollector
	{
	public:
		TYPEDELEMENTFRAMEWORK_API ~FConcurrentRowHandleCollector();

		/**
		 * Moves the provided array into the collected list by either appending (Array) or uniquely merging (Set).
		 * This function can be repeatedly collect from multiple threads at the same time.
		 * 
		 * The approach for writing results back is a two step process. In the first step it tries to grab the 
		 * temp local array that's shared by all threads. If that returns a nullptr it means there's no array yet or
		 * another thread has already claimed it, in which case a new array is created. If it does manage to get a
		 * hold of the array it merges its own local changes into it.
		 * In the next phase it tries to place the locally update version of the array back so other threads can
		 * add to it. If the shared array is a nullptr than it can move the local version into it, but if another
		 * thread has already set it, it needs to try to grab that version and merge the local copy into it. This
		 * local copy becomes the new target to place in the shared location.
		 *
		 * In practice there will be multiple merges collections present for the majority of processing and separate
		 * threads will merge them together into a single final collection over time. This does result in later threads
		 * having to merge increasingly large list, with one thread have to merge the final largest list. As a result this
		 * leads to threads starting fast but taking increasingly long to run. It's therefore recommended to run queries
		 * with `AutoBalanceParallelChunkProcessing`.
		 */
		TYPEDELEMENTFRAMEWORK_API void Append(FRowHandleArray&& LocalArray);
		
		/**
		 * Collect the final results and merge with the provided list.
		 * Returns true if additions were mode, otherwise false.
		 * 
		 * This function should be called once from a single thread to collect the final list. This can be called if threads 
		 * are still adding new values to the collector, in which case this will remove the row handles merged into "Result" 
		 * and will merge the rest when this is called again.
		 */
		TYPEDELEMENTFRAMEWORK_API bool Collect(FRowHandleArray& AccumulatedCollection);

	private:
		std::atomic<FRowHandleArray*> WorkingSet = nullptr;
	};

	using FConcurrentRowHandleArrayCollector = FConcurrentRowHandleCollector<EConcurrentRowHandleCollectorType::Array>;
	using FConcurrentRowHandleSetCollector = FConcurrentRowHandleCollector<EConcurrentRowHandleCollectorType::Set>;
	
	extern template TYPEDELEMENTFRAMEWORK_API FConcurrentRowHandleCollector<EConcurrentRowHandleCollectorType::Array>::~FConcurrentRowHandleCollector();
	extern template TYPEDELEMENTFRAMEWORK_API void FConcurrentRowHandleCollector<EConcurrentRowHandleCollectorType::Array>::Append(FRowHandleArray&&);
	extern template TYPEDELEMENTFRAMEWORK_API bool FConcurrentRowHandleCollector<EConcurrentRowHandleCollectorType::Array>::Collect(FRowHandleArray&);

	extern template TYPEDELEMENTFRAMEWORK_API FConcurrentRowHandleCollector<EConcurrentRowHandleCollectorType::Set>::~FConcurrentRowHandleCollector();
	extern template TYPEDELEMENTFRAMEWORK_API void FConcurrentRowHandleCollector<EConcurrentRowHandleCollectorType::Set>::Append(FRowHandleArray&&);
	extern template TYPEDELEMENTFRAMEWORK_API bool FConcurrentRowHandleCollector<EConcurrentRowHandleCollectorType::Set>::Collect(FRowHandleArray&);
} // namespace UE::Editor::DataStorage