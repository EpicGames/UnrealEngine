// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/SortedMap.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Function.h"
#include "Templates/Tuple.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/Interface.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include <type_traits>

#include "TypedElementCounter.generated.h"

class UObject;
class UTypedElementRegistry;

template <typename KeyType>
inline FName GetTypedElementCounterKeyName()
{
	if constexpr (std::is_convertible_v<KeyType, const UObject*>)
	{
		return std::remove_pointer_t<KeyType>::StaticClass()->GetFName();
	}
	else
	{
		static const FName KeyTypeName = TNameOf<KeyType>::GetName();
		return KeyTypeName;
	}
}

/**
 * Tracks various categories of counters for the typed elements (eg, the number of elements of a given type).
 * The categories counted may be expanded by element types that implement the UTypedElementCounterInterface API.
 */
class FTypedElementCounter
{
public:
	using FCounterValue = int32;

	FTypedElementCounter() = default;
	TYPEDELEMENTFRAMEWORK_API explicit FTypedElementCounter(UTypedElementRegistry* InRegistry);

	FTypedElementCounter(const FTypedElementCounter&) = delete;
	FTypedElementCounter& operator=(const FTypedElementCounter&) = delete;

	FTypedElementCounter(FTypedElementCounter&&) = default;
	FTypedElementCounter& operator=(FTypedElementCounter&&) = default;

	/**
	 * Explicitly initialize this instance, if it was previously default constructed.
	 */
	TYPEDELEMENTFRAMEWORK_API void Initialize(UTypedElementRegistry* InRegistry);

	/**
	 * Update the counter categories to include the given element.
	 */
	TYPEDELEMENTFRAMEWORK_API void AddElement(const FTypedElementHandle& InElementHandle);

	/**
	 * Update the counter categories to remove the given element.
	 */
	TYPEDELEMENTFRAMEWORK_API void RemoveElement(const FTypedElementHandle& InElementHandle);

	/**
	 * Increment the counter for the given category and key.
	 * @note For use by UTypedElementCounterInterface.
	 */
	template <typename KeyType>
	void IncrementCounter(const FName InCategory, const KeyType& InKey)
	{
		TUniquePtr<ICounterCategory>& CounterCategory = CounterCategories.FindOrAdd(InCategory);
		if (!CounterCategory)
		{
			CounterCategory = MakeUnique<TCounterCategory<KeyType>>();
		}
		CounterCategory->IncrementCounter(InKey);
	}

	/**
	 * Decrement the counter for the given category and key.
	 * @note For use by UTypedElementCounterInterface.
	 */
	template <typename KeyType>
	void DecrementCounter(const FName InCategory, const KeyType& InKey)
	{
		if (TUniquePtr<ICounterCategory>* CounterCategory = CounterCategories.Find(InCategory))
		{
			(*CounterCategory)->DecrementCounter(InKey);
		}
	}
	
	/**
	 * Get the value of the counter for the given category and key, or zero if the category or key are unknown.
	 */
	template <typename KeyType>
	FCounterValue GetCounterValue(const FName InCategory, const KeyType& InKey) const
	{
		if (const TUniquePtr<ICounterCategory>* CounterCategory = CounterCategories.Find(InCategory))
		{
			return (*CounterCategory)->GetCounterValue(InKey);
		}
		return 0;
	}

	/**
	 * Enumerate the value of the counters for the given category.
	 */
	template <typename KeyType>
	void ForEachCounterValue(const FName InCategory, TFunctionRef<bool(const KeyType&, FCounterValue)> InCallback) const
	{
		if (const TUniquePtr<ICounterCategory>* CounterCategory = CounterCategories.Find(InCategory))
		{
			(*CounterCategory)->ForEachCounterValue<KeyType>(InCallback);
		}
	}

	/**
	 * Clear the value of the counter for the given category and key.
	 */
	template <typename KeyType>
	void ClearCounter(const FName InCategory, const KeyType& InKey)
	{
		if (TUniquePtr<ICounterCategory>* CounterCategory = CounterCategories.Find(InCategory))
		{
			(*CounterCategory)->ClearCounter(InKey);
		}
	}

	/**
	 * Clear the value of the counters for the given category.
	 */
	TYPEDELEMENTFRAMEWORK_API void ClearCounters(const FName InCategory);

	/**
	 * Clear all counters.
	 */
	TYPEDELEMENTFRAMEWORK_API void ClearCounters();

	/**
	 * Get the category name used to count element types (by type ID).
	 */
	static TYPEDELEMENTFRAMEWORK_API FName GetElementTypeCategoryName();

private:
	class ICounterCategory
	{
	public:
		virtual ~ICounterCategory() = default;

		template <typename KeyType>
		FCounterValue GetCounterValue(const KeyType& InKey) const
		{
			return GetCounterValueImpl(&InKey, GetTypedElementCounterKeyName<KeyType>());
		}

		template <typename KeyType>
		void ForEachCounterValue(TFunctionRef<bool(const KeyType&, FCounterValue)> InCallback) const
		{
			ForEachCounterValueImpl(&InCallback, GetTypedElementCounterKeyName<KeyType>());
		}

		template <typename KeyType>
		void IncrementCounter(const KeyType& InKey)
		{
			IncrementCounterImpl(&InKey, GetTypedElementCounterKeyName<KeyType>());
		}

		template <typename KeyType>
		void DecrementCounter(const KeyType& InKey)
		{
			DecrementCounterImpl(&InKey, GetTypedElementCounterKeyName<KeyType>());
		}

		template <typename KeyType>
		void ClearCounter(const KeyType& InKey)
		{
			ClearCounterImpl(&InKey, GetTypedElementCounterKeyName<KeyType>());
		}

		void ClearCounters()
		{
			ClearCountersImpl();
		}

	protected:
		virtual FCounterValue GetCounterValueImpl(const void* InKey, const FName InKeyTypeName) const = 0;
		virtual void ForEachCounterValueImpl(const void* InCallback, const FName InKeyTypeName) const = 0;
		virtual void IncrementCounterImpl(const void* InKey, const FName InKeyTypeName) = 0;
		virtual void DecrementCounterImpl(const void* InKey, const FName InKeyTypeName) = 0;
		virtual void ClearCounterImpl(const void* InKey, const FName InKeyTypeName) = 0;
		virtual void ClearCountersImpl() = 0;
	};

	template <typename KeyType>
	class TCounterCategory : public ICounterCategory
	{
	public:
		TCounterCategory()
			: KeyTypeName(GetTypedElementCounterKeyName<KeyType>())
		{
		}

	private:
		virtual FCounterValue GetCounterValueImpl(const void* InKey, const FName InKeyTypeName) const override
		{
			checkf(KeyTypeName == InKeyTypeName, TEXT("Invalid counter access! Counter is keyed against '%s' but '%s' was given."), *KeyTypeName.ToString(), *InKeyTypeName.ToString());

			const KeyType* KeyPtr = static_cast<const KeyType*>(InKey);
			return Counters.FindRef(*KeyPtr);
		}

		virtual void ForEachCounterValueImpl(const void* InCallback, const FName InKeyTypeName) const override
		{
			checkf(KeyTypeName == InKeyTypeName, TEXT("Invalid counter access! Counter is keyed against '%s' but '%s' was given."), *KeyTypeName.ToString(), *InKeyTypeName.ToString());

			const TFunctionRef<bool(const KeyType&, FCounterValue)>* CallbackPtr = static_cast<const TFunctionRef<bool(const KeyType&, FCounterValue)>*>(InCallback);
			for (const TTuple<KeyType, FCounterValue>& CounterValuePair : Counters)
			{
				if (!(*CallbackPtr)(CounterValuePair.Key, CounterValuePair.Value))
				{
					break;
				}
			}
		}

		virtual void IncrementCounterImpl(const void* InKey, const FName InKeyTypeName) override
		{
			checkf(KeyTypeName == InKeyTypeName, TEXT("Invalid counter access! Counter is keyed against '%s' but '%s' was given."), *KeyTypeName.ToString(), *InKeyTypeName.ToString());

			const KeyType* KeyPtr = static_cast<const KeyType*>(InKey);
			FCounterValue& CounterValue = Counters.FindOrAdd(*KeyPtr);
			++CounterValue;
		}

		virtual void DecrementCounterImpl(const void* InKey, const FName InKeyTypeName) override
		{
			checkf(KeyTypeName == InKeyTypeName, TEXT("Invalid counter access! Counter is keyed against '%s' but '%s' was given."), *KeyTypeName.ToString(), *InKeyTypeName.ToString());

			const KeyType* KeyPtr = static_cast<const KeyType*>(InKey);
			if (FCounterValue* CounterValue = Counters.Find(*KeyPtr))
			{
				checkf(*CounterValue > 0, TEXT("Counter value unflow!"));
				if (--(*CounterValue) == 0)
				{
					Counters.Remove(*KeyPtr);
				}
			}
		}

		virtual void ClearCounterImpl(const void* InKey, const FName InKeyTypeName) override
		{
			checkf(KeyTypeName == InKeyTypeName, TEXT("Invalid counter access! Counter is keyed against '%s' but '%s' was given."), *KeyTypeName.ToString(), *InKeyTypeName.ToString());

			const KeyType* KeyPtr = static_cast<const KeyType*>(InKey);
			Counters.Remove(*KeyPtr);
		}

		virtual void ClearCountersImpl() override
		{
			Counters.Empty();
		}

	private:
		FName KeyTypeName;
		TMap<KeyType, FCounterValue> Counters;
	};

	/**
	 * Element registry this element counter is associated with.
	 */
	TWeakObjectPtr<UTypedElementRegistry> Registry;

	/**
	 * Counter categories.
	 */
	TSortedMap<FName, TUniquePtr<ICounterCategory>, FDefaultAllocator, FNameFastLess> CounterCategories;
};

UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UTypedElementCounterInterface : public UInterface
{
	GENERATED_BODY()
};

class ITypedElementCounterInterface
{
	GENERATED_BODY()

public:
	/**
	 * Increment additional counters for the given element.
	 */
	virtual void IncrementCountersForElement(const FTypedElementHandle& InElementHandle, FTypedElementCounter& InOutCounter) {}

	/**
	 * Decrement additional counters for the given element.
	 */
	virtual void DecrementCountersForElement(const FTypedElementHandle& InElementHandle, FTypedElementCounter& InOutCounter) {}
};

template <>
struct TTypedElement<ITypedElementCounterInterface> : public TTypedElementBase<ITypedElementCounterInterface>
{
	void IncrementCountersForElement(FTypedElementCounter& InOutCounter) const { InterfacePtr->IncrementCountersForElement(*this, InOutCounter); }
	void DecrementCountersForElement(FTypedElementCounter& InOutCounter) const { InterfacePtr->DecrementCountersForElement(*this, InOutCounter); }
};
