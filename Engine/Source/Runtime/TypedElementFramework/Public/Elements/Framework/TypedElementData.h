// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "HAL/PlatformStackWalk.h"
#include "Containers/SparseArray.h"
#include "Containers/ChunkedArray.h"
#include "Elements/Framework/TypedElementId.h"

/**
 * Macro to declare the required RTTI data for types representing element data.
 * @note Place this in the public section of your type declaration.
 */
#define UE_DECLARE_TYPED_ELEMENT_DATA_RTTI(ELEMENT_DATA_TYPE, ...)					\
	static __VA_ARGS__ FTypedHandleTypeId Private_RegisteredTypeId;				\
	static FTypedHandleTypeId StaticTypeId() { return Private_RegisteredTypeId; }	\
	static FName StaticTypeName() { static const FName TypeName = #ELEMENT_DATA_TYPE; return TypeName; }

/**
 * Macro to define the required RTTI data for types representing element data.
 * @note Place this in the cpp file for your type definition.
 */
#define UE_DEFINE_TYPED_ELEMENT_DATA_RTTI(ELEMENT_DATA_TYPE)						\
	FTypedHandleTypeId ELEMENT_DATA_TYPE::Private_RegisteredTypeId = 0;

/**
 * Templated util to get the low-level debug ID for an element data instance.
 * Specialize this to provide exta/custom data for your element data type.
 */
template <typename ElementDataType>
inline FString GetTypedElementDebugId(const ElementDataType& InElementData)
{
	return FString();
}

#if UE_TYPED_ELEMENT_HAS_REFTRACKING
/**
 * Debugging information used to locate reference leaks.
 */
class FTypedElementReference
{
public:
	FTypedElementReference()
	{
		CallstackDepth = FPlatformStackWalk::CaptureStackBackTrace(Callstack, UE_ARRAY_COUNT(Callstack));
	}

	FTypedElementReference(const FTypedElementReference&) = default;
	FTypedElementReference& operator=(const FTypedElementReference&) = default;

	FTypedElementReference(FTypedElementReference&&) = default;
	FTypedElementReference& operator=(FTypedElementReference&&) = default;

	void LogReference() const
	{
		ANSICHAR CallstackText[4096];
		for (uint32 CallstackIndex = TypedHandleRefTrackingSkipCount; CallstackIndex < CallstackDepth; ++CallstackIndex)
		{
			CallstackText[0] = 0;
			FPlatformStackWalk::ProgramCounterToHumanReadableString(CallstackIndex - TypedHandleRefTrackingSkipCount, Callstack[CallstackIndex], CallstackText, UE_ARRAY_COUNT(CallstackText));
			UE_LOGF(LogCore, Error, "%ls", ANSI_TO_TCHAR(CallstackText));
		}
	}

private:
	uint64 Callstack[TypedHandleRefTrackingDepth + TypedHandleRefTrackingSkipCount];
	uint32 CallstackDepth = 0;
};

/**
 * Debugging information used to locate reference leaks.
 */
class FTypedElementReferences
{
public:
	static TUniquePtr<FTypedElementReferences> Create()
	{
		return ReferenceTrackingEnabled()
			? MakeUnique<FTypedElementReferences>()
			: nullptr;
	}

	void Reset()
	{
		FScopeLock ReferencesLock(&ReferencesCS);
		References.Reset();
		DestructionRequestCallstack.Reset();
	}

	FTypedElementReferenceId AddRef()
	{
		FScopeLock ReferencesLock(&ReferencesCS);
		return References.Add(FTypedElementReference());
	}

	void ReleaseRef(const FTypedElementReferenceId InReferenceId)
	{
		if (InReferenceId != INDEX_NONE)
		{
			FScopeLock ReferencesLock(&ReferencesCS);
			References.RemoveAt(InReferenceId);
		}
	}

	void LogReferences() const
	{
		FScopeLock ReferencesLock(&ReferencesCS);
		UE_LOGF(LogCore, Error, "===============================================");
		UE_LOGF(LogCore, Error, "External Element References:");
		for (const FTypedElementReference& Reference : References)
		{
			UE_LOGF(LogCore, Error, "-----------------------------------------------");
			Reference.LogReference();
		}
		UE_LOGF(LogCore, Error, "===============================================");
		if (DestructionRequestCallstack)
		{
			UE_LOGF(LogCore, Error, "Destruction requested by:");
			DestructionRequestCallstack->LogReference();
			UE_LOGF(LogCore, Error, "===============================================");
		}
	}

	void StoreDestructionRequestCallstack()
	{
		FScopeLock ReferencesLock(&ReferencesCS);
#if DO_CHECK
		if (DestructionRequestCallstack)
		{
			UE_LOGF(LogCore, Error, "===============================================");
			UE_LOGF(LogCore, Error, "Destruction requested by:");
			DestructionRequestCallstack->LogReference();
			UE_LOGF(LogCore, Error, "===============================================");
			UE_LOGF(LogCore, Fatal, "Element has already had its destruction callstack set! (see above)");
		}
#endif	// DO_CHECK
		DestructionRequestCallstack = MakeUnique<FTypedElementReference>();
	}

private:
	TYPEDELEMENTFRAMEWORK_API static bool ReferenceTrackingEnabled();

	mutable FCriticalSection ReferencesCS;
	TSparseArray<FTypedElementReference> References;
	TUniquePtr<FTypedElementReference> DestructionRequestCallstack;
};
#endif	// UE_TYPED_ELEMENT_HAS_REFTRACKING

/**
 * Base class for the internal payload data associated with elements.
 */
class FTypedElementInternalData
{
public:
	FTypedElementInternalData() = default;

	FTypedElementInternalData(const FTypedElementInternalData&) = delete;
	FTypedElementInternalData& operator=(const FTypedElementInternalData&) = delete;

	FTypedElementInternalData(FTypedElementInternalData&& InOther) = delete;
	FTypedElementInternalData& operator=(FTypedElementInternalData&&) = delete;

	virtual ~FTypedElementInternalData()
	{
		Id.Private_DestroyNoRef();
	}

	void Initialize(const FTypedHandleTypeId InTypeId, const FTypedHandleElementId InElementId)
	{
		checkSlow(!Id.IsSet());
		Id.Private_InitializeNoRef(InTypeId, InElementId);
#if UE_TYPED_ELEMENT_HAS_REFTRACKING
		if (!References)
		{
			// Do this in Initialize rather than the constructor, as the CVar value 
			// may change while the already constructed instances are re-used
			References = FTypedElementReferences::Create();
		}
#endif	// UE_TYPED_ELEMENT_HAS_REFTRACKING
	}

	virtual void Reset()
	{
		Id.Private_DestroyNoRef();
#if UE_TYPED_ELEMENT_HAS_REFCOUNTING
		FTypedElementRefCount OldRefCount = 0;
		UE_AUTORTFM_OPEN
		{
			OldRefCount = FPlatformAtomics::InterlockedExchange(&RefCount, 0);
		};
		UE_AUTORTFM_ONABORT(this, OldRefCount)
		{
			FTypedElementRefCount RefCountDuringAbort = FPlatformAtomics::InterlockedAdd(&RefCount, OldRefCount);
			// Is there a real-world scenario where rolling back a Reset() could yield a non-zero value here?
			// Any further AddRefs would also be paired with an OnAbort handler, undoing its increment.
			// Any ReleaseRefs would be inert until commit time.
			ensure(RefCountDuringAbort == 0);
		};
#endif	// UE_TYPED_ELEMENT_HAS_REFCOUNTING
#if UE_TYPED_ELEMENT_HAS_REFTRACKING
		if (References)
		{
			References->Reset();
		}
#endif	// UE_TYPED_ELEMENT_HAS_REFTRACKING
	}

	inline const FTypedElementId& GetId() const
	{
		return Id;
	}

	inline FTypedElementReferenceId AddRef(const bool bCanTrackReference) const
	{
		FTypedElementReferenceId ReferenceId = INDEX_NONE;
#if UE_TYPED_ELEMENT_HAS_REFCOUNTING
		checkSlow(RefCount < TNumericLimits<FTypedElementRefCount>::Max());
		UE_AUTORTFM_OPEN
		{
			FPlatformAtomics::InterlockedIncrement(&RefCount);
		};
		UE_AUTORTFM_ONABORT(this)
		{
			checkSlow(RefCount > 0);
			FPlatformAtomics::InterlockedDecrement(&RefCount);
		};
#endif	// UE_TYPED_ELEMENT_HAS_REFCOUNTING
#if UE_TYPED_ELEMENT_HAS_REFTRACKING
		if (bCanTrackReference && References)
		{
			ReferenceId = References->AddRef();
		}
#endif	// UE_TYPED_ELEMENT_HAS_REFTRACKING
		return ReferenceId;
	}

	inline void ReleaseRef(const FTypedElementReferenceId InReferenceId) const
	{
#if UE_TYPED_ELEMENT_HAS_REFCOUNTING
		checkSlow(RefCount > 0);
		UE_AUTORTFM_ONCOMMIT(this)
		{
			checkSlow(RefCount > 0);
			FPlatformAtomics::InterlockedDecrement(&RefCount);
		};
#endif	// UE_TYPED_ELEMENT_HAS_REFCOUNTING
#if UE_TYPED_ELEMENT_HAS_REFTRACKING
		if (References)
		{
			References->ReleaseRef(InReferenceId);
		}
#endif	// UE_TYPED_ELEMENT_HAS_REFTRACKING
	}

	UE_AUTORTFM_ALWAYS_OPEN 
	inline FTypedElementRefCount GetRefCount() const
	{
#if UE_TYPED_ELEMENT_HAS_REFCOUNTING
		return FPlatformAtomics::AtomicRead(&RefCount);
#else	// UE_TYPED_ELEMENT_HAS_REFCOUNTING
		return 0;
#endif	// UE_TYPED_ELEMENT_HAS_REFCOUNTING
	}

	void StoreDestructionRequestCallstack() const
	{
#if UE_TYPED_ELEMENT_HAS_REFTRACKING
		if (References)
		{
			References->StoreDestructionRequestCallstack();
		}
#endif	// UE_TYPED_ELEMENT_HAS_REFTRACKING
	}

	void LogExternalReferencesOnDestruction() const
	{
#if DO_CHECK
		const FTypedElementRefCount LocalRefCount = GetRefCount();
		if (LocalRefCount > 1)
		{
#if UE_TYPED_ELEMENT_HAS_REFTRACKING
			UE_LOGF(LogCore, Warning, "Element '%ls' is still externally referenced when being destroyed. Ref-count: %d;", *GetDebugId(), LocalRefCount);
			if (References)
			{
				References->LogReferences();
			}
			else
			{
				UE_LOGF(LogCore, Warning, "Enable CVar 'TypedElements.EnableReferenceTracking' for Element reference tracking.");
			}
#else	// UE_TYPED_ELEMENT_HAS_REFTRACKING
			UE_LOGF(LogCore, Warning, "Element '%ls' is still externally referenced when being destroyed. Ref-count: %d", *GetDebugId(), LocalRefCount);
			UE_LOGF(LogCore, Warning, "UE_TYPED_ELEMENT_HAS_REFTRACKING is disabled. Enable it, recompile and enable CVar 'TypedElements.EnableReferenceTracking' for reference tracking.");
#endif	// UE_TYPED_ELEMENT_HAS_REFTRACKING
		}
#endif	// DO_CHECK
	}

	virtual const void* GetUntypedData() const
	{
		return nullptr;
	}

	virtual FString GetDebugId() const
	{
		return FString::Printf(TEXT("ID: %d"), Id.GetElementId());
	}

private:
	FTypedElementId Id;
#if UE_TYPED_ELEMENT_HAS_REFCOUNTING
	mutable FTypedElementRefCount RefCount = 0;
#endif	// UE_TYPED_ELEMENT_HAS_REFCOUNTING
#if UE_TYPED_ELEMENT_HAS_REFTRACKING
	TUniquePtr<FTypedElementReferences> References;
#endif	// UE_TYPED_ELEMENT_HAS_REFTRACKING
};

/**
 * Internal payload data associated with typed elements.
 */
template <typename ElementDataType>
class TTypedElementInternalData : public FTypedElementInternalData
{
public:
	virtual void Reset() override
	{
		FTypedElementInternalData::Reset();
		Data = ElementDataType();
	}

	inline const ElementDataType& GetData() const
	{
		return Data;
	}

	inline ElementDataType& GetMutableData()
	{
		return Data;
	}

	virtual const void* GetUntypedData() const override
	{
		return &Data;
	}

	virtual FString GetDebugId() const override
	{
		const FString ElementDebugId = GetTypedElementDebugId(Data);
		return ElementDebugId.IsEmpty()
			? FTypedElementInternalData::GetDebugId()
			: FString::Printf(TEXT("%s - %s"), *ElementDebugId, *FTypedElementInternalData::GetDebugId());
	}

private:
	ElementDataType Data;
};

/**
 * Internal payload data associated with typeless elements.
 */
template <>
class TTypedElementInternalData<void> : public FTypedElementInternalData
{
};

/**
 * Internal data that act as a ptr to a control block for the data represented by a ScriptTypedElementHandle
 */
class FScriptTypedElementInternalDataPtr
{
public:

	FScriptTypedElementInternalDataPtr() = default;

	~FScriptTypedElementInternalDataPtr()
	{
		DecrementCount();
	}

	FScriptTypedElementInternalDataPtr(const FScriptTypedElementInternalDataPtr& Other)
		: ControlBlock(Other.ControlBlock)
	{
		IncrementCount();
	}

	FScriptTypedElementInternalDataPtr(FScriptTypedElementInternalDataPtr&& Other)
		: ControlBlock(Other.ControlBlock)
	{
		Other.ControlBlock = nullptr;
	}

	FScriptTypedElementInternalDataPtr& operator=(const FScriptTypedElementInternalDataPtr& Other)
	{
		ControlBlock = Other.ControlBlock;
		IncrementCount();
		return *this;
	}

	FScriptTypedElementInternalDataPtr& operator=(FScriptTypedElementInternalDataPtr&& Other)
	{
		ControlBlock = Other.ControlBlock;
		Other.ControlBlock = nullptr;
		return *this;
	}

	bool operator==(const FScriptTypedElementInternalDataPtr& Other) const
	{
		return ControlBlock == Other.ControlBlock;
	}

	bool operator!=(const FScriptTypedElementInternalDataPtr& Other) const
	{
		return !(*this == Other);
	}

	bool IsSet() const
	{
		return ControlBlock && ControlBlock->Data;
	}

	void Release()
	{
		DecrementCount();
		ControlBlock = nullptr;
	}

	inline const FTypedElementId& GetId() const
	{
		if (ControlBlock)
		{
			if (ControlBlock->Data)
			{
				return ControlBlock->Data->GetId();
			}
		}

		return FTypedElementId::Unset;
	}

	inline FTypedElementInternalData* GetInternalData() const
	{
		if (ControlBlock)
		{
			if (ControlBlock->Data)
			{
				return ControlBlock->Data;
			}
		}

		return nullptr;
	}

protected:
	void IncrementCount()
	{
		if (ControlBlock)
		{
			++ControlBlock->WeakRefCount;
		}
	}

	void DecrementCount()
	{
		if (ControlBlock)
		{
			--ControlBlock->WeakRefCount;
			if (ControlBlock->WeakRefCount == 0)
			{
				delete ControlBlock;
				ControlBlock = nullptr;
			}
		}
	}

	FScriptTypedElementInternalDataPtr(FTypedElementInternalData& InternalData)
		: ControlBlock(new FScriptTypedElementInternalDataControlBlock(InternalData))
	{
		IncrementCount();
	}

	/**
	 * Internal data that act as a weak reference count and a control block for the ScriptTypedElementHandle
	 */
	struct FScriptTypedElementInternalDataControlBlock
	{
		FScriptTypedElementInternalDataControlBlock(FTypedElementInternalData& InData)
			: Data(&InData)
		{
		}

		FScriptTypedElementInternalDataControlBlock() = delete;

		// The control block should never be copied/Moved
		FScriptTypedElementInternalDataControlBlock(const FScriptTypedElementInternalDataControlBlock&) = delete;
		FScriptTypedElementInternalDataControlBlock(FScriptTypedElementInternalDataControlBlock&&) = delete;
		FScriptTypedElementInternalDataControlBlock& operator=(const FScriptTypedElementInternalDataControlBlock&) = delete;
		FScriptTypedElementInternalDataControlBlock& operator=(FScriptTypedElementInternalDataControlBlock&&) = delete;


		FTypedElementInternalData* Data = nullptr;
		FTypedElementRefCount WeakRefCount = 0;
	};

	FScriptTypedElementInternalDataControlBlock* ControlBlock = nullptr;
};

class FScriptTypedElementInternalDataOwner : public FScriptTypedElementInternalDataPtr
{
public:

	FScriptTypedElementInternalDataOwner(FTypedElementInternalData& InInternalData)
		: FScriptTypedElementInternalDataPtr(InInternalData)
	{
		ControlBlock = new FScriptTypedElementInternalDataControlBlock(InInternalData);
		IncrementCount();
	}

	// Tell all the weak reference that the data they point to is no longer valid
	~FScriptTypedElementInternalDataOwner()
	{
		ControlBlock->Data = nullptr;
		DecrementCount();
		ControlBlock = nullptr;
	}
};

/**
 * Data store implementation used by the element registry to manage internal data. 
 * @note This is the generic implementation that uses an array and manages the IDs itself.
 */
template <typename ElementDataType>
class TTypedElementInternalDataStore
{
public:
	static_assert(TNumericLimits<int32>::Max() >= TypedHandleMaxElementId, "TTypedElementInternalDataStore internally uses signed 32-bit indices so cannot store TypedHandleMaxElementId! Consider making this container 64-bit aware, or explicitly remove this compile time check.");

	TTypedElementInternalData<ElementDataType>& AddDataForElement(const FTypedHandleTypeId InTypeId, FTypedHandleElementId& InOutElementId)
	{
		UE::TWriteScopeLock InternalDataLock(InternalDataRW);

		checkSlow(InOutElementId < 0);

		InOutElementId = InternalDataFreeIndices.Num() > 0
			? InternalDataFreeIndices.Pop(EAllowShrinking::No)
			: InternalDataArray.Add();

		TTypedElementInternalData<ElementDataType>& InternalData = InternalDataArray[InOutElementId];
		InternalData.Initialize(InTypeId, InOutElementId);
		return InternalData;
	}

	FScriptTypedElementInternalDataPtr GetInternalDataForScriptHandle(const FTypedHandleElementId InElementId)
	{
		// Script handles are not thread safe
		uint32 Hash = GetTypeHash(InElementId);
		if (FScriptTypedElementInternalDataOwner* ScriptElementInternalData  = ScriptInternalDataMap.FindByHash(Hash, InElementId))
		{
			return *ScriptElementInternalData;
		}

		UE::TReadScopeLock InternalDataLock(InternalDataRW);
		checkSlow(InternalDataArray.IsValidIndex(InElementId));
		return ScriptInternalDataMap.EmplaceByHash(Hash,InElementId, InternalDataArray[InElementId]);
	}

	void DisableScriptHandlesForElement(const FTypedHandleElementId InElementId)
	{
		// Script handles are not thread safe
		ScriptInternalDataMap.Remove(InElementId);
	}

	void RemoveDataForElement(const FTypedHandleElementId InElementId, const FTypedElementInternalData* InExpectedDataPtr)
	{
		UE::TWriteScopeLock InternalDataLock(InternalDataRW);

		checkSlow(InternalDataArray.IsValidIndex(InElementId));
		
		TTypedElementInternalData<ElementDataType>& InternalData = InternalDataArray[InElementId];
		checkf(InExpectedDataPtr == &InternalData, TEXT("Internal data pointer did not match the expected value! Does this handle belong to a different element registry?"));
		InternalData.LogExternalReferencesOnDestruction();
		InternalData.Reset();
		InternalDataFreeIndices.Add(InElementId);
	}

	const TTypedElementInternalData<ElementDataType>& GetDataForElement(const FTypedHandleElementId InElementId) const
	{
		UE::TReadScopeLock InternalDataLock(InternalDataRW);

		checkSlow(InternalDataArray.IsValidIndex(InElementId));
		return InternalDataArray[InElementId];
	}

	static inline void SetStaticDataTypeId(const FTypedHandleTypeId InTypeId)
	{
		checkSlow(ElementDataType::Private_RegisteredTypeId == 0);
		ElementDataType::Private_RegisteredTypeId = InTypeId;
	}

	static inline FTypedHandleTypeId StaticDataTypeId()
	{
		return ElementDataType::StaticTypeId();
	}

	static inline FName StaticDataTypeName()
	{
		return ElementDataType::StaticTypeName();
	}

private:
	mutable FTransactionallySafeRWLock InternalDataRW;
	TArray<int32> InternalDataFreeIndices;
	TChunkedArray<TTypedElementInternalData<ElementDataType>> InternalDataArray;
	TMap<FTypedHandleElementId, FScriptTypedElementInternalDataOwner> ScriptInternalDataMap;
};

/**
 * Data store implementation used by the element registry to manage internal data. 
 * @note This is the typeless implementation that uses external IDs, and exists only to track ref counts.
 */
template <>
class TTypedElementInternalDataStore<void>
{
public:
	static_assert(TNumericLimits<int32>::Max() >= TypedHandleMaxElementId, "TTypedElementInternalDataStore internally uses signed 32-bit indices so cannot store TypedHandleMaxElementId! Consider making this container 64-bit aware, or explicitly remove this compile time check.");

	TTypedElementInternalData<void>& AddDataForElement(const FTypedHandleTypeId InTypeId, FTypedHandleElementId& InOutElementId)
	{
		UE::TWriteScopeLock InternalDataLock(InternalDataRW);

		checkSlow(InOutElementId >= 0);
		checkSlow(!ElementIdToArrayIndex.Contains(InOutElementId));

		const int32 InternalDataArrayIndex = InternalDataFreeIndices.Num() > 0
			? InternalDataFreeIndices.Pop(EAllowShrinking::No)
			: InternalDataArray.Add();

		ElementIdToArrayIndex.Add(InOutElementId, InternalDataArrayIndex);

		TTypedElementInternalData<void>& InternalData = InternalDataArray[InternalDataArrayIndex];
		InternalData.Initialize(InTypeId, InOutElementId);
		return InternalData;
	}

	FScriptTypedElementInternalDataPtr GetInternalDataForScriptHandle(const FTypedHandleElementId InElementId)
	{
		// Script handles are not thread safe
		uint32 Hash = GetTypeHash(InElementId);
		if (FScriptTypedElementInternalDataOwner* ScriptElementInternalData = ScriptInternalDataMap.FindByHash(Hash, InElementId))
		{
			return *ScriptElementInternalData;
		}

		UE::TReadScopeLock InternalDataLock(InternalDataRW);
		const int32 Index = ElementIdToArrayIndex.FindChecked(InElementId);
		return ScriptInternalDataMap.EmplaceByHash(Hash, InElementId, InternalDataArray[Index]);
	}

	void DisableScriptHandlesForElement(const FTypedHandleElementId InElementId)
	{
		// Script handles are not thread safe
		ScriptInternalDataMap.Remove(InElementId);
	}

	void RemoveDataForElement(const FTypedHandleElementId InElementId, const FTypedElementInternalData* InExpectedDataPtr)
	{
		UE::TWriteScopeLock InternalDataLock(InternalDataRW);

		int32 InternalDataArrayIndex = INDEX_NONE;
		ElementIdToArrayIndex.RemoveAndCopyValue(InElementId, InternalDataArrayIndex);

		checkSlow(InternalDataArray.IsValidIndex(InternalDataArrayIndex));

		TTypedElementInternalData<void>& InternalData = InternalDataArray[InternalDataArrayIndex];
		checkf(InExpectedDataPtr == &InternalData, TEXT("Internal data pointer did not match the expected value! Does this handle belong to a different element registry?"));
		InternalData.LogExternalReferencesOnDestruction();
		InternalData.Reset();
		InternalDataFreeIndices.Add(InternalDataArrayIndex);
	}

	const TTypedElementInternalData<void>& GetDataForElement(const FTypedHandleElementId InElementId) const
	{
		UE::TReadScopeLock InternalDataLock(InternalDataRW);

		const int32* InternalDataArrayIndexPtr = ElementIdToArrayIndex.Find(InElementId);
		checkSlow(InternalDataArrayIndexPtr && InternalDataArray.IsValidIndex(*InternalDataArrayIndexPtr));
		return InternalDataArray[*InternalDataArrayIndexPtr];
	}

	static inline void SetStaticDataTypeId(const FTypedHandleTypeId InTypeId)
	{
	}

	static inline FTypedHandleTypeId StaticDataTypeId()
	{
		return 0;
	}

	static inline FName StaticDataTypeName()
	{
		return FName();
	}

private:
	mutable FTransactionallySafeRWLock InternalDataRW;
	TChunkedArray<TTypedElementInternalData<void>> InternalDataArray;
	TArray<int32> InternalDataFreeIndices;
	TMap<FTypedHandleElementId, int32> ElementIdToArrayIndex;
	TMap<FTypedHandleElementId, FScriptTypedElementInternalDataOwner> ScriptInternalDataMap;
};
