// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Widgets/SWidget.h"

namespace UE::Editor::DataStorage
{
	class ITedsWidget
	{
	public:
		virtual ~ITedsWidget() = default;

		virtual RowHandle GetRowHandle() const = 0;

		virtual void SetContent(const TSharedRef< SWidget >& InContent) = 0;

		virtual TSharedRef<SWidget> AsWidget() = 0;
	};
}

// This is for backwards compatibility with the STedsWidget class in the global namespace that used to exist in this file but has been replaced
// by the ITedsWidget interface
using STedsWidget UE_DEPRECATED(5.6, "Use the new ITedsWidget interface directly instead") = UE::Editor::DataStorage::ITedsWidget;