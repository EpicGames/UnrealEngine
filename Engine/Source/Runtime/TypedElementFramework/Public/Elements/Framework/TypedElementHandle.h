// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "CoreMinimal.h"
#include "Elements/Framework/TypedElementData.h"

#include "TypedElementHandle.generated.h"

/**
 * A representation of an element that includes its handle data.
 * This type is the most standard way that an element is passed through to interfaces, and also the type that is stored in element lists.
 * C++ code may choose to use TTypedElement instead, which is a combination of an element handle and its associated element interface.
 * @note Handles auto-release on destruction.
 */
struct FTypedElementHandle
{
public:
	FTypedElementHandle() = default;

	FTypedElementHandle(const FTypedElementHandle& InOther)
	{
		if (InOther)
		{
			Private_InitializeAddRef(*InOther.DataPtr);
		}
	}

	FTypedElementHandle& operator=(const FTypedElementHandle& InOther)
	{
		if (this != &InOther)
		{
			Private_DestroyReleaseRef();

			if (InOther)
			{
				Private_InitializeAddRef(*InOther.DataPtr);
			}
		}
		return *this;
	}

	FTypedElementHandle(FTypedElementHandle&& InOther)
		: DataPtr(InOther.DataPtr)
#if UE_TYPED_ELEMENT_HAS_REFTRACKING
		, ReferenceId(InOther.ReferenceId)
#endif	// UE_TYPED_ELEMENT_HAS_REFTRACKING
	{
		InOther.DataPtr = nullptr;
#if UE_TYPED_ELEMENT_HAS_REFTRACKING
		InOther.ReferenceId = INDEX_NONE;
#endif	// UE_TYPED_ELEMENT_HAS_REFTRACKING
		checkSlow(!InOther.IsSet());
	}

	FTypedElementHandle& operator=(FTypedElementHandle&& InOther)
	{
		if (this != &InOther)
		{
			Private_DestroyReleaseRef();

			DataPtr = InOther.DataPtr;
#if UE_TYPED_ELEMENT_HAS_REFTRACKING
			ReferenceId = InOther.ReferenceId;
#endif	// UE_TYPED_ELEMENT_HAS_REFTRACKING

			InOther.DataPtr = nullptr;
#if UE_TYPED_ELEMENT_HAS_REFTRACKING
			InOther.ReferenceId = INDEX_NONE;
#endif	// UE_TYPED_ELEMENT_HAS_REFTRACKING
			checkSlow(!InOther.IsSet());
		}
		return *this;
	}

	~FTypedElementHandle()
	{
		Private_DestroyReleaseRef();
	}

	inline explicit operator bool() const
	{
		return IsSet();
	}

	/**
	 * Has this handle been initialized to a valid element?
	 */
	inline bool IsSet() const
	{
		return DataPtr != nullptr;
	}

	/**
	 * Release this handle and set it back to an empty state.
	 */
	inline void Release()
	{
		Private_DestroyReleaseRef();
	}

	/**
	 * Get the ID that this element represents.
	 */
	inline const FTypedElementId& GetId() const
	{
		return DataPtr
			? DataPtr->GetId()
			: FTypedElementId::Unset;
	}

	/**
	 * Test to see whether the data stored within this handle is of the given type.
	 * @note This is not typically something you'd want to query outside of data access within an interface implementation.
	 */
	template <typename ElementDataType>
	inline bool IsDataOfType() const
	{
		return GetId().GetTypeId() == ElementDataType::StaticTypeId();
	}

	/**
	 * Attempt to access the data stored within this handle as the given type, returning null if it isn't possible and logging an access error for scripting.
	 * @note This is not typically something you'd want to use outside of data access within an interface implementation.
	 */
	template <typename ElementDataType>
	const ElementDataType* GetData(const bool bSilent = false) const
	{
		if (!DataPtr)
		{
			if (!bSilent)
			{
				FFrame::KismetExecutionMessage(TEXT("Element handle data is null!"), ELogVerbosity::Error);
			}
			return nullptr;
		}

		if (!IsDataOfType<ElementDataType>())
		{
			if (!bSilent)
			{
				FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Element handle data type is '%d', but '%d' (%s) was requested!"), GetId().GetTypeId(), ElementDataType::StaticTypeId(), *ElementDataType::StaticTypeName().ToString()), ELogVerbosity::Error);
			}
			return nullptr;
		}

		return static_cast<const ElementDataType*>(DataPtr->GetUntypedData());
	}

	/**
	 * Attempt to access the data stored within this handle as the given type, asserting if it isn't possible.
	 * @note This is not typically something you'd want to use outside of data access within an interface implementation.
	 */
	template <typename ElementDataType>
	inline const ElementDataType& GetDataChecked() const
	{
		checkf(DataPtr, TEXT("Element handle data is null!"));
		checkf(IsDataOfType<ElementDataType>(), TEXT("Element handle data type is '%d', but '%d' (%s) was requested!"), GetId().GetTypeId(), ElementDataType::StaticTypeId(), *ElementDataType::StaticTypeName().ToString());
		return *static_cast<const ElementDataType*>(DataPtr->GetUntypedData());
	}
	
	/**
	 * Acquire a copy of the ID that this element represents.
	 * @note This must be paired with a call to ReleaseId.
	 */
	FTypedElementId AcquireId() const
	{
		FTypedElementId ElementId;
		if (IsSet())
		{
			DataPtr->AddRef(/*bCanTrackReference*/false); // Cannot track element ID references as we have no space to store the reference ID
			ElementId.Private_InitializeNoRef(DataPtr->GetId().GetTypeId(), DataPtr->GetId().GetElementId());
		}
		return ElementId;
	}

	/**
	 * Release a copy of the ID that this element represents.
	 * @note This should have come from a call to AcquireId.
	 */
	void ReleaseId(FTypedElementId& InOutElementId) const
	{
		checkf(InOutElementId == GetId(), TEXT("Element ID does not match this handle!"));
		if (InOutElementId)
		{
			DataPtr->ReleaseRef(INDEX_NONE); // Cannot track element ID references as we have no space to store the reference ID
			InOutElementId.Private_DestroyNoRef();
		}
	}

	inline friend bool operator==(const FTypedElementHandle& InLHS, const FTypedElementHandle& InRHS)
	{
		return InLHS.DataPtr == InRHS.DataPtr;
	}

	inline friend bool operator!=(const FTypedElementHandle& InLHS, const FTypedElementHandle& InRHS)
	{
		return !(InLHS == InRHS);
	}

	friend inline uint32 GetTypeHash(const FTypedElementHandle& InElementHandle)
	{
		return GetTypeHash(InElementHandle.GetId());
	}

	inline void Private_InitializeNoRef(const FTypedElementInternalData& InData)
	{
		DataPtr = &InData;
	}

	inline void Private_InitializeAddRef(const FTypedElementInternalData& InData)
	{
		Private_InitializeNoRef(InData);
		RegisterRef();
	}

	inline void Private_DestroyNoRef()
	{
		DataPtr = nullptr;
#if UE_TYPED_ELEMENT_HAS_REFTRACKING
		ReferenceId = INDEX_NONE;
#endif	// UE_TYPED_ELEMENT_HAS_REFTRACKING
	}

	inline void Private_DestroyReleaseRef()
	{
		UnregisterRef();
		Private_DestroyNoRef();
	}

	inline const FTypedElementInternalData* Private_GetInternalData() const
	{
		return DataPtr;
	}

private:
	inline void RegisterRef()
	{
		if (DataPtr)
		{
#if !UE_TYPED_ELEMENT_HAS_REFTRACKING
			FTypedElementReferenceId ReferenceId = INDEX_NONE;
#endif	// !UE_TYPED_ELEMENT_HAS_REFTRACKING
			ReferenceId = DataPtr->AddRef(/*bCanTrackReference*/true);
		}
	}

	inline void UnregisterRef()
	{
		if (DataPtr)
		{
#if !UE_TYPED_ELEMENT_HAS_REFTRACKING
			FTypedElementReferenceId ReferenceId = INDEX_NONE;
#endif	// !UE_TYPED_ELEMENT_HAS_REFTRACKING
			DataPtr->ReleaseRef(ReferenceId);
		}
	}

	const FTypedElementInternalData* DataPtr = nullptr;
#if UE_TYPED_ELEMENT_HAS_REFTRACKING
	FTypedElementReferenceId ReferenceId = INDEX_NONE;
#endif	// UE_TYPED_ELEMENT_HAS_REFTRACKING
};

/**
 * Common implementation of TTypedElement that is inherited by all specializations.
 */
template <typename BaseInterfaceType>
struct TTypedElementBase : public FTypedElementHandle
{
public:
	TTypedElementBase() = default;

	TTypedElementBase(const TTypedElementBase&) = default;
	TTypedElementBase& operator=(const TTypedElementBase&) = default;

	TTypedElementBase(TTypedElementBase&& InOther)
		: FTypedElementHandle(MoveTemp(InOther))
		, InterfacePtr(InOther.InterfacePtr)
	{
		InOther.InterfacePtr = nullptr;
		checkSlow(!InOther.IsSet());
	}

	TTypedElementBase& operator=(TTypedElementBase&& InOther)
	{
		if (this != &InOther)
		{
			FTypedElementHandle::operator=(MoveTemp(InOther));
			InterfacePtr = InOther.InterfacePtr;

			InOther.InterfacePtr = nullptr;
			checkSlow(!InOther.IsSet());
		}
		return *this;
	}

	~TTypedElementBase()
	{
		Private_DestroyReleaseRef();
	}

	inline explicit operator bool() const
	{
		return IsSet();
	}

	/**
	 * Has this element been initialized to a valid handle and interface?
	 */
	inline bool IsSet() const
	{
		return FTypedElementHandle::IsSet()
			&& InterfacePtr;
	}

	/**
	 * Release this element and set it back to an empty state.
	 */
	inline void Release()
	{
		Private_DestroyReleaseRef();
	}

	/**
	 * Attempt to access the interface stored within this element, returning null if it isn't set.
	 */
	inline BaseInterfaceType* GetInterface() const
	{
		return InterfacePtr;
	}

	/**
	 * Attempt to access the interface stored within this element, asserting if it isn't set.
	 */
	template <typename U = BaseInterfaceType, std::enable_if_t<!std::is_void<U>::value>* = nullptr>
	inline U& GetInterfaceChecked() const
	{
		static_assert(std::is_same<U, BaseInterfaceType>::value, "Don't explicitly specify U!");
		checkf(InterfacePtr, TEXT("Interface is null!"));
		return *InterfacePtr;
	}

	inline friend bool operator==(const TTypedElementBase& InLHS, const TTypedElementBase& InRHS)
	{
		return static_cast<const FTypedElementHandle&>(InLHS) == static_cast<const FTypedElementHandle&>(InRHS)
			&& InLHS.InterfacePtr == InRHS.InterfacePtr;
	}

	inline friend bool operator!=(const TTypedElementBase& InLHS, const TTypedElementBase& InRHS)
	{
		return !(InLHS == InRHS);
	}

	friend inline uint32 GetTypeHash(const TTypedElementBase& InElement)
	{
		return HashCombine(GetTypeHash(static_cast<const FTypedElementHandle&>(InElement)), GetTypeHash(InElement.InterfacePtr));
	}

	inline void Private_InitializeNoRef(const FTypedElementInternalData& InData, BaseInterfaceType* InInterfacePtr)
	{
		FTypedElementHandle::Private_InitializeNoRef(InData);
		InterfacePtr = InInterfacePtr;
	}

	inline void Private_InitializeAddRef(const FTypedElementInternalData& InData, BaseInterfaceType* InInterfacePtr)
	{
		FTypedElementHandle::Private_InitializeAddRef(InData);
		InterfacePtr = InInterfacePtr;
	}

	inline void Private_DestroyNoRef()
	{
		FTypedElementHandle::Private_DestroyNoRef();
		InterfacePtr = nullptr;
	}

	inline void Private_DestroyReleaseRef()
	{
		FTypedElementHandle::Private_DestroyReleaseRef();
		InterfacePtr = nullptr;
	}

protected:
	BaseInterfaceType* InterfacePtr = nullptr;
};

/**
 * A combination of an element handle and its associated element interface.
 * @note This should be specialized for top-level element interfaces to include their interface API.
 * @note Elements auto-release on destruction.
 */
template <typename BaseInterfaceType>
struct TTypedElement : public TTypedElementBase<BaseInterfaceType>
{
};

// The typed element InterfacePtr will points toward the vtable of the interface
using FTypedElement = TTypedElement<void>;

/**
 * A representation of the owner of an element that includes its mutable handle data.
 * This type is returned when creating an element, and should be used to populate its internal payload data (if any).
 * @note Owners do not auto-release on destruction, and must be manually destroyed via their owner element registry.
 */
template <typename ElementDataType>
struct TTypedElementOwner
{
public:
	TTypedElementOwner() = default;

	TTypedElementOwner(const TTypedElementOwner&) = delete;
	TTypedElementOwner& operator=(const TTypedElementOwner&) = delete;

	TTypedElementOwner(TTypedElementOwner&& InOther)
		: DataPtr(InOther.DataPtr)
	{
		InOther.DataPtr = nullptr;
		checkSlow(!InOther.IsSet());
	}

	TTypedElementOwner& operator=(TTypedElementOwner&& InOther)
	{
		if (this != &InOther)
		{
			DataPtr = InOther.DataPtr;

			InOther.DataPtr = nullptr;
			checkSlow(!InOther.IsSet());
		}
		return *this;
	}

	~TTypedElementOwner()
	{
		checkf(!IsSet(), TEXT("Element owner was still set during destruction! This will leak an element, and you should destroy this element prior to destruction!"));
	}

	inline explicit operator bool() const
	{
		return IsSet();
	}

	/**
	 * Has this owner been initialized to a valid element?
	 */
	inline bool IsSet() const
	{
		return DataPtr != nullptr;
	}

	/**
	 * Get the ID that this element represents.
	 */
	inline const FTypedElementId& GetId() const
	{
		return DataPtr
			? DataPtr->GetId()
			: FTypedElementId::Unset;
	}

	/**
	 * Attempt to access the mutable data stored within this owner, returning null if it isn't possible.
	 */
	template <typename U = ElementDataType, std::enable_if_t<!std::is_void<U>::value>* = nullptr>
	inline U* GetData() const
	{
		static_assert(std::is_same<U, ElementDataType>::value, "Don't explicitly specify U!");
		return DataPtr
			? &DataPtr->GetMutableData()
			: nullptr;
	}

	/**
	 * Attempt to access the mutable data stored within this owner, asserting if it isn't possible.
	 */
	template <typename U = ElementDataType, std::enable_if_t<!std::is_void<U>::value>* = nullptr>
	inline U& GetDataChecked() const
	{
		static_assert(std::is_same<U, ElementDataType>::value, "Don't explicitly specify U!");
		checkf(DataPtr, TEXT("Handle data is null!"));
		return DataPtr->GetMutableData();
	}

	/**
	 * Acquire a copy of the ID that this element represents.
	 * @note This must be paired with a call to ReleaseId.
	 */
	FTypedElementId AcquireId() const
	{
		FTypedElementId ElementId;
		if (IsSet())
		{
			DataPtr->AddRef(/*bCanTrackReference*/false); // Cannot track element ID references as we have no space to store the reference ID
			ElementId.Private_InitializeNoRef(DataPtr->GetId().GetTypeId(), DataPtr->GetId().GetElementId());
		}
		return ElementId;
	}

	/**
	 * Release a copy of the ID that this element represents.
	 * @note This should have come from a call to AcquireId.
	 */
	void ReleaseId(FTypedElementId& InOutElementId) const
	{
		checkf(InOutElementId == GetId(), TEXT("Element ID does not match this owner!"));
		if (InOutElementId)
		{
			DataPtr->ReleaseRef(INDEX_NONE); // Cannot track element ID references as we have no space to store the reference ID
			InOutElementId.Private_DestroyNoRef();
		}
	}

	/**
	 * Acquire a copy of the handle that this element represents.
	 * @note This must be paired with a call to ReleaseHandle (or a call to Release on the handle instance).
	 */
	FTypedElementHandle AcquireHandle() const
	{
		FTypedElementHandle ElementHandle;
		if (IsSet())
		{
			ElementHandle.Private_InitializeAddRef(*DataPtr);
		}
		return ElementHandle;
	}

	/**
	 * Release a copy of the handle that this element represents.
	 * @note This should have come from a call to AcquireHandle.
	 */
	void ReleaseHandle(FTypedElementHandle& InOutElementHandle) const
	{
		checkf(InOutElementHandle.GetId() == GetId(), TEXT("Element handle ID does not match this owner!"));
		InOutElementHandle.Release();
	}

	inline friend bool operator==(const TTypedElementOwner& InLHS, const TTypedElementOwner& InRHS)
	{
		return InLHS.DataPtr == InRHS.DataPtr;
	}

	inline friend bool operator!=(const TTypedElementOwner& InLHS, const TTypedElementOwner& InRHS)
	{
		return !(InLHS == InRHS);
	}

	friend inline uint32 GetTypeHash(const TTypedElementOwner& InElementOwner)
	{
		return GetTypeHash(InElementOwner.GetId());
	}

	inline void Private_InitializeNoRef(TTypedElementInternalData<ElementDataType>& InData)
	{
		DataPtr = &InData;
	}

	inline void Private_InitializeAddRef(TTypedElementInternalData<ElementDataType>& InData)
	{
		Private_InitializeNoRef(InData);
		RegisterRef();
	}

	inline void Private_DestroyNoRef()
	{
		DataPtr = nullptr;
#if UE_TYPED_ELEMENT_HAS_REFTRACKING
		ReferenceId = INDEX_NONE;
#endif	// UE_TYPED_ELEMENT_HAS_REFTRACKING
	}

	inline void Private_DestroyReleaseRef()
	{
		UnregisterRef();
		Private_DestroyNoRef();
	}

	inline const TTypedElementInternalData<ElementDataType>* Private_GetInternalData() const
	{
		return DataPtr;
	}

private:
	inline void RegisterRef()
	{
		if (DataPtr)
		{
#if !UE_TYPED_ELEMENT_HAS_REFTRACKING
			FTypedElementReferenceId ReferenceId = INDEX_NONE;
#endif	// !UE_TYPED_ELEMENT_HAS_REFTRACKING
			ReferenceId = DataPtr->AddRef(/*bCanTrackReference*/true);
		}
	}

	inline void UnregisterRef()
	{
		if (DataPtr)
		{
#if !UE_TYPED_ELEMENT_HAS_REFTRACKING
			FTypedElementReferenceId ReferenceId = INDEX_NONE;
#endif	// !UE_TYPED_ELEMENT_HAS_REFTRACKING
			DataPtr->ReleaseRef(ReferenceId);
		}
	}

	TTypedElementInternalData<ElementDataType>* DataPtr = nullptr;
#if UE_TYPED_ELEMENT_HAS_REFTRACKING
	FTypedElementReferenceId ReferenceId = INDEX_NONE;
#endif	// UE_TYPED_ELEMENT_HAS_REFTRACKING
};
using FTypedElementOwner = TTypedElementOwner<void>;


/**
 * Script exposure for the typed element handle struct type
 * Act as a weak handle to simplify the scripting use of the typed element framework and making it safer to use by avoiding crash in case of a bad usage.
 * This type is the standard way that an element is passed through to interfaces for a script (Blueprint or Python), and also the type that is stored in the script element lists.
 * C++ code may choose to use TTypedElement instead, which is a combination of an element handle and its associated element interface.
 *
 * Note: This type shouldn't be used in the engine code as it come with a performance and memory overhead that we want to avoid when compare to the native handles (FTypedElementHandle).
 */
USTRUCT(BlueprintType)
struct FScriptTypedElementHandle
{
	GENERATED_BODY()
public:

	FScriptTypedElementHandle() = default;

	FScriptTypedElementHandle(const FScriptTypedElementHandle& InOther)
		: InternalData(InOther.InternalData)
	{
	}

	FScriptTypedElementHandle(FScriptTypedElementHandle&& InOther)
		: InternalData(MoveTemp(InOther.InternalData))
	{
	}

	FScriptTypedElementHandle& operator=(const FScriptTypedElementHandle& InOther)
	{
		InternalData = InOther.InternalData;
		return *this;
	}

	FScriptTypedElementHandle& operator=(FScriptTypedElementHandle&& InOther)
	{
		InternalData = MoveTemp(InOther.InternalData);
		return *this;
	}

	inline bool operator==(const FScriptTypedElementHandle& InOther) const
	{
		return InternalData == InOther.InternalData;
	}

	inline bool operator!=(const FScriptTypedElementHandle& InOther) const
	{
		return !(*this == InOther);
	}

	inline explicit operator bool() const
	{
		return IsSet();
	}

	inline bool IsSet() const
	{
		return InternalData.IsSet();
	}

	inline void Release()
	{
		InternalData.Release();
	}

	inline void Private_Initialize(FScriptTypedElementInternalDataPtr&& InInternalData)
	{
		InternalData = InInternalData;
	}

	inline const FTypedElementId& GetId() const
	{
		return InternalData.GetId();
	}

	/**
	 * Return typed element handle from the script typed element handle
	 * If this script handle is invalid it will return a invalid TypedElementHandle
	 */
	FTypedElementHandle GetTypedElementHandle() const
	{
		FTypedElementHandle Handle;
		if (FTypedElementInternalData* TypedElementInternalData = InternalData.GetInternalData())
		{
			Handle.Private_InitializeAddRef(*TypedElementInternalData);
		}

		return Handle;
	}

private:
	FScriptTypedElementInternalDataPtr InternalData;
};

/** Script exposure for FScriptTypedElementHandle. */
UCLASS()
class UTypedElementHandleLibrary : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Has this handle been initialized to a valid element?
	 */
	UFUNCTION(BlueprintPure, Category="TypedElementFramework|Handle", meta=(ScriptMethod, ScriptOperator="bool"))
	static bool IsSet(const FScriptTypedElementHandle& ElementHandle)
	{
		return ElementHandle.IsSet();
	}

	/**
	 * Release this handle and set it back to an empty state.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|Handle", meta=(ScriptMethod))
	static void Release(UPARAM(ref) FScriptTypedElementHandle& ElementHandle)
	{
		ElementHandle.Release();
	}

	/**
	 * Are these two handles equal?
	 */
	UFUNCTION(BlueprintPure, Category="TypedElementFramework|Handle", meta=(DisplayName="Equal (TypedElementHandle)", CompactNodeTitle="==", Keywords="== equal", ScriptMethod, ScriptOperator="=="))
	static bool Equal(const FScriptTypedElementHandle& LHS, const FScriptTypedElementHandle& RHS)
	{
		return LHS == RHS;
	}

	/**
	 * Are these two handles not equal?
	 */
	UFUNCTION(BlueprintPure, Category="TypedElementFramework|Handle", meta=(DisplayName="NotEqual (TypedElementHandle)", CompactNodeTitle="!=", Keywords="!= not equal", ScriptMethod, ScriptOperator="!="))
	static bool NotEqual(const FScriptTypedElementHandle& LHS, const FScriptTypedElementHandle& RHS)
	{
		return LHS != RHS;
	}
};
