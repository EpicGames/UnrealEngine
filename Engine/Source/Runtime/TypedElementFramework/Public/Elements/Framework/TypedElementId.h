// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//#include "CoreMinimal.h"
#include "Elements/Framework/TypedElementLimits.h"
#include "Misc/AssertionMacros.h"
#include "Templates/TypeHash.h"

/**
 * The most minimal representation of an element - its ID!
 * This type is not immediately useful on its own, but can be used to find an element from the element registry or an element list.
 * @note This is ref-counted like handles themselves are, so as long as an ID is available, the handle will be too.
 * @note IDs lack the information needed to auto-release on destruction, so must be manually released, either via the corresponding handle or their owner element registry.
 */
struct FTypedElementId
{
public:
	FTypedElementId()
		: CombinedId(0)
	{
	}

	FTypedElementId(const FTypedElementId&) = delete;
	FTypedElementId& operator=(const FTypedElementId&) = delete;
	
	FTypedElementId(FTypedElementId&& InOther)
		: CombinedId(InOther.CombinedId)
	{
		InOther.Private_DestroyNoRef();
	}

	FTypedElementId& operator=(FTypedElementId&& InOther)
	{
		if (this != &InOther)
		{
			CombinedId = InOther.CombinedId;

			InOther.Private_DestroyNoRef();
		}
		return *this;
	}

	~FTypedElementId()
	{
		checkf(!IsSet(), TEXT("Element ID was still set during destruction! This will leak an element reference, and you should release this ID prior to destruction!"));
	}

	inline explicit operator bool() const
	{
		return IsSet();
	}

	/**
	 * Has this ID been initialized to a valid element?
	 */
	inline bool IsSet() const
	{
		return TypeId != 0;
	}

	/**
	 * Access the type ID portion of this element ID.
	 */
	inline FTypedHandleTypeId GetTypeId() const
	{
		return TypeId;
	}

	/**
	 * Access the element ID portion of this element ID.
	 */
	inline FTypedHandleElementId GetElementId() const
	{
		return ElementId;
	}

	/**
	 * Access the combined value of this element ID.
	 * @note You typically don't want to store this directly as the element ID could be re-used.
	 *       It is primarily useful as a secondary cache where something is keeping a reference to an element ID or element handle (eg, how FTypedElementList uses it internally).
	 */
	inline FTypedHandleCombinedId GetCombinedId() const
	{
		return CombinedId;
	}

	inline friend bool operator==(const FTypedElementId& InLHS, const FTypedElementId& InRHS)
	{
		return InLHS.CombinedId == InRHS.CombinedId;
	}

	inline friend bool operator!=(const FTypedElementId& InLHS, const FTypedElementId& InRHS)
	{
		return !(InLHS == InRHS);
	}

	friend inline uint32 GetTypeHash(const FTypedElementId& InElementId)
	{
		return GetTypeHash(InElementId.CombinedId);
		
	}

	inline void Private_InitializeNoRef(const FTypedHandleTypeId InTypeId, const FTypedHandleElementId InElementId)
	{
		TypeId = InTypeId;
		ElementId = InElementId;
	}

	inline void Private_DestroyNoRef()
	{
		CombinedId = 0;
	}

	/** An unset element ID */
	static TYPEDELEMENTFRAMEWORK_API const FTypedElementId Unset;

private:
	union
	{
		struct
		{
			// Note: These are arranged in this order to give CombinedId better hash distribution for GetTypeHash!
			FTypedHandleCombinedId ElementId : TypedHandleElementIdBits;
			FTypedHandleCombinedId TypeId : TypedHandleTypeIdBits;
		};
		FTypedHandleCombinedId CombinedId;
	};
};
