// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "Elements/Interfaces/TypedElementQueryStorageInterfaces.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

namespace UE::Editor::DataStorage
{
	template<typename T>
	UE_DEPRECATED(5.6, "Use FMapKey(View) with the new mapping functions.")
	IndexHash GenerateIndexHash(const T* Object);
	
	template<typename T>
	UE_DEPRECATED(5.6, "Use FMapKey(View) with the new mapping functions.")
	IndexHash GenerateIndexHash(const TWeakObjectPtr<T>& Object);
	template<typename T>
	UE_DEPRECATED(5.6, "Use FMapKey(View) with the new mapping functions.")
	IndexHash GenerateIndexHash(const TObjectPtr<T>& Object);
	template<typename T>
	UE_DEPRECATED(5.6, "Use FMapKey(View) with the new mapping functions.")
	IndexHash GenerateIndexHash(const TStrongObjectPtr<T>& Object);

	// UE_DEPRECATED(5.6, "Use FMapKey(View) with the new mapping functions.") // Disabled deprecation as it's in use by other deprecated functions.
	inline IndexHash GenerateIndexHash(const FString& Object);
	UE_DEPRECATED(5.6, "Use FMapKey(View) with the new mapping functions.")
	inline IndexHash GenerateIndexHash(FStringView Object);
	// UE_DEPRECATED(5.6, "Use FMapKey(View) with the new mapping functions.") // Disabled deprecation as it's in use by other deprecated functions.
	inline IndexHash GenerateIndexHash(FName Object);
	UE_DEPRECATED(5.6, "Use FMapKey(View) with the new mapping functions.")
	inline IndexHash GenerateIndexHash(const FSoftObjectPath& ObjectPath);
} // namespace UE::Editor::DataStorage

#include "Elements/Framework/TypedElementIndexHasher.inl"