// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Hash/CityHash.h"
#include "UObject/TopLevelAssetPath.h"

namespace UE::Editor::DataStorage
{
	template<typename T>
	IndexHash GenerateIndexHash(const T* Object)
	{
		return reinterpret_cast<IndexHash>(Object);
	}

	template<typename T>
	IndexHash GenerateIndexHash(const TWeakObjectPtr<T>& Object)
	{
		return GenerateIndexHash(Object.Get());
	}

	template<typename T>
	IndexHash GenerateIndexHash(const TObjectPtr<T>& Object)
	{
		return GenerateIndexHash(Object.Get());
	}
	
	template<typename T>
	IndexHash GenerateIndexHash(const TStrongObjectPtr<T>& Object)
	{
		return GenerateIndexHash(Object.Get());
	}

	IndexHash GenerateIndexHash(const FString& Object)
	{
		return CityHash64(reinterpret_cast<const char*>(*Object), Object.Len() * sizeof(**Object));
	}

	IndexHash GenerateIndexHash(FStringView Object)
	{
		return CityHash64(reinterpret_cast<const char*>(Object.GetData()), Object.Len() * sizeof(*Object.GetData()));
	}

	IndexHash GenerateIndexHash(FName Object)
	{
		constexpr static const char SeedName[] = "FName";
		static uint64 Seed = CityHash64(SeedName, sizeof(SeedName) - 1);
		return CityHash128to64({ Seed, Object.ToUnstableInt() });
	}

	IndexHash GenerateIndexHash(const FSoftObjectPath& ObjectPath)
	{
		constexpr static const char SeedName[] = "FSoftObjectPath";
		static uint64 Seed = CityHash64(SeedName, sizeof(SeedName) - 1);

		FTopLevelAssetPath TopLevelAssetPath = ObjectPath.GetAssetPath();
		uint64 Hash = CityHash128to64({ Seed, GenerateIndexHash(TopLevelAssetPath.GetPackageName()) });
		Hash = CityHash128to64({ Hash, GenerateIndexHash(TopLevelAssetPath.GetAssetName()) });
		return CityHash128to64({ Hash, GenerateIndexHash(ObjectPath.GetSubPathString()) });
	}
} // namespace UE::Editor::DataStorage
