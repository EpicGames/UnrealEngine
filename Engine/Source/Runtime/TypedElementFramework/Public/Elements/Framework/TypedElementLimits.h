// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/NumericLimits.h"

#define UE_TYPED_ELEMENT_HAS_REFCOUNTING (1)
#define UE_TYPED_ELEMENT_HAS_REFTRACKING (UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT || WITH_EDITOR)
#define UE_TYPED_ELEMENT_HAS_REFERENCING (UE_TYPED_ELEMENT_HAS_REFCOUNTING || UE_TYPED_ELEMENT_HAS_REFTRACKING)

/**
 * Handle ID limits, as used by FTypedElementId.
 * @note Limited to a combined 32-bits so that they can be used directly within render targets, though could be made 64-bits if the 
 *       editor used 64-bit render targets (this would also require 64-bit container support in TTypedElementInternalDataStore).
 */
inline constexpr SIZE_T TypedHandleTypeIdBits = 8;
inline constexpr SIZE_T TypedHandleElementIdBits = 24;

inline constexpr SIZE_T TypedHandleTypeIdBytes = TypedHandleTypeIdBits >> 3;
inline constexpr SIZE_T TypedHandleElementIdBytes = TypedHandleElementIdBits >> 3;

inline constexpr SIZE_T TypedHandleMaxTypeId = ((SIZE_T)1 << TypedHandleTypeIdBits) - 1;
inline constexpr SIZE_T TypedHandleMaxElementId = ((SIZE_T)1 << TypedHandleElementIdBits) - 1;

inline constexpr SIZE_T TypedHandleRefTrackingSkipCount = 1;
inline constexpr SIZE_T TypedHandleRefTrackingDepth = 31;

using FTypedHandleTypeId = uint8;
using FTypedHandleElementId = int32;
using FTypedHandleCombinedId = uint32;

using FTypedElementRefCount = int32;
using FTypedElementReferenceId = int32;

static_assert(sizeof(FTypedHandleCombinedId) >= (TypedHandleTypeIdBytes + TypedHandleElementIdBytes), "FTypedHandleCombinedId is not large enough to hold the combination of TypedHandleTypeIdBytes and TypedHandleElementIdBytes!");

static_assert(TNumericLimits<FTypedHandleTypeId>::Max() >= TypedHandleMaxTypeId, "FTypedHandleTypeId is not large enough to hold TypedHandleMaxTypeId!");
static_assert(TNumericLimits<FTypedHandleElementId>::Max() >= TypedHandleMaxElementId, "FTypedHandleElementId is not large enough to hold TypedHandleMaxElementId!");
