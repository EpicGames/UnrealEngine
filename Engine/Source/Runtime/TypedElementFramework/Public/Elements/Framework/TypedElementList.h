// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Elements/Framework/TypedElementCounter.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "Elements/Framework/TypedElementLimits.h"
#include "Elements/Framework/TypedElementListFwd.h"
#include "HAL/Platform.h"
#include "Templates/Casts.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UInterface;
class UObject;
class UTypedElementRegistry;
struct FTypedElementId;

namespace TypedElementList_Private
{

TYPEDELEMENTFRAMEWORK_API void GetElementImpl(const UTypedElementRegistry* InRegistry, const FTypedElementHandle& InElementHandle, const TSubclassOf<UInterface>& InBaseInterfaceType, FTypedElement& OutElement);

TYPEDELEMENTFRAMEWORK_API void GetElementImpl(const UTypedElementRegistry* InRegistry, const FScriptTypedElementHandle& InElementHandle, const TSubclassOf<UInterface>& InBaseInterfaceType, FTypedElement& OutElement);

template <typename BaseInterfaceType, class HandleType>
inline void GetElement(const UTypedElementRegistry* InRegistry, const HandleType& InElementHandle, TTypedElement<BaseInterfaceType>& OutElement)
{
	static_assert(sizeof(TTypedElement<BaseInterfaceType>) == sizeof(FTypedElement), "All TTypedElement instances must be the same size for this cast implementation to work!");
	GetElementImpl(InRegistry, InElementHandle, BaseInterfaceType::UClassType::StaticClass(), reinterpret_cast<FTypedElement&>(OutElement));
}

template <typename BaseInterfaceType, class HandleType>
inline TTypedElement<BaseInterfaceType> GetElement(const UTypedElementRegistry* InRegistry, const HandleType& InElementHandle)
{
	TTypedElement<BaseInterfaceType> Element;
	GetElement(InRegistry, InElementHandle, Element);
	return Element;
}

} // namespace TypedElementList_Private


/**
 * A list of element handles.
 * Provides high-level access to groups of elements, including accessing elements that implement specific interfaces.
 */
template<class HandleType>
class TTypedElementList final : public TSharedFromThis<TTypedElementList<HandleType>>
{
public:

	using FHandleType = HandleType;
	using TTypedElementListRef = TSharedRef<TTypedElementList>;
	using TTypedElementListConstRef = TSharedRef<const TTypedElementList>;


	TYPEDELEMENTFRAMEWORK_API ~TTypedElementList();

	/**
	 * Clone this list instance.
	 * @note Only copies elements; does not copy any bindings!
	 */
	TYPEDELEMENTFRAMEWORK_API TTypedElementListRef Clone() const;

	/**
	 * Get the element handle at the given index.
	 * @note Use IsValidIndex to test for validity.
	 */
	inline HandleType operator[](const int32 InIndex) const
	{
		return GetElementHandleAt(InIndex);
	}

	/**
	 * Get the element handle at the given index.
	 * @note Use IsValidIndex to test for validity.
	 */
	inline HandleType GetElementHandleAt(const int32 InIndex) const
	{
		return ElementHandles[InIndex];
	}

	/**
	 * Get the element at the given index.
	 * @note Use IsValidIndex to test for validity.
	 */
	template <typename BaseInterfaceType>
	inline TTypedElement<BaseInterfaceType> GetElementAt(const int32 InIndex) const
	{
		return GetElement<BaseInterfaceType>(GetElementHandleAt(InIndex));
	}

	/**
	 * Get the element at the given index.
	 * @note Use IsValidIndex to test for validity.
	 */
	template <typename BaseInterfaceType>
	inline void GetElementAt(const int32 InIndex, TTypedElement<BaseInterfaceType>& OutElement) const
	{
		GetElement(GetElementHandleAt(InIndex), OutElement);
	}

	/**
	 * Get the element from the given handle.
	 */
	template <typename BaseInterfaceType>
	inline TTypedElement<BaseInterfaceType> GetElement(const HandleType& InElementHandle) const
	{
		return TypedElementList_Private::GetElement<BaseInterfaceType>(Registry.Get(), InElementHandle);
	}

	/**
	 * Get the element from the given handle.
	 */
	template <typename BaseInterfaceType>
	inline void GetElement(const HandleType& InElementHandle, TTypedElement<BaseInterfaceType>& OutElement) const
	{
		TypedElementList_Private::GetElement(Registry.Get(), InElementHandle, OutElement);
	}

	/**
	 * Get the first element implementing the given interface.
	 */
	template <typename BaseInterfaceType>
	TTypedElement<BaseInterfaceType> GetTopElement() const
	{
		TTypedElement<BaseInterfaceType> TempElement;
		for (int32 ElementIndex = 0; ElementIndex < Num(); ++ElementIndex)
		{
			GetElementAt(ElementIndex, TempElement);
			if (TempElement)
			{
				break;
			}
		}
		return TempElement;
	}

	/**
	 * Get the first element that implement the given interface and pass the predicate
	 * @Predicate A function that should return true when the element is desirable
	 */
	template <typename BaseInterfaceType>
	TTypedElement<BaseInterfaceType> GetTopElement(TFunctionRef<bool (const TTypedElement<BaseInterfaceType>&)> Predicate) const
	{
		TTypedElement<BaseInterfaceType> TempElement;
		TTypedElement<BaseInterfaceType> ElementToReturn;
		for (int32 ElementIndex = 0; ElementIndex < Num(); ++ElementIndex)
		{
			GetElementAt(ElementIndex, TempElement);
			if (TempElement && Predicate(TempElement))
			{
				ElementToReturn = MoveTemp(TempElement);
				break;
			}
		}
		return ElementToReturn;
	}


	/**
	 * Get the last element implementing the given interface.
	 */
	template <typename BaseInterfaceType>
	TTypedElement<BaseInterfaceType> GetBottomElement() const
	{
		TTypedElement<BaseInterfaceType> TempElement;
		for (int32 ElementIndex = Num() - 1; ElementIndex >= 0; --ElementIndex)
		{
			GetElementAt(ElementIndex, TempElement);
			if (TempElement)
			{
				break;
			}
		}
		return TempElement;
	}

	/**
	 * Get the last element that implement the given interface and pass the predicate.
	 * @Predicate A function that return should true when the element is desirable
	 */
	template <typename BaseInterfaceType>
	TTypedElement<BaseInterfaceType> GetBottomElement(TFunctionRef<bool (const TTypedElement<BaseInterfaceType>&)> Predicate) const
	{
		TTypedElement<BaseInterfaceType> TempElement;
		TTypedElement<BaseInterfaceType> ElementToReturn;
		for (int32 ElementIndex = Num() - 1; ElementIndex >= 0; --ElementIndex)
		{
			GetElementAt(ElementIndex, TempElement);
			if (TempElement && Predicate(TempElement))
			{
				ElementToReturn = MoveTemp(TempElement);
				break;
			}
		}
		return ElementToReturn;
	}



	/**
	 * Get the element interface from the given handle.
	 */
	template <typename BaseInterfaceType>
	BaseInterfaceType* GetElementInterface(const FTypedElementHandle& InElementHandle) const
	{
		return Cast<BaseInterfaceType>(GetElementInterface(InElementHandle, BaseInterfaceType::UClassType::StaticClass()));
	}

	/**
	 * Get the element interface from the given handle.
	 */
	TYPEDELEMENTFRAMEWORK_API UObject* GetElementInterface(const HandleType& InElementHandle, const TSubclassOf<UInterface>& InBaseInterfaceType) const;

	/**
	 * Test whether there are elements in this list, optionally filtering to elements that implement the given interface.
	 */
	TYPEDELEMENTFRAMEWORK_API bool HasElements(const TSubclassOf<UInterface>& InBaseInterfaceType = {}) const;

	/**
	 * Count the number of elements in this list, optionally filtering to elements that implement the given interface.
	 */
	TYPEDELEMENTFRAMEWORK_API int32 CountElements(const TSubclassOf<UInterface>& InBaseInterfaceType = {}) const;

	/**
	 * Test whether there are elements in this list of the given type.
	 */
	TYPEDELEMENTFRAMEWORK_API bool HasElementsOfType(const FName InElementTypeName) const;
	TYPEDELEMENTFRAMEWORK_API bool HasElementsOfType(const FTypedHandleTypeId InElementTypeId) const;

	/**
	 * Count the number of elements in this list of the given type.
	 */
	TYPEDELEMENTFRAMEWORK_API int32 CountElementsOfType(const FName InElementTypeName) const;
	TYPEDELEMENTFRAMEWORK_API int32 CountElementsOfType(const FTypedHandleTypeId InElementTypeId) const;

	/**
	 * Get the handle of every element in this list, optionally filtering to elements that implement the given interface.
	 */
	TYPEDELEMENTFRAMEWORK_API TArray<HandleType> GetElementHandles(const TSubclassOf<UInterface>& InBaseInterfaceType = {}) const;

	/**
	 * Get the handle of every element in this list, optionally filtering to elements that implement the given interface.
	 */
	template <typename ArrayAllocator>
	void GetElementHandles(TArray<HandleType, ArrayAllocator>& OutArray, const TSubclassOf<UInterface>& InBaseInterfaceType = {}) const
	{
		OutArray.Reset();
		OutArray.Reserve(ElementHandles.Num());
		ForEachElementHandle([&OutArray](const HandleType& InElementHandle)
		{
			OutArray.Add(InElementHandle);
			return true;
		}, InBaseInterfaceType);
	}

	/**
	 * Enumerate the handle of every element in this list, optionally filtering to elements that implement the given interface.
	 * @note Return true from the callback to continue enumeration.
	 */
	TYPEDELEMENTFRAMEWORK_API void ForEachElementHandle(TFunctionRef<bool(const HandleType&)> InCallback, const TSubclassOf<UInterface>& InBaseInterfaceType = {}) const;

	/**
	 * Enumerate the elements in this list that implement the given interface.
	 * @note Return true from the callback to continue enumeration.
	 */
	template <typename BaseInterfaceType>
	void ForEachElement(TFunctionRef<bool(const TTypedElement<BaseInterfaceType>&)> InCallback) const
	{
		TTypedElement<BaseInterfaceType> TempElement;
		for (const HandleType& ElementHandle : ElementHandles)
		{
			GetElement(ElementHandle, TempElement);
			if (TempElement && !InCallback(TempElement))
			{
				break;
			}
		}
	}

	/**
	 * Is the given index a valid entry within this element list?
	 */
	inline bool IsValidIndex(const int32 InIndex) const
	{
		return ElementHandles.IsValidIndex(InIndex);
	}

	/**
	 * Get the number of entries within this element list.
	 */
	inline int32 Num() const
	{
		return ElementHandles.Num();
	}

	/**
	 * Shrink this element list storage to avoid slack.
	 */
	inline void Shrink()
	{
		ElementCombinedIds.Shrink();
		ElementHandles.Shrink();
	}

	/**
	 * Pre-allocate enough memory in this element list to store the given number of entries.
	 */
	inline void Reserve(const int32 InSize)
	{
		ElementCombinedIds.Reserve(InSize);
		ElementHandles.Reserve(InSize);
	}

	/**
	 * Remove all entries from this element list, potentially leaving space allocated for the given number of entries.
	 */
	inline void Empty(const int32 InSlack = 0)
	{
		// Avoid creating unnecessary notifications
		const bool bWasEmpty = ElementHandles.IsEmpty();
		if (!bWasEmpty)
		{
			NoteListMayChange();
		}

		ElementCombinedIds.Empty(InSlack);
		ElementHandles.Empty(InSlack);
		ElementCounts.ClearCounters();

		if (!bWasEmpty)
		{
			ElementCounts.ClearCounters();
			NoteListChanged(EChangeType::Cleared);
		}
	}

	/**
	 * Remove all entries from this element list, preserving existing allocations.
	 */
	inline void Reset()
	{
		// Avoid creating unnecessary notifications
		if (!ElementHandles.IsEmpty())
		{
			NoteListMayChange();
			ElementCombinedIds.Reset();
			ElementHandles.Reset();
			ElementCounts.ClearCounters();
			NoteListChanged(EChangeType::Cleared);
		}
	}

	/**
	 * Does this element list contain an entry for the given element ID?
	 */
	inline bool Contains(const FTypedElementId& InElementId) const
	{
		return ContainsElementImpl(InElementId);
	}

	/**
	 * Does this element list contain an entry for the given element handle?
	 */
	inline bool Contains(const HandleType& InElementHandle) const
	{
		return ContainsElementImpl(InElementHandle.GetId());
	}

	/**
	 * Does this element list contain an entry for the given element owner?
	 */
	template <typename ElementDataType>
	inline bool Contains(const TTypedElementOwner<ElementDataType>& InElementOwner)
	{
		return ContainsElementImpl(InElementOwner.GetId());
	}

	/**
	 * Add the given element handle to this element list, if it isn't already in the list.
	 * @return True if the element handle was added, false if it is already in the list.
	 */
	inline bool Add(const HandleType& InElementHandle)
	{
		return AddElementImpl(CopyTemp(InElementHandle));
	}

	/**
	 * Add the given element handle to this element list, if it isn't already in the list.
	 * @return True if the element handle was added, false if it is already in the list.
	 */
	inline bool Add(HandleType&& InElementHandle)
	{
		return AddElementImpl(MoveTemp(InElementHandle));
	}

	/**
	 * Add the given element owner to this element list, if it isn't already in the list.
	 * @return True if the element owner was added, false if it is already in the list.
	 */
	template <typename ElementDataType>
	inline bool Add(const TTypedElementOwner<ElementDataType>& InElementOwner)
	{
		return AddElementImpl(InElementOwner.AcquireHandle());
	}

	/**
	 * Append another element list to this element list.
	 */
	void Append(const TTypedElementListConstRef& InElementList)
	{
		if (this != &InElementList.Get())
		{
			FLegacySyncScopedBatch LegacySyncBatch(*this);

			Reserve(Num() + InElementList->Num());
			InElementList->ForEachElementHandle([this](const HandleType& ElementHandle)
			{
				AddElementImpl(CopyTemp(ElementHandle));
				return true;
			});
		}
	}

	/**
	 * Append the given element handles to this element list.
	 */
	void Append(TArrayView<const HandleType> InElementHandles)
	{
		FLegacySyncScopedBatch LegacySyncBatch(*this);

		Reserve(Num() + InElementHandles.Num());
		for (const HandleType& ElementHandle : InElementHandles)
		{
			AddElementImpl(CopyTemp(ElementHandle));
		}
	}

	/**
	 * Append the given element owners to this element list.
	 */
	template <typename ElementDataType>
	inline void Append(const TArray<TTypedElementOwner<ElementDataType>>& InElementOwners)
	{
		Append(MakeArrayView(InElementOwners));
	}

	/**
	 * Append the given element owners to this element list.
	 */
	template <typename ElementDataType>
	void Append(TArrayView<const TTypedElementOwner<ElementDataType>> InElementOwners)
	{
		FLegacySyncScopedBatch LegacySyncBatch(*this);

		Reserve(Num() + InElementOwners.Num());
		for (const TTypedElementOwner<ElementDataType>& ElementOwner : InElementOwners)
		{
			AddElementImpl(ElementOwner.AcquireHandle());
		}
	}

	/**
	 * Remove the given element ID from this element list, if it is in the list.
	 * @return True if the element ID was removed, false if it isn't in the list.
	 */
	inline bool Remove(const FTypedElementId& InElementId)
	{
		return RemoveElementImpl(InElementId);
	}

	/**
	 * Remove the given element handle from this element list, if it is in the list.
	 * @return True if the element handle was removed, false if it isn't in the list.
	 */
	inline bool Remove(const HandleType& InElementHandle)
	{
		return RemoveElementImpl(InElementHandle.GetId());
	}

	/**
	 * Remove the given element owner from this element list, if it is in the list.
	 * @return True if the element owner was removed, false if it isn't in the list.
	 */
	template <typename ElementDataType>
	inline bool Remove(const TTypedElementOwner<ElementDataType>& InElementOwner)
	{
		return RemoveElementImpl(InElementOwner.GetId());
	}

	/**
	 * Remove any element handles that match the given predicate from this element list.
	 * @return The number of element handles removed.
	 */
	inline int32 RemoveAll(TFunctionRef<bool(const HandleType&)> InPredicate)
	{
		return RemoveAllElementsImpl(InPredicate);
	}

	/**
	 * Remove any elements that match the given predicate from this element list.
	 * @return The number of elements removed.
	 */
	template <typename BaseInterfaceType>
	int32 RemoveAll(TFunctionRef<bool(const TTypedElement<BaseInterfaceType>&)> InPredicate)
	{
		TTypedElement<BaseInterfaceType> TempElement;
		return RemoveAllElementsImpl([this, &TempElement, &InPredicate](const HandleType& InElementHandle)
		{
			GetElement(InElementHandle, TempElement);
			return TempElement && InPredicate(TempElement);
		});
	}

	/**
	 * Get the counter for the elements within the list.
	 */
	const FTypedElementCounter& GetCounter() const
	{
		return ElementCounts;
	}

	/**
	 * Access the delegate that is invoked whenever this element list is potentially about to change.
	 * @note This may be called even if no actual change happens, so may be called multiple times without a corresponding OnChanged notification.
	 */
	DECLARE_EVENT_OneParam(TTypedElementList, FOnPreChange, const TTypedElementList& /*InElementList*/);
	FOnPreChange& OnPreChange()
	{
		return OnPreChangeDelegate;
	}

	/**
	 * Access the delegate that is invoked whenever this element list has been changed.
	 * @note This is called automatically at the end of each frame, but can also be manually invoked by NotifyPendingChanges.
	 */
	DECLARE_EVENT_OneParam(TTypedElementList, FOnChanged, const TTypedElementList& /*InElementList*/);
	FOnChanged& OnChanged()
	{
		return OnChangedDelegate;
	}

	/**
	 * Invoke the delegate called whenever this element list has been changed.
	 * @return true if a change notification was emitted
	 */
	TYPEDELEMENTFRAMEWORK_API bool NotifyPendingChanges();

	/**
	 * A utility struct that help to cancel any new pending notification that happened in a scope.
	 * Note: it won't cancel a notification if there is a legacy batch operation ongoing
	 */
	struct FScopedClearNewPendingChange
	{
		FScopedClearNewPendingChange() = default;
		TYPEDELEMENTFRAMEWORK_API FScopedClearNewPendingChange(TTypedElementList& InTypeElementList);

		FScopedClearNewPendingChange(const FScopedClearNewPendingChange&) = delete;
		FScopedClearNewPendingChange& operator=(const FScopedClearNewPendingChange&) = delete;

		TYPEDELEMENTFRAMEWORK_API FScopedClearNewPendingChange(FScopedClearNewPendingChange&& Other);
		TYPEDELEMENTFRAMEWORK_API FScopedClearNewPendingChange& operator=(FScopedClearNewPendingChange&& Other);

		TYPEDELEMENTFRAMEWORK_API ~FScopedClearNewPendingChange();

	private:
		TTypedElementList* TypedElementList = nullptr;
	};

	/**
	 * Interface to allow external systems (such as USelection) to receive immediate sync notifications as an element list is changed.
	 * This exists purely as a bridging mechanism and shouldn't be relied on for new code. It is lazily created as needed.
	 */
	class FLegacySync
	{
	public:
		enum class ESyncType : uint8
		{
			/**
			 * An element was added to the element list.
			 * The ElementHandle argument will be set to the element that was added.
			 */
			Added,

			/**
			 * An element was removed from the element list.
			 * The ElementHandle argument will be set to the element that was removed.
			 */
			Removed,

			/**
			 * The element list was modified in an unknown way.
			 * The ElementHandle argument will be unset.
			 */
			Modified,

			/**
			 * The element list was cleared.
			 * The ElementHandle argument will be unset.
			 */
			Cleared,

			/**
			 * The element list was modified as part of a batch or bulk operation.
			 * The ElementHandle argument will be unset.
			 * @note A batch operation will emit internal (bIsWithinBatchOperation=true) Added, Removed, Modified and Cleared updates during the batch, 
			 *       so if you respond to those internal updates you may choose to ignore this one. Otherwise you should treat it the same as Modified.
			 */
			BatchComplete,
		};
	
		TYPEDELEMENTFRAMEWORK_API FLegacySync(const TTypedElementList& InElementList);

		TYPEDELEMENTFRAMEWORK_API void Private_EmitSyncEvent(const ESyncType InSyncType, const HandleType& InElementHandle = HandleType());

		DECLARE_EVENT_FourParams(FLegacySync, FOnSyncEvent, const TTypedElementList& /*InElementList*/, ESyncType /*InSyncType*/, const HandleType& /*InElementHandle*/, bool /*bIsWithinBatchOperation*/);
		TYPEDELEMENTFRAMEWORK_API FOnSyncEvent& OnSyncEvent();

		TYPEDELEMENTFRAMEWORK_API bool IsRunningBatchOperation() const;
		TYPEDELEMENTFRAMEWORK_API void BeginBatchOperation();
		TYPEDELEMENTFRAMEWORK_API void EndBatchOperation(const bool InNotify = true);
		TYPEDELEMENTFRAMEWORK_API bool IsBatchOperationDirty() const;
		TYPEDELEMENTFRAMEWORK_API void ForceBatchOperationDirty();

	private:
		const TTypedElementList& ElementList;

		FOnSyncEvent OnSyncEventDelegate;

		int32 NumOpenBatchOperations = 0;
		bool bBatchOperationIsDirty = false;
	};

	/**
	 * Helper to batch immediate sync notifications for legacy code.
	 * Does nothing if no legacy sync has been created for the given instance.
	 */
	class FLegacySyncScopedBatch
	{
	public:
		TYPEDELEMENTFRAMEWORK_API explicit FLegacySyncScopedBatch(const TTypedElementList& InElementList, const bool InNotify = true);
		TYPEDELEMENTFRAMEWORK_API ~FLegacySyncScopedBatch();

		FLegacySyncScopedBatch(const FLegacySyncScopedBatch&) = delete;
		FLegacySyncScopedBatch& operator=(const FLegacySyncScopedBatch&) = delete;

		FLegacySyncScopedBatch(FLegacySyncScopedBatch&&) = delete;
		FLegacySyncScopedBatch& operator=(FLegacySyncScopedBatch&&) = delete;

		TYPEDELEMENTFRAMEWORK_API bool IsDirty() const;
		TYPEDELEMENTFRAMEWORK_API void ForceDirty();

	private:
		FLegacySync* ElementListLegacySync = nullptr;
		bool bNotify = true;
	};

	/**
	 * Get a scoped object that when destroyed it clear a pending change notification without emitting the notification if it happened during its lifecycle.
	 */
	TYPEDELEMENTFRAMEWORK_API FScopedClearNewPendingChange GetScopedClearNewPendingChange();

	/**
	 * Access the interface to allow external systems (such as USelection) to receive immediate sync notifications as an element list is changed.
	 * This exists purely as a bridging mechanism and shouldn't be relied on for new code. It is lazily created as needed.
	 */
	TYPEDELEMENTFRAMEWORK_API FLegacySync& Legacy_GetSync();

	/**
	 * Access the interface to allow external systems (such as USelection) to receive immediate sync notifications as an element list is changed.
	 * This exists purely as a bridging mechanism and shouldn't be relied on for new code. This will return null if no legacy sync has been created for this instance.
	 */
	TYPEDELEMENTFRAMEWORK_API FLegacySync* Legacy_GetSyncPtr() const;

	/**
	 * Internal function used by the element registry to create an element list instance.
	 */
	static TYPEDELEMENTFRAMEWORK_API TTypedElementListRef Private_CreateElementList(UTypedElementRegistry* InRegistry);

	UTypedElementRegistry* GetRegistry() const
	{
		return Registry.Get();
	}

private:
	enum class EChangeType : uint8
	{
		/**
		* An element was added to the element list.
		* The ElementHandle argument will be set to the element that was added.
		*/
		Added,

		/**
		* An element was removed from the element list.
		* The ElementHandle argument will be set to the element that was removed.
		*/
		Removed,

		/**
		* The element list was cleared.
		* The ElementHandle argument will be unset.
		*/
		Cleared,
	};

	TYPEDELEMENTFRAMEWORK_API explicit TTypedElementList(UTypedElementRegistry* InRegistry);

	TYPEDELEMENTFRAMEWORK_API bool AddElementImpl(HandleType&& InElementHandle);
	TYPEDELEMENTFRAMEWORK_API bool RemoveElementImpl(const FTypedElementId& InElementId);
	TYPEDELEMENTFRAMEWORK_API int32 RemoveAllElementsImpl(TFunctionRef<bool(const HandleType&)> InPredicate);
	TYPEDELEMENTFRAMEWORK_API bool ContainsElementImpl(const FTypedElementId& InElementId) const;

	TYPEDELEMENTFRAMEWORK_API void NoteListMayChange();
	TYPEDELEMENTFRAMEWORK_API void NoteListChanged(const EChangeType InChangeType, const HandleType& InElementHandle = HandleType());

	/**
	 * Element registry this element list is associated with.
	 */
	TWeakObjectPtr<UTypedElementRegistry> Registry;

	/**
	 * Set of combined ID values that are currently present in this element list.
	 * Used to perform optimized querying of which elements are in this list, and to avoid adding duplicate entries.
	 */
	TSet<FTypedHandleCombinedId> ElementCombinedIds;

	/**
	 * Array of element handles present in this element list.
	 * These are stored in the same order that they are added, and the set above can be used to optimize certain queries.
	 */
	TArray<HandleType> ElementHandles;

	/**
	 * Tracks various categories of counters for the elements within the list (eg, the number of elements of a given type).
	 */
	FTypedElementCounter ElementCounts;

	/**
	 * Delegate that is invoked whenever this element list is potentially about to change.
	 */
	FOnPreChange OnPreChangeDelegate;

	/**
	 * Delegate that is invoked whenever this element list has been changed.
	 */
	FOnChanged OnChangedDelegate;

	/**
	 * Whether there are pending changes for OnChangedDelegate to notify for.
	 */
	bool bHasPendingNotify = false;

	/**
	 * Interface to allow external systems (such as USelection) to receive immediate sync notifications as an element list is changed.
	 * This exists purely as a bridging mechanism and shouldn't be relied on for new code. It is lazily created as needed.
	 */
	TUniquePtr<FLegacySync> LegacySync;
};


namespace UE::TypedElementFramework
{
	/**
	 * Functions to convert a script list to a native one
	 * @note Only copies elements; does not copy any bindings!
	 */
	TYPEDELEMENTFRAMEWORK_API FTypedElementListPtr ConvertToNativeTypedElementList(const FScriptTypedElementListConstPtr& ScriptList);

	/**
	 * Functions to convert a script list to a native one
	 * @note Only copies elements; does not copy any bindings!
	 */
	TYPEDELEMENTFRAMEWORK_API FScriptTypedElementListPtr ConvertToScriptTypedElementList(const FTypedElementListConstPtr& NativeList);

}
