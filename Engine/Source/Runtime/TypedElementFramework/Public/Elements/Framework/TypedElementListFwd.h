// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

template<class HandleType>
class TTypedElementList;


struct FTypedElementHandle;

// Native typed element list.
using FTypedElementList = TTypedElementList<FTypedElementHandle>;

using FTypedElementListPtr = TSharedPtr<FTypedElementList>;
using FTypedElementListRef = TSharedRef<FTypedElementList>;

using FTypedElementListConstPtr = TSharedPtr<const FTypedElementList>;
using FTypedElementListConstRef = TSharedRef<const FTypedElementList>;


struct FScriptTypedElementHandle;

// Script typed element list. It should only be use for the script exposure apis since the script handles does have a performance overhead over the normal handles.
using FScriptTypedElementList = TTypedElementList<FScriptTypedElementHandle>;

using FScriptTypedElementListPtr = TSharedPtr<FScriptTypedElementList>;
using FScriptTypedElementListRef = TSharedRef<FScriptTypedElementList>;

using FScriptTypedElementListConstPtr = TSharedPtr<const FScriptTypedElementList>;
using FScriptTypedElementListConstRef = TSharedRef<const FScriptTypedElementList>;

