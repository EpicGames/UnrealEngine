// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Elements/Framework/TypedElementListFwd.h"
#include "TypedElementListProxy.generated.h"

/**
 * A list of script element handles (proxy to a FScriptTypedElementList instance).
 * Provides high-level access to groups of elements, including accessing elements that implement specific interfaces.
 * 
 * Note: the script list proxy use should be avoided when not using it for the script exposure apis.
 * The weak model for the handles come with an additional cost to the runtime performance and the memory usage.
 */
USTRUCT(BlueprintType, DisplayName="Typed Element List", meta=(ScriptName="TypedElementList"))
struct FScriptTypedElementListProxy
{
	GENERATED_BODY()

public:
	FScriptTypedElementListProxy() = default;

	FScriptTypedElementListProxy(FScriptTypedElementListRef InElementList)
		: ElementList(MoveTemp(InElementList))
	{
	}

	FScriptTypedElementListProxy(FScriptTypedElementListConstRef InElementList)
		: ElementList(ConstCastSharedRef<FScriptTypedElementList>(InElementList))
	{
	}

	FScriptTypedElementListPtr GetElementList()
	{
		return ElementList;
	}

	FScriptTypedElementListConstPtr GetElementList() const
	{
		return ElementList;
	}

private:
	FScriptTypedElementListPtr ElementList;
};

