// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "CoreMinimal.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/TVariant.h"
#include "Templates/Tuple.h"
#include "UObject/WeakObjectPtr.h"

class UScriptStruct;

namespace UE::Editor::DataStorage
{
	struct FQueryDescription;

	inline static const FName IsEditableName("IsEditable");
	inline static const FName IsConstName("IsConst");

	using MetaDataType = TVariant<bool, uint64, int64, double, FString>;
	using MetaDataTypeView = TVariant<FEmptyVariantState, bool, uint64, int64, double, const FString*>;

	/**
	 * Short lived view of single entry in the meta data container.
	 */
	class FMetaDataEntryView final
	{
	public:
		TYPEDELEMENTFRAMEWORK_API FMetaDataEntryView();
		TYPEDELEMENTFRAMEWORK_API explicit FMetaDataEntryView(const MetaDataType& MetaData);
		TYPEDELEMENTFRAMEWORK_API explicit FMetaDataEntryView(const FString& MetaDataString);
		/** Explicit constructor. The provided type must match exactly to one of the available stored types. */
		template<typename T>
		explicit FMetaDataEntryView(T&& MetaDataValue);

		/** Returns true if set to a value, otherwise false. */
		TYPEDELEMENTFRAMEWORK_API bool IsSet() const;
		/** Checks if the stored value matches the requested type. */
		template<typename T>
		bool IsType() const;
		/** Returns the value if the requested type matches exactly with the stored type, otherwise returns a nullptr. */
		template<typename T>
		const T* TryGetExact() const;

	private:
		MetaDataTypeView DataView;
	};

	/**
	 * Base class to store meta data for use within the Typed Elements Data Storage.
	 */
	class FMetaDataBase
	{
	public:
		template<typename T>
		bool AddImmutableData(FName Name, T&& Value);
		template<typename T>
		void AddOrSetMutableData(FName Name, T&& Value);

		TYPEDELEMENTFRAMEWORK_API virtual FMetaDataEntryView Find(FName Name) const;

		TYPEDELEMENTFRAMEWORK_API virtual void Shrink();
		
		TYPEDELEMENTFRAMEWORK_API virtual bool IsEmpty() const;

	protected:
		FMetaDataBase() = default;
		virtual ~FMetaDataBase() = default;

		/** Data that can be added once but can't be changed afterwards. Values here always take priority over other values. */
		TMap<FName, MetaDataType> ImmutableData;
		/** Data that can be added and can have their value updated afterwards. */
		TMap<FName, MetaDataType> MutableData;
	};

	/** General storage for meta data for the Typed Elements Data Storage. */
	class FMetaData final : public FMetaDataBase {};

	/**
	 * Meta data that's specifically associated with a single column.
	 */
	class FColumnMetaData final : public FMetaDataBase
	{
	public:
		enum class EFlags
		{
			None = 0,
			IsMutable = 1 << 0
		};

		FColumnMetaData() = default;
		TYPEDELEMENTFRAMEWORK_API FColumnMetaData(const UScriptStruct* InColumnType, EFlags InFlags);

		TYPEDELEMENTFRAMEWORK_API FMetaDataEntryView Find(FName Name) const override;

		TYPEDELEMENTFRAMEWORK_API const UScriptStruct* GetColumnType() const;
		
		TYPEDELEMENTFRAMEWORK_API bool IsEmpty() const override;

	private:
		/** If set, properties on the column will also be included if a value isn't found in the (im)mutable data map. */
		const UScriptStruct* ColumnType{ nullptr };
		/** Flags indicating the behavior of the column */
		EFlags Flags = EFlags::None;
	};

	/**
	 * Short lived view of a meta data container.
	 */
	class FMetaDataView
	{
	public:
		virtual ~FMetaDataView() = default;

		TYPEDELEMENTFRAMEWORK_API virtual FMetaDataEntryView FindGeneric(FName AttributeName) const;
		TYPEDELEMENTFRAMEWORK_API virtual FMetaDataEntryView FindForColumn(
			TWeakObjectPtr<const UScriptStruct> Column, FName AttributeName) const;
		template<typename Column>
		FMetaDataEntryView FindForColumn(FName AttributeName) const;

		/** First tries FindForColumn and if unsuccessful, calls FindGeneric. */
		TYPEDELEMENTFRAMEWORK_API FMetaDataEntryView FindForColumnOrGeneric(
			TWeakObjectPtr<const UScriptStruct> Column, FName AttributeName) const;
		template<typename Column>
		FMetaDataEntryView FindForColumnOrGeneric(FName AttributeName) const;
		/** Tries FindForColumn for each given column in order until successful, otherwise calls FindGeneric. */
		TYPEDELEMENTFRAMEWORK_API FMetaDataEntryView FindForColumnsOrGeneric(
			const TConstArrayView<TWeakObjectPtr<const UScriptStruct>> Columns, FName AttributeName) const;
		template<typename... Columns>
		FMetaDataEntryView FindForColumnsOrGeneric(FName AttributeName) const;
	};

	/**
	 * Short lived view of a meta data container that wraps a query.
	 */
	class FQueryMetaDataView final : public FMetaDataView
	{
	public:
		~FQueryMetaDataView() override = default;

		TYPEDELEMENTFRAMEWORK_API explicit FQueryMetaDataView(const FQueryDescription& InQuery);
		
		TYPEDELEMENTFRAMEWORK_API FMetaDataEntryView FindGeneric(FName AttributeName) const override;
		TYPEDELEMENTFRAMEWORK_API FMetaDataEntryView FindForColumn(
			TWeakObjectPtr<const UScriptStruct> Column, FName AttributeName) const override;

	private:
		const FQueryDescription& Query;
	};

	/**
	 * Short lived view of a meta data container that wraps a list of columns.
	 */
	class FColumnsMetaDataView final : public FMetaDataView
	{
	public:
		~FColumnsMetaDataView() override = default;

		TYPEDELEMENTFRAMEWORK_API explicit FColumnsMetaDataView(TConstArrayView<TWeakObjectPtr<const UScriptStruct>> InColumns);
		TYPEDELEMENTFRAMEWORK_API explicit FColumnsMetaDataView(TConstArrayView<TSharedPtr<FColumnMetaData>> InColumnsMetaData);
		TYPEDELEMENTFRAMEWORK_API FMetaDataEntryView FindForColumn(
			TWeakObjectPtr<const UScriptStruct> Column, FName AttributeName) const override;

	private:
		TConstArrayView<TWeakObjectPtr<const UScriptStruct>> Columns;
		TConstArrayView<TSharedPtr<FColumnMetaData>> ColumnsMetaData;
	};

	/**
	 * Short lived view of a meta data container that wraps generic meta data.
	 */
	class FGenericMetaDataView final : public FMetaDataView
	{
	public:
		~FGenericMetaDataView() override = default;

		TYPEDELEMENTFRAMEWORK_API explicit FGenericMetaDataView(const FMetaData& InMetaData);
		TYPEDELEMENTFRAMEWORK_API FMetaDataEntryView FindGeneric(FName AttributeName) const override;

	private:
		const FMetaData& MetaData;
	};

	/**
	 * Short lived view of a meta data container that wraps around another meta data view so they can be chained.
	 */
	class FForwardingMetaDataView final : public FMetaDataView
	{
	public:
		~FForwardingMetaDataView() override = default;

		TYPEDELEMENTFRAMEWORK_API explicit FForwardingMetaDataView(const FMetaDataView& InView);
		TYPEDELEMENTFRAMEWORK_API FMetaDataEntryView FindGeneric(FName AttributeName) const override;
		TYPEDELEMENTFRAMEWORK_API FMetaDataEntryView FindForColumn(
			TWeakObjectPtr<const UScriptStruct> Column, FName AttributeName) const override;

	private:
		const FMetaDataView& View;
	};

	/**
	 * Short lived view of a meta data container.
	 */
	template<typename... ViewTypes>
	class FComboMetaDataView final : public FMetaDataView
	{
	public:
		FComboMetaDataView(const ViewTypes&... InViews);
		FComboMetaDataView(ViewTypes&&... InViews);

		template<typename NextViewType>
		FComboMetaDataView<ViewTypes..., NextViewType> Next(NextViewType&& NextView);

		FMetaDataEntryView FindGeneric(FName AttributeName) const override;
		FMetaDataEntryView FindForColumn(TWeakObjectPtr<const UScriptStruct> Column, FName AttributeName) const;

		TTuple<ViewTypes...> Views;
	};

} // UE::Editor::DataStorage

#include "Elements/Framework/TypedElementMetaData.inl"