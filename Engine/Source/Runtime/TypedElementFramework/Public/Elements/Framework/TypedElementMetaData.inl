// Copyright Epic Games, Inc. All Rights Reserved.

namespace UE::Editor::DataStorage
{
	namespace Private
	{
		template<typename TContainer, typename T>
		void AddDataToContainer(TContainer& Container, FName Name, T Value)
		{
			using BaseType = std::remove_const_t<std::remove_reference_t<T>>;

			if constexpr (std::is_arithmetic_v<BaseType>)
			{
				if constexpr (std::is_same_v<bool, BaseType>)
				{
					Container.Add(Name, MetaDataType(TInPlaceType<bool>(), static_cast<bool>(Value)));
				}
				else if constexpr (std::is_integral_v<BaseType>)
				{
					if constexpr (std::is_signed_v<BaseType>)
					{
						Container.Add(Name, MetaDataType(TInPlaceType<int64>(), static_cast<int64>(Value)));
					}
					else
					{
						Container.Add(Name, MetaDataType(TInPlaceType<uint64>(), static_cast<uint64>(Value)));
					}
				}
				else if constexpr (std::is_floating_point_v<BaseType>)
				{
					Container.Add(Name, MetaDataType(TInPlaceType<double>(), static_cast<double>(Value)));
				}
			}
			else if constexpr (std::is_convertible_v<BaseType, FString>)
			{
				Container.Add(Name, MetaDataType(TInPlaceType<FString>(), Forward<T>(Value)));
			}
			else
			{
				// !std::is_same_v<T, T> is used to force evaluation of the type so it's not always called, which would happen if "false" is used instead.
				static_assert(!std::is_same_v<T, T>, "Provided meta data type can't be stored in the Typed Element Data Storage.");
			}
		}
	}

	ENUM_CLASS_FLAGS(FColumnMetaData::EFlags)

		template<typename T>
	bool FMetaDataBase::AddImmutableData(FName Name, T&& Value)
	{
		if (!ImmutableData.Contains(Name))
		{
			Private::AddDataToContainer(ImmutableData, Name, Forward<T>(Value));
			return true;
		}
		else
		{
			return false;
		}
	}

	template<typename T>
	void FMetaDataBase::AddOrSetMutableData(FName Name, T&& Value)
	{
		Private::AddDataToContainer(MutableData, Name, Forward<T>(Value));
	}



	//
	// FMetaDataEntryView
	//

	template<typename T>
	FMetaDataEntryView::FMetaDataEntryView(T&& MetaDataValue)
		: DataView(TInPlaceType<T>(), Forward<T>(MetaDataValue))
	{}

	template<typename T>
	bool FMetaDataEntryView::IsType() const
	{
		return DataView.IsType<T>();
	}

	template<typename T>
	const T* FMetaDataEntryView::TryGetExact() const
	{
		if constexpr (std::is_same_v<T, const FString&>)
		{
			return DataView.TryGet<const FString*>();
		}
		return DataView.TryGet<T>();
	}



	//
	// FMetaDataView
	//

	template<typename Column>
	FMetaDataEntryView FMetaDataView::FindForColumn(FName AttributeName) const
	{
		return FindForColumn(Column::StaticStruct(), AttributeName);
	}

	template<typename Column>
	FMetaDataEntryView FMetaDataView::FindForColumnOrGeneric(FName AttributeName) const
	{
		return FindForColumnOrGeneric(Column::StaticStruct(), AttributeName);
	}

	template<typename... Columns>
	FMetaDataEntryView FMetaDataView::FindForColumnsOrGeneric(FName AttributeName) const
	{
		const TWeakObjectPtr<const UScriptStruct> ColumnStructs[] = { Columns::StaticStruct()... };
		return FindForColumnsOrGeneric(ColumnStructs, AttributeName);
	}



	//
	// FComboMetaDataView
	//

	template<typename... ViewTypes>
	FComboMetaDataView<ViewTypes...>::FComboMetaDataView(const ViewTypes&... InViews)
		: Views(InViews...)
	{
	}

	template<typename... ViewTypes>
	FComboMetaDataView<ViewTypes...>::FComboMetaDataView(ViewTypes&&... InViews)
		: Views(Forward<ViewTypes>(InViews)...)
	{
	}

	template<typename... ViewTypes>
	template<typename NextViewType>
	FComboMetaDataView<ViewTypes..., NextViewType> FComboMetaDataView<ViewTypes...>::Next(NextViewType&& NextView)
	{
		return Views.ApplyBefore([](const ViewTypes&... InViews, const NextViewType& InNextView)
			{
				return FComboMetaDataView<ViewTypes..., NextViewType>(InViews..., InNextView);
			}, Forward<NextViewType>(NextView));
	}

	template<typename... ViewTypes>
	FMetaDataEntryView FComboMetaDataView<ViewTypes...>::FindGeneric(FName AttributeName) const
	{
		return Views.ApplyAfter([&AttributeName](const ViewTypes&... InViews)
			{
				FMetaDataEntryView Result;
				auto Run = [](FName AttributeName, FMetaDataEntryView& Result, const auto& View)
				{
					Result = View.FindGeneric(AttributeName);
					return Result.IsSet();
				};
				(Run(AttributeName, Result, InViews) || ...);
				return Result;
			});
	}

	template<typename... ViewTypes>
	FMetaDataEntryView FComboMetaDataView<ViewTypes...>::FindForColumn(TWeakObjectPtr<const UScriptStruct> Column, FName AttributeName) const
	{
		return Views.ApplyAfter([&Column, &AttributeName](const ViewTypes&... InViews)
			{
				FMetaDataEntryView Result;
				auto Run = [](TWeakObjectPtr<const UScriptStruct> Column, FName AttributeName, FMetaDataEntryView& Result, const auto& View)
				{
					Result = View.FindForColumn(Column, AttributeName);
					return Result.IsSet();
				};
				(Run(Column, AttributeName, Result, InViews) || ...);
				return Result;
			});
	}
} // namespace UE::Editor::DataStorage
