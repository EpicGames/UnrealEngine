// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/ScopeLock.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Elements/Framework/TypedElementHandle.h"

/**
 * Type to keep a lock on the TTypedElementOwnerStore for the duration 
 * of time that a TTypedElementOwner is being externally referenced.
 * This avoids invalid memory access if the map is reallocated.
 */
template <typename ElementOwnerType>
struct TTypedElementOwnerScopedAccessImpl
{
public:
	explicit TTypedElementOwnerScopedAccessImpl(FTransactionallySafeCriticalSection* InSynchObject)
		: SynchObject(InSynchObject)
	{
		check(SynchObject);
		SynchObject->Lock();
	}

	~TTypedElementOwnerScopedAccessImpl()
	{
		if (SynchObject)
		{
			SynchObject->Unlock();
		}
	}

	TTypedElementOwnerScopedAccessImpl(const TTypedElementOwnerScopedAccessImpl&) = delete;
	TTypedElementOwnerScopedAccessImpl& operator=(const TTypedElementOwnerScopedAccessImpl&) = delete;

	TTypedElementOwnerScopedAccessImpl(TTypedElementOwnerScopedAccessImpl&& InOther)
		: SynchObject(InOther.SynchObject)
		, ElementOwner(InOther.ElementOwner)
	{
		InOther.SynchObject = nullptr;
		InOther.ElementOwner = nullptr;
	}

	TTypedElementOwnerScopedAccessImpl& operator=(TTypedElementOwnerScopedAccessImpl&&) = delete;

	inline explicit operator bool() const
	{
		return IsSet();
	}

	inline bool IsSet() const
	{
		return ElementOwner != nullptr;
	}

	inline ElementOwnerType& operator*() const
	{
		check(ElementOwner);
		return *ElementOwner;
	}

	inline ElementOwnerType* operator->() const
	{
		check(ElementOwner);
		return ElementOwner;
	}

	void Private_SetElementOwner(ElementOwnerType* InElementOwner)
	{
		checkf(!ElementOwner, TEXT("Element owner was already set!"));
		ElementOwner = InElementOwner;

		// If we didn't find an element then we can unlock immediately
		// Otherwise we hold the lock for as long as the element is being read to avoid a map write from invalidating the reference from reallocation
		check(SynchObject);
		if (!ElementOwner)
		{
			SynchObject->Unlock();
			SynchObject = nullptr;
		}
	}

private:
	FTransactionallySafeCriticalSection* SynchObject = nullptr;
	ElementOwnerType* ElementOwner = nullptr;
};

template <typename ElementDataType>
using TTypedElementOwnerScopedAccess = TTypedElementOwnerScopedAccessImpl<const TTypedElementOwner<ElementDataType>>;

template <typename ElementDataType>
using TTypedElementOwnerScopedMutableAccess = TTypedElementOwnerScopedAccessImpl<TTypedElementOwner<ElementDataType>>;

template <typename KeyDataType>
inline FString GetTypedElementOwnerStoreKeyDebugString(const KeyDataType& InKey)
{
	return TEXT("<GetTypedElementOwnerStoreKeyDebugString not implemented for key type>");
}

inline FString GetTypedElementOwnerStoreKeyDebugString(const UObject* InKey)
{
	return InKey->GetPathName();
}

/**
 * A store of element owners tied to their corresponding owner instance.
 * This can be used to track element owners for an instance without having to add new data to the type itself.
 */
template <typename ElementDataType, typename KeyDataType = const void*>
struct TTypedElementOwnerStore
{
public:
	static_assert(TNumericLimits<int32>::Max() >= TypedHandleMaxElementId, "TTypedElementOwnerStore internally uses TMap so cannot store TypedHandleMaxElementId! Consider making this container 64-bit aware, or explicitly remove this compile time check.");

	TTypedElementOwnerStore() = default;

	~TTypedElementOwnerStore()
	{
		checkf(ElementOwnerMap.Num() == 0, TEXT("Element owners were still registered during destruction! This will leak elements, and you should unregister and destroy all elements prior to destruction!"));
	}

	/**
	 * Register the given element owner, transferring ownership of the owner instance to this store.
	 * @note This will lock the store for writes until the scoped access struct is no longer being used.
	 * @note This must be paired with a call to UnregisterElementOwner.
	 */
	TTypedElementOwnerScopedAccess<ElementDataType> RegisterElementOwner(const KeyDataType& InKey, TTypedElementOwner<ElementDataType>&& InElementOwner)
	{
		TTypedElementOwnerScopedAccess<ElementDataType> ScopedAccess(&ElementOwnerMapCS);
		checkf(!ElementOwnerMap.Contains(InKey), TEXT("Element owner has already been registered for this key (%s)! This will leak elements, and you should unregister and destroy the old element owner for this instance before adding a new one!"), *GetTypedElementOwnerStoreKeyDebugString(InKey));
		checkf(InElementOwner, TEXT("Element owner passed to RegisterElementOwner was null for key (%s)! Element owners must be valid!"), *GetTypedElementOwnerStoreKeyDebugString(InKey));
		ScopedAccess.Private_SetElementOwner(&ElementOwnerMap.Add(InKey, MoveTemp(InElementOwner)));
		return ScopedAccess;
	}

	/**
	 * Unregister the given element owner, transferring ownership of the owner instance back to the caller.
	 * @note The returned owner may be unset if the key was not registered.
	 */
	TTypedElementOwner<ElementDataType> UnregisterElementOwner(const KeyDataType& InKey)
	{
		TTypedElementOwner<ElementDataType> ElementOwner;
		{
			UE::TScopeLock Lock(ElementOwnerMapCS);
			ElementOwnerMap.RemoveAndCopyValue(InKey, ElementOwner);
		}
		return ElementOwner;
	}

	/**
	 * Unregister any given element owners that match the given predicate, transferring ownership of the owner instance to the given callback.
	 */
	void UnregisterElementOwners(TFunctionRef<bool(const TTypedElementOwner<ElementDataType>&)> InShouldUnregisterElement, TFunctionRef<void(TTypedElementOwner<ElementDataType>&&)> InOnUnregisterElement)
	{
		UE::TScopeLock Lock(ElementOwnerMapCS);
		for (auto It = ElementOwnerMap.CreateIterator(); It; ++It)
		{
			if (InShouldUnregisterElement(It->Value))
			{
				InOnUnregisterElement(MoveTemp(It->Value));
				It.RemoveCurrent();
			}
		}
	}

	/**
	 * Find the given element owner, or call the register callback to attempt to create one if none is found.
	 * @note This must be paired with a call to UnregisterElementOwner if a new element owner is created.
	 */
	TTypedElementOwnerScopedAccess<ElementDataType> FindOrRegisterElementOwner(const KeyDataType& InKey, TFunctionRef<TTypedElementOwner<ElementDataType>()> InCreateElement)
	{
		TTypedElementOwnerScopedAccess<ElementDataType> ScopedAccess(&ElementOwnerMapCS);
		if (const TTypedElementOwner<ElementDataType>* ExistingElement = ElementOwnerMap.Find(InKey))
		{
			ScopedAccess.Private_SetElementOwner(ExistingElement);
		}
		else if (TTypedElementOwner<ElementDataType> NewElement = InCreateElement())
		{
			ScopedAccess.Private_SetElementOwner(&ElementOwnerMap.Add(InKey, MoveTemp(NewElement)));
		}
		return ScopedAccess;
	}

	/**
	 * Provide const access to the given element owner.
	 * @note This will lock the store for writes until the scoped access struct is no longer being used. The returned struct may be unset if the key was not registered.
	 */
	TTypedElementOwnerScopedAccess<ElementDataType> FindElementOwner(const KeyDataType& InKey) const
	{
		TTypedElementOwnerScopedAccess<ElementDataType> ScopedAccess(&ElementOwnerMapCS);
		ScopedAccess.Private_SetElementOwner(ElementOwnerMap.Find(InKey));
		return ScopedAccess;
	}

	/**
	 * Provide mutable access to the given element owner.
	 * @note This will lock the store for writes until the scoped access struct is no longer being used. The returned struct may be unset if the key was not registered.
	 */
	TTypedElementOwnerScopedMutableAccess<ElementDataType> FindMutableElementOwner(const KeyDataType& InKey)
	{
		TTypedElementOwnerScopedMutableAccess<ElementDataType> ScopedAccess(&ElementOwnerMapCS);
		ScopedAccess.Private_SetElementOwner(ElementOwnerMap.Find(InKey));
		return ScopedAccess;
	}

	/**
	 * Test to see whether the store has an entry for the given element owner.
	 */
	bool ContainsElementOwner(const KeyDataType& InKey) const
	{
		UE::TScopeLock Lock(ElementOwnerMapCS);
		return ElementOwnerMap.Contains(InKey);
	}

private:
	TMap<KeyDataType, TTypedElementOwner<ElementDataType>> ElementOwnerMap;
	mutable FTransactionallySafeCriticalSection ElementOwnerMapCS;
};
