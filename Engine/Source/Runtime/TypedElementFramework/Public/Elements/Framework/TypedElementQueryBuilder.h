// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM/Defines.h"
#include "DataStorage/CommonTypes.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Templates/SubclassOf.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UScriptStruct;
class USubsystem;

/**
 * The TypedElementQueryBuilder allows for the construction of queries for use by the Typed Element Data Storage.
 * There are two types of queries, simple and normal. Simple queries are guaranteed to be supported by the data
 * storage backend and guaranteed to have no performance side effects. <Normal queries pending development.>
 * 
 * Queries are constructed with the following section:
 * - Select		A list of the data objects that are returned as the result of the query.
 * - Count		Counts the total number or rows that pass the filter.
 * - Where		A list of conditions that restrict what's accepted by the query.
 * - DependsOn	A list of systems outside the data storage that will be accessed by the query('s user).
 * - Compile	Compiles the query into its final form and can be used afterwards.
 * 
 * Calls to the sections become increasingly restrictive, e.g. after calling Where only DependsOn can be
 * called again.
 * 
 * Arguments to the various functions take a pointer to a description of a UStruct. These can be provided
 * in the follow ways:
 * - By using the templated version, e.g. Any<FStructExample>()
 * - By calling the static StaticStruct() function on the UStruct, e.g. FStructExample::StaticStruct();
 * - By name using the Type or TypeOptional string operator, e.g. "/Script/ExamplePackage.FStructExample"_Type or
 *		"/Script/OptionalPackage.FStructOptional"_TypeOptional
 * All functions allow for a single type to be added or a list of types, e.g. ReadOnly(Type<FStructExample>() or
 *		ReadOnly({ Type<FStructExample1>(), FStructExample2::StaticStruct(), "/Script/ExamplePackage.FStructExample3"_Type });
 *
 * Some functions allow binding to a callback. In these cases the arguments to the provided callback are analyzed and
 * added to the query automatically. Const arguments are added as ReadOnly, while non-const arguments are added as 
 * ReadWrite. Callbacks can be periodically called if constructed as a processor, in which case the callback is triggered
 * repeatedly, usually once per frame and called for all row (ranges) that match the query. If constructed as an observer
 * the provided target type is monitored for actions like addition or deletion into/from any table and will trigger the
 * callback once if the query matches. The following function signatures are accepted by "Select":
 *	- void([const]Column&...) 
 *	- void([const]Column*...) 
 *	- void(RowHandle, [const]Column&...) 
 *	- void(<Context>&, [const]Column&...) 
 *	- void(<Context>&, RowHandle, [const]Column&...) 
 *	- void(<Context>&, [const]Column*...) 
 *	- void(<Context>&, const RowHandle*, [const]Column*...) 
 *	Where <Context> is IQueryContext or FCachedQueryContext<...>	e.g.:
 *		void(
 *			FCachedQueryContext<Subsystem1, const Subsystem2>& Context, 
 *			RowHandle Row, 
 *			ColumnType0& ColumnA, 
 *			const ColumnType1& ColumnB) 
 *			{...}
 *
 * FCachedQueryContext can be used to store cached pointers to dependencies to reduce the overhead of retrieving these. The same
 * const principle as for other arguments applies so dependencies marked as const can only be accessed as read-only or otherwise
 * can be accessed as readwrite.
 *
 * The following is a simplified example of these options combined together:
 *		FProcessor Info(
 *			EQueryTickPhase::FrameEnd, 
 *			DataStorage->GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage);
 *		Query = Select(FName("Example Callback"), Info, 
 *				[](FCachedQueryContext<Subsystem1, const Subsystem2>&, const FDataExample1&, FDataExample2&) {});
 * 
 * "Select" is constructed with: 
 * - ReadOnly: Indicates that the data object will only be read from
 * - ReadWrite: Indicated that the data object will be read and written to.
 * 
 * "Count" does not have any construction options.
 * 
 * "Where" is constructed with:
 * - All: The query will be accepted only if all the types listed here are present in a table.
 * - Any: The query will be accepted if at least one of the listed types is present in a table.
 * - Not: The query will be accepted if none of the listed types are present in a table.
 * The above construction calls can be mixed and be called multiple times.
 * All functions accept a nullptr for the type in which case the call will have no effect. This can be used to
 *		reference types in plugins that may not be loaded when using the TypeOptional string operator.

 * "DependsOn" is constructed with:
 * - ReadOnly: Indicates that the external system will only be used to read data from.
 * - ReadWrite: Indicates that the external system will be used to write data to.
 *
 * Usage example:
 * FQueryDescription Query =
 *		Select()
 *			.ReadWrite({ FDataExample1::StaticStruct() })
 *			.ReadWrite<FDataExample2, FDataExample3>()
 *			.ReadOnly<FDataExample4>()
 *		.Where()
 *			.All<FTagExample1, FDataExample5>()
 *			.Any("/Script/ExamplePackage.FStructExample"_TypeOptional)
 *			.None(FTagExample2::StaticStruct())
 *		.DependsOn()
 *			.ReadOnly<USystemExample1, USystemExample2>()
 *			.ReadWrite(USystemExample2::StaticClass())
 *		.Compile();
 *
 * Creating a query is expensive on the builder and the back-end side. It's therefore recommended to create a query
 * and store its compiled form for repeated use instead of rebuilding the query on every update.
 */

namespace UE::Editor::DataStorage::Queries
{
	TYPEDELEMENTFRAMEWORK_API const UScriptStruct* Type(FTopLevelAssetPath Name);
	TYPEDELEMENTFRAMEWORK_API const UScriptStruct* TypeOptional(FTopLevelAssetPath Name);
	TYPEDELEMENTFRAMEWORK_API const UScriptStruct* operator""_Type(const char* Name, std::size_t NameSize);
	TYPEDELEMENTFRAMEWORK_API const UScriptStruct* operator""_TypeOptional(const char* Name, std::size_t NameSize);

	enum class EOptional
	{
		No,
		Yes
	};

	class FDependency final
	{
		friend class Count;
		friend class Select;
		friend class FSimpleQuery;
		friend class FQueryConditionQuery;
	public:
		template<typename... TargetTypes>
		FDependency& ReadOnly();
		TYPEDELEMENTFRAMEWORK_API FDependency& ReadOnly(const UClass* Target);
		TYPEDELEMENTFRAMEWORK_API FDependency& ReadOnly(TConstArrayView<const UClass*> Targets);
		template<typename... TargetTypes>
		FDependency& ReadWrite();
		TYPEDELEMENTFRAMEWORK_API FDependency& ReadWrite(const UClass* Target);
		TYPEDELEMENTFRAMEWORK_API FDependency& ReadWrite(TConstArrayView<const UClass*> Targets);

		TYPEDELEMENTFRAMEWORK_API FDependency& SubQuery(QueryHandle Handle);
		TYPEDELEMENTFRAMEWORK_API FDependency& SubQuery(TConstArrayView<QueryHandle> Handles);

		TYPEDELEMENTFRAMEWORK_API FQueryDescription&& Compile();

	private:
		TYPEDELEMENTFRAMEWORK_API explicit FDependency(FQueryDescription* Query);

		FQueryDescription* Query;
	};

	// ============================================================================
	// Relation Predicate Factory Functions
	// ============================================================================

	TYPEDELEMENTFRAMEWORK_API FQueryDescription::FRelationPredicate IsSubjectOf(RelationTypeHandle RelationType);
	TYPEDELEMENTFRAMEWORK_API FQueryDescription::FRelationPredicate IsSubjectOf(RelationTypeHandle RelationType, RowHandle Object);
	TYPEDELEMENTFRAMEWORK_API FQueryDescription::FRelationPredicate IsObjectOf(RelationTypeHandle RelationType);
	TYPEDELEMENTFRAMEWORK_API FQueryDescription::FRelationPredicate IsObjectOf(RelationTypeHandle RelationType, RowHandle Subject);
	TYPEDELEMENTFRAMEWORK_API FQueryDescription::FRelationPredicate IsDescendantOf(RelationTypeHandle RelationType, RowHandle Ancestor, bool bIncludeSelf = false);
	TYPEDELEMENTFRAMEWORK_API FQueryDescription::FRelationPredicate IsAncestorOf(RelationTypeHandle RelationType, RowHandle Descendant, bool bIncludeSelf = false);

	// ============================================================================
	// FQueryExpression — composite expression of column conditions and relation predicates.
	//
	// Build one with operator&& between:
	//   - TColumn<T>() or FConditions  (column conditions, evaluated at archetype level)
	//   - IsDescendantOf / IsSubjectOf / etc. free functions (relation predicates, per-entity)
	// All conditions are AND-composed.
	//
	// Example:
	//   using namespace UE::Editor::DataStorage::Queries;
	//   Select(...).Where(TColumn<FMyColumn>() && IsDescendantOf(MyRelation, RootRow)).Compile()
	// ============================================================================

	/**
	 * Composite query expression holding both column conditions (evaluated at archetype selection time
	 * via SetArchetypeMatchOverride) and relation predicates (evaluated per-entity after chunk selection).
	 * Column and relation conditions are AND-composed with each other.
	 */
	struct FQueryExpression
	{
		/** Column conditions. Evaluated at archetype level. Absent when only relation predicates are combined. */
		TOptional<FConditions> ColumnConditions;
		/** Relation predicates AND-composed with each other and with ColumnConditions. */
		TArray<FQueryDescription::FRelationPredicate, TInlineAllocator<2>> RelationAtoms;
	};

	// ---- Produce FQueryExpression: column atom + relation predicate ----
	TYPEDELEMENTFRAMEWORK_API FQueryExpression operator&&(FColumnBase Lhs, const FQueryDescription::FRelationPredicate& Rhs);
	TYPEDELEMENTFRAMEWORK_API FQueryExpression operator&&(const FQueryDescription::FRelationPredicate& Lhs, FColumnBase Rhs);
	// ---- Produce FQueryExpression: FConditions + relation predicate ----
	TYPEDELEMENTFRAMEWORK_API FQueryExpression operator&&(const FConditions& Lhs, const FQueryDescription::FRelationPredicate& Rhs);
	TYPEDELEMENTFRAMEWORK_API FQueryExpression operator&&(const FQueryDescription::FRelationPredicate& Lhs, const FConditions& Rhs);
	// ---- Produce FQueryExpression: relation predicate + relation predicate ----
	TYPEDELEMENTFRAMEWORK_API FQueryExpression operator&&(const FQueryDescription::FRelationPredicate& Lhs, const FQueryDescription::FRelationPredicate& Rhs);
	// ---- Extend an existing FQueryExpression ----
	TYPEDELEMENTFRAMEWORK_API FQueryExpression operator&&(FQueryExpression Lhs, const FQueryDescription::FRelationPredicate& Rhs);
	TYPEDELEMENTFRAMEWORK_API FQueryExpression operator&&(FQueryExpression Lhs, FColumnBase Rhs);
	TYPEDELEMENTFRAMEWORK_API FQueryExpression operator&&(FQueryExpression Lhs, const FConditions& Rhs);

	class FSimpleQuery final
	{
	public:
		friend class Count;
		friend class Select;

		TYPEDELEMENTFRAMEWORK_API FDependency DependsOn();
		TYPEDELEMENTFRAMEWORK_API FQueryDescription&& Compile();

		template<TColumnType... TargetTypes>
		FSimpleQuery& All();
		TYPEDELEMENTFRAMEWORK_API FSimpleQuery& All(const UScriptStruct* Target);
		TYPEDELEMENTFRAMEWORK_API FSimpleQuery& All(TConstArrayView<const UScriptStruct*> Targets);
		
		template<TColumnType... TargetTypes>
		FSimpleQuery& Any();
		TYPEDELEMENTFRAMEWORK_API FSimpleQuery& Any(const UScriptStruct* Target);
		TYPEDELEMENTFRAMEWORK_API FSimpleQuery& Any(TConstArrayView<const UScriptStruct*> Targets);

		// Dynamic Column Support
		TYPEDELEMENTFRAMEWORK_API FSimpleQuery& Any(const FDynamicColumnDescription& Description);
		template<TDynamicColumnTemplate T>
		FSimpleQuery& Any(const FName&);

		// ValueTags not yet supported for Any
		template<TValueTagType>
		FSimpleQuery& Any(const FName&) = delete;
		template<TValueTagType>
		FSimpleQuery& Any(const FName&, const FName&) = delete;
		
		template<TColumnType... TargetTypes>
		FSimpleQuery& None();
		
		TYPEDELEMENTFRAMEWORK_API FSimpleQuery& None(const UScriptStruct* Target);
		TYPEDELEMENTFRAMEWORK_API FSimpleQuery& None(TConstArrayView<const UScriptStruct*> Targets);
		
		// ValueTags not yet supported for None
		template<TValueTagType>
		FSimpleQuery& None(const FName& Tag) = delete;
		template<TValueTagType>
		FSimpleQuery& None(const FName& Tag, const FName& Value) = delete;

		// Dynamic Column Support
		TYPEDELEMENTFRAMEWORK_API FSimpleQuery& None(const FDynamicColumnDescription& Description);
		template<TDynamicColumnTemplate T>
		FSimpleQuery& None(const FName&);
		
		// Value Tags
		// ============
		// Adds a filter to the query which must match a given ValueTag.  The value of the ValueTag will not be checked.
		TYPEDELEMENTFRAMEWORK_API FSimpleQuery& All(const FValueTag& Tag);
		// Adds a filter to the query which must match a given ValueTag.  The value of the ValueTag must also match.
		// Note: The query can only match a single value.  Multiple value queries are not supported at this time.
		TYPEDELEMENTFRAMEWORK_API FSimpleQuery& All(const FValueTag& Tag, const FName& Value);
		
		TYPEDELEMENTFRAMEWORK_API FSimpleQuery& All(const UEnum& Enum);
		TYPEDELEMENTFRAMEWORK_API FSimpleQuery& All(const UEnum& Enum, int64 Value);

		// Dynamic Column support
		TYPEDELEMENTFRAMEWORK_API FSimpleQuery& All(const FDynamicColumnDescription& Description);
		template<TDynamicColumnTemplate T>
		FSimpleQuery& All(const FName&);

		// Adds a filter to the query which must match a given ValueTag.  The value of the DynamicTag will not be checked.
		template<TValueTagType>
		FSimpleQuery& All(const FName& Tag);
		// Adds a filter to the query which must match a given ValueTag.  The value of the DynamicTag must also match.
		// Note: The query can only match a single value.  Multiple value queries are not supported at this time.
		template<TValueTagType>
		FSimpleQuery& All(const FName& Tag, const FName& Value);
		
		template<TEnumType EnumT>
		FSimpleQuery& All();
		
		template<TEnumType EnumT>
		FSimpleQuery& All(EnumT EnumValue);

		template<auto Value, TEnumType EnumT = decltype(Value)>
		FSimpleQuery& All();

		// ============================================================================
		// Relation Conditions
		// ============================================================================

		/**
		 * Filter to rows that are subjects in relations of the given type.
		 * E.g., IsSubjectOf(SubclassOfType) finds all rows that inherit from something.
		 */
		TYPEDELEMENTFRAMEWORK_API FSimpleQuery& IsSubjectOf(RelationTypeHandle RelationType);

		/**
		 * Filter to rows that are subjects in relations of the given type where the object
		 * is the specified row. E.g., IsSubjectOf(SubclassOfType, UObject) finds direct subclasses of UObject.
		 */
		TYPEDELEMENTFRAMEWORK_API FSimpleQuery& IsSubjectOf(RelationTypeHandle RelationType, RowHandle Object);

		/**
		 * Filter to rows that are objects in relations of the given type.
		 * E.g., IsObjectOf(SubclassOfType) finds all rows that have something inheriting from them.
		 */
		TYPEDELEMENTFRAMEWORK_API FSimpleQuery& IsObjectOf(RelationTypeHandle RelationType);

		/**
		 * Filter to rows that are objects in relations of the given type where the subject
		 * is the specified row. E.g., IsObjectOf(SubclassOfType, UDerivedClass) finds UDerivedClass's parent.
		 */
		TYPEDELEMENTFRAMEWORK_API FSimpleQuery& IsObjectOf(RelationTypeHandle RelationType, RowHandle Subject);

		// ============================================================================
		// Hierarchy Conditions (for hierarchical relation types)
		// ============================================================================

		/**
		 * Filter to rows that are descendants of the specified ancestor in the given hierarchical relation.
		 * Uses interval encoding for O(1) evaluation per row.
		 * @param RelationType Must be a hierarchical relation type (HierarchyMode != EHierarchyMode::Disabled)
		 * @param Ancestor The ancestor row to test against
		 * @param bIncludeSelf If true, the ancestor itself will also match
		 */
		TYPEDELEMENTFRAMEWORK_API FSimpleQuery& IsDescendantOf(RelationTypeHandle RelationType, RowHandle Ancestor, bool bIncludeSelf = false);

		/**
		 * Filter to rows that are ancestors of the specified descendant in the given hierarchical relation.
		 * @param RelationType Must be a hierarchical relation type (HierarchyMode != EHierarchyMode::Disabled)
		 * @param Descendant The descendant row to test against
		 * @param bIncludeSelf If true, the descendant itself will also match
		 */
		TYPEDELEMENTFRAMEWORK_API FSimpleQuery& IsAncestorOf(RelationTypeHandle RelationType, RowHandle Descendant, bool bIncludeSelf = false);

	private:
		TYPEDELEMENTFRAMEWORK_API explicit FSimpleQuery(FQueryDescription* Query);

		FQueryDescription* Query;
	};

	struct FQueryCallbackType{};

	struct FProcessor final : public FQueryCallbackType
	{
		TYPEDELEMENTFRAMEWORK_API FProcessor(EQueryTickPhase Phase, FName Group);
		TYPEDELEMENTFRAMEWORK_API FProcessor& SetPhase(EQueryTickPhase NewPhase);
		TYPEDELEMENTFRAMEWORK_API FProcessor& SetGroup(FName GroupName);
		TYPEDELEMENTFRAMEWORK_API FProcessor& SetBeforeGroup(FName GroupName);
		TYPEDELEMENTFRAMEWORK_API FProcessor& SetAfterGroup(FName GroupName);
		TYPEDELEMENTFRAMEWORK_API FProcessor& SetExecutionMode(EExecutionMode Mode);
		TYPEDELEMENTFRAMEWORK_API FProcessor& MakeActivatable(FName Name);
		TYPEDELEMENTFRAMEWORK_API FProcessor& BatchModifications(bool bBatch);

		EQueryTickPhase Phase;
		FName Group;
		FName BeforeGroup;
		FName AfterGroup;
		FName ActivationName;
		EExecutionMode ExecutionMode = EExecutionMode::Default;
		bool bBatchModifications = false;
	};

	struct FObserver final : public FQueryCallbackType
	{
		enum class EEvent : uint8
		{
			Add,
			Remove
		};

		TYPEDELEMENTFRAMEWORK_API FObserver(EEvent MonitorForEvent, const UScriptStruct* MonitoredColumn);

		template<TColumnType ColumnType>
		static FObserver OnAdd();
		template<TColumnType ColumnType>
		static FObserver OnRemove();

		TYPEDELEMENTFRAMEWORK_API FObserver& SetEvent(EEvent MonitorForEvent);
		TYPEDELEMENTFRAMEWORK_API FObserver& SetMonitoredColumn(const UScriptStruct* MonitoredColumn);
		template<TColumnType ColumnType>
		FObserver& SetMonitoredColumn();
		TYPEDELEMENTFRAMEWORK_API FObserver& SetExecutionMode(EExecutionMode Mode);
		TYPEDELEMENTFRAMEWORK_API FObserver& MakeActivatable(FName Name);

		const UScriptStruct* Monitor;
		EEvent Event;
		FName ActivationName;
		EExecutionMode ExecutionMode = EExecutionMode::Default;
	};

	struct FPhaseAmble final : public FQueryCallbackType
	{
		enum class ELocation : uint8
		{
			Preamble,
			Postamble
		};

		TYPEDELEMENTFRAMEWORK_API FPhaseAmble(ELocation InLocation, EQueryTickPhase InPhase);
		TYPEDELEMENTFRAMEWORK_API FPhaseAmble& SetLocation(ELocation NewLocation);
		TYPEDELEMENTFRAMEWORK_API FPhaseAmble& SetPhase(EQueryTickPhase NewPhase);
		TYPEDELEMENTFRAMEWORK_API FPhaseAmble& SetExecutionMode(EExecutionMode Mode);
		TYPEDELEMENTFRAMEWORK_API FPhaseAmble& MakeActivatable(FName Name);

		EQueryTickPhase Phase;
		ELocation Location;
		FName ActivationName;
		EExecutionMode ExecutionMode = EExecutionMode::Default;
	};

	// Because this is a thin wrapper called from within a query callback, it's better to inline fully so all
	// function pre/postambles can be optimized away.
	struct FQueryContextForwarder : public IQueryContext
	{
		inline FQueryContextForwarder(
			const FQueryDescription& InDescription, 
			IQueryContext& InParentContext);
		inline ~FQueryContextForwarder() = default;

		inline const void* GetColumn(const UScriptStruct* ColumnType) const override;
		inline void* GetMutableColumn(const UScriptStruct* ColumnType) override;
		inline void GetColumns(TArrayView<char*> RetrievedAddresses, TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ColumnTypes,
			TConstArrayView<EQueryAccessType> AccessTypes) override;
		inline void GetColumnsUnguarded(int32 TypeCount, char** RetrievedAddresses, const TWeakObjectPtr<const UScriptStruct>* ColumnTypes,
			const EQueryAccessType* AccessTypes) override;

		inline bool HasColumn(const UScriptStruct* ColumnType) const override;
		
		inline UObject* GetMutableDependency(const UClass* DependencyClass) override;
		inline const UObject* GetDependency(const UClass* DependencyClass) override;
		inline void GetDependencies(TArrayView<UObject*> RetrievedAddresses, TConstArrayView<TWeakObjectPtr<const UClass>> SubsystemTypes,
			TConstArrayView<EQueryAccessType> AccessTypes) override;

		inline uint32 GetRowCount() const override;
		inline TConstArrayView<RowHandle> GetRowHandles() const override;
		inline void RemoveRow(RowHandle Row) override;
		inline void RemoveRows(TConstArrayView<RowHandle> Rows) override;

		inline void AddColumns(RowHandle Row, TConstArrayView<const UScriptStruct*> ColumnTypes) override;
		inline void AddColumns(TConstArrayView<RowHandle> Rows, TConstArrayView<const UScriptStruct*> ColumnTypes) override;
		inline void RemoveColumns(RowHandle Row, TConstArrayView<const UScriptStruct*> ColumnTypes) override;
		inline void RemoveColumns(TConstArrayView<RowHandle> Rows, TConstArrayView<const UScriptStruct*> ColumnTypes) override;

		inline void PushCommand(void(* CommandFunction)(void*), void* CommandData) override;

		inline FQueryResult RunQuery(QueryHandle Query) override;
		inline FQueryResult RunSubquery(int32 SubqueryIndex) override;
		inline FQueryResult RunSubquery(int32 SubqueryIndex, SubqueryCallbackRef Callback) override;
		inline FQueryResult RunSubquery(int32 SubqueryIndex, RowHandle Row,
			SubqueryCallbackRef Callback) override;

		IQueryContext& ParentContext;
		const FQueryDescription& Description;
	};

	template<typename... Dependencies>
	struct FCachedQueryContext final : public FQueryContextForwarder
	{
		explicit FCachedQueryContext(
			const FQueryDescription& InDescription, 
			IQueryContext& InParentContext);
		
		static void Register(FQueryDescription& Query);

		template<typename Dependency>
		Dependency& GetCachedMutableDependency();
		template<typename Dependency>
		const Dependency& GetCachedDependency() const;
	};

	class FQueryConditionQuery
	{
	public:
		friend class Count;
		friend class Select;

		TYPEDELEMENTFRAMEWORK_API FDependency DependsOn();
		TYPEDELEMENTFRAMEWORK_API FQueryDescription&& Compile();
	private:
		TYPEDELEMENTFRAMEWORK_API explicit FQueryConditionQuery(FQueryDescription* Query);

		FQueryDescription* Query;
	};

	// Explicitly not following the naming convention in order to present this as a query that can be read as such.
	class Select final
	{
	public:
		TYPEDELEMENTFRAMEWORK_API Select();

		/** Select the columns to operate one, giving read - only or read / write access. */
		template<typename CallbackType, typename Function>
		Select(FName Name, const CallbackType& Type, Function&& Callback);
		/** Select the columns to operate one, giving read - only or read / write access. */
		template<typename CallbackType, typename Class, typename Function>
		Select(FName Name, const CallbackType& Type, Class* Instance, Function&& Callback);

		/** Request read-only access to the listed columns. */
		template<TDataColumnType... TargetTypes>
		Select& ReadOnly();
		/** Request read-only access to the listed columns. */
		TYPEDELEMENTFRAMEWORK_API Select& ReadOnly(const UScriptStruct* Target);
		/** Request read-only access to the listed columns. */
		TYPEDELEMENTFRAMEWORK_API Select& ReadOnly(TConstArrayView<const UScriptStruct*> Targets);
		TYPEDELEMENTFRAMEWORK_API Select& ReadOnly(const FDynamicColumnDescription& Description);
		/** 
		 * Request read-only access to the listed columns. If optional is true read access will be given if the column is in the table but
		 * it will not be used for finding matching tables. Columns bound with optional can not be bound to a query callback argument.
		 */
		template<TDataColumnType... TargetTypes>
		Select& ReadOnly(EOptional Optional);
		/**
		 * Request read-only access to the listed columns. If optional is true read access will be given if the column is in the table but
		 * it will not be used for finding matching tables. Columns bound with optional can not be bound to a query callback argument.
		 */
		TYPEDELEMENTFRAMEWORK_API Select& ReadOnly(const UScriptStruct* Target, EOptional Optional);
		/**
		 * Request read-only access to the listed columns. If optional is true read access will be given if the column is in the table but
		 * it will not be used for finding matching tables. Columns bound with optional can not be bound to a query callback argument.
		 */
		TYPEDELEMENTFRAMEWORK_API Select& ReadOnly(TConstArrayView<const UScriptStruct*> Targets, EOptional Optional);
		template<TDynamicColumnTemplate Target>
		Select& ReadOnly(const FName& Identifier);
		template<TDynamicColumnTemplate Target>
		Select& ReadOnly();
		/** Request read and write access to the listed columns. */
		template<TDataColumnType... TargetTypes>
		Select& ReadWrite();
		/** Request read and write access to the listed columns. */
		TYPEDELEMENTFRAMEWORK_API Select& ReadWrite(const UScriptStruct* Target);
		/** Request read and write access to the listed columns. */
		TYPEDELEMENTFRAMEWORK_API Select& ReadWrite(TConstArrayView<const UScriptStruct*> Targets);
		TYPEDELEMENTFRAMEWORK_API Select& ReadWrite(const FDynamicColumnDescription& Description);
		template<TDynamicColumnTemplate Target>
		Select& ReadWrite(const FName& Identifier);
		template<TDynamicColumnTemplate Target>
		Select& ReadWrite();

		// Query Condition API
		TYPEDELEMENTFRAMEWORK_API FQueryConditionQuery Where(Queries::FConditions Condition);
		TYPEDELEMENTFRAMEWORK_API FQueryConditionQuery Where(Queries::FQueryExpression Expression);

		TYPEDELEMENTFRAMEWORK_API FQueryDescription&& Compile();
		TYPEDELEMENTFRAMEWORK_API FSimpleQuery Where();
		TYPEDELEMENTFRAMEWORK_API FDependency DependsOn();
		TYPEDELEMENTFRAMEWORK_API Select& AccessesHierarchy(const FName& HierarchyName);

	private:
		FQueryDescription Query;
	};

	// Explicitly not following the naming convention in order to keep readability consistent. It now reads like a query sentence.
	class Count final
	{
	public:
		TYPEDELEMENTFRAMEWORK_API Count();

		TYPEDELEMENTFRAMEWORK_API FSimpleQuery Where();
		TYPEDELEMENTFRAMEWORK_API FDependency DependsOn();

	private:
		FQueryDescription Query;
	};

	template<typename Function>
	AUTORTFM_INFER DirectQueryCallback CreateDirectQueryCallbackBinding(Function&& Callback);
	template<typename Function>
	AUTORTFM_INFER SubqueryCallback CreateSubqueryCallbackBinding(Function&& Callback);

	// Merge the two input queries by copying the source query into the destination query. If the merge is not possible the original data will be
	// preserved and an optional error message can be requested
	TYPEDELEMENTFRAMEWORK_API bool MergeQueries(FQueryDescription& Destination, const FQueryDescription& Source, FText* OutErrorMessage = nullptr);

} // namespace UE::Editor::DataStorage::Queries

#include "Elements/Framework/TypedElementQueryBuilder.inl"
