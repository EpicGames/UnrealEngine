// Copyright Epic Games, Inc. All Rights Reserved.

#include <tuple>
#include <type_traits>
#include <utility>

#include "AutoRTFM/Defines.h"
#include "Templates/UnrealTypeTraits.h"
#include "DataStorage/Queries/Description.h"
#include "DataStorage/Queries/Types.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Interfaces/TypedElementQueryStorageInterfaces.h"

namespace UE::Editor::DataStorage::Queries
{
	using namespace UE::Editor::DataStorage;

	namespace Private
	{
		// This assumes that the types are unique, but for queries this should be true and otherwise
		// both results would point to the first found index.
		template<typename Target, typename ArgsCurrent, typename... ArgsRemainder>
		constexpr uint32 GetVarArgIndex()
		{
			if constexpr (std::is_same_v<Target, ArgsCurrent>)
			{
				return 0;
			}
			else
			{
				return 1 + GetVarArgIndex<Target, ArgsRemainder...>();
			}
		}

		template<typename Type>
		constexpr EQueryAccessType GetAccessType()
		{
			using BaseType = typename std::remove_reference_t<Type>;
			if constexpr (std::is_const_v<BaseType>)
			{
				return EQueryAccessType::ReadOnly;
			}
			else
			{
				return EQueryAccessType::ReadWrite;
			}
		}

		template<typename Type>
		constexpr EQueryDependencyFlags GetDependencyFlags()
		{
			using BaseType = typename std::remove_reference_t<Type>;
			using SubsystemTraits = TTypedElementSubsystemTraits<std::remove_const_t<BaseType>>;

			EQueryDependencyFlags Result = EQueryDependencyFlags::None;
			if constexpr (SubsystemTraits::RequiresGameThread())
			{
				EnumAddFlags(Result, EQueryDependencyFlags::GameThreadBound);
			}
			if constexpr (SubsystemTraits::IsHotReloadable())
			{
				EnumAddFlags(Result, EQueryDependencyFlags::AlwaysRefresh);
			}
			if constexpr (std::is_const_v<BaseType>)
			{
				EnumAddFlags(Result, EQueryDependencyFlags::ReadOnly);
			}
			
			return Result;
		}
	} // namespace Private

	//
	// FDependecy
	//

	template<typename... TargetTypes>
	FDependency& FDependency::ReadOnly()
	{
		ReadOnly({ TargetTypes::StaticClass()... });
		return *this;
	}

	template<typename... TargetTypes>
	FDependency& FDependency::ReadWrite()
	{
		ReadWrite({ TargetTypes::StaticClass()... });
		return *this;
	}

	
	//
	// FObserver
	//
	
	template<TColumnType ColumnType>
	FObserver FObserver::OnAdd()
	{
		return FObserver(FObserver::EEvent::Add, ColumnType::StaticStruct());
	}

	template<TColumnType ColumnType>
	FObserver FObserver::OnRemove()
	{
		return FObserver(FObserver::EEvent::Remove, ColumnType::StaticStruct());
	}

	template<TColumnType ColumnType>
	FObserver& FObserver::SetMonitoredColumn()
	{
		return SetMonitoredColumn(ColumnType::StaticStruct());
	}


	//
	// FQueryContextForwarder
	//

	FQueryContextForwarder::FQueryContextForwarder(
		const FQueryDescription& InDescription,
		IQueryContext& InParentContext)
		: ParentContext(InParentContext)
		, Description(InDescription)
	{}

	const void* FQueryContextForwarder::GetColumn(const UScriptStruct* ColumnType) const
	{
		return ParentContext.GetColumn(ColumnType);
	}

	void* FQueryContextForwarder::GetMutableColumn(const UScriptStruct* ColumnType)
	{
		return ParentContext.GetMutableColumn(ColumnType);
	}

	void FQueryContextForwarder::GetColumns(TArrayView<char*> RetrievedAddresses, TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ColumnTypes,
		TConstArrayView<EQueryAccessType> AccessTypes)
	{
		ParentContext.GetColumns(RetrievedAddresses, ColumnTypes, AccessTypes);
	}

	void FQueryContextForwarder::GetColumnsUnguarded(int32 TypeCount, char** RetrievedAddresses, const TWeakObjectPtr<const UScriptStruct>* ColumnTypes,
		const EQueryAccessType* AccessTypes)
	{
		ParentContext.GetColumnsUnguarded(TypeCount, RetrievedAddresses, ColumnTypes, AccessTypes);
	}
	
	bool FQueryContextForwarder::HasColumn(const UScriptStruct* ColumnType) const
	{
		return ParentContext.HasColumn(ColumnType);
	}

	UObject* FQueryContextForwarder::GetMutableDependency(const UClass* DependencyClass)
	{
		return ParentContext.GetMutableDependency(DependencyClass);
	}

	const UObject* FQueryContextForwarder::GetDependency(const UClass* DependencyClass)
	{
		return ParentContext.GetDependency(DependencyClass);
	}

	void FQueryContextForwarder::GetDependencies(TArrayView<UObject*> RetrievedAddresses, TConstArrayView<TWeakObjectPtr<const UClass>> DependencyTypes,
		TConstArrayView<EQueryAccessType> AccessTypes)
	{
		ParentContext.GetDependencies(RetrievedAddresses, DependencyTypes, AccessTypes);
	}

	uint32 FQueryContextForwarder::GetRowCount() const
	{
		return ParentContext.GetRowCount();
	}

	TConstArrayView<RowHandle> FQueryContextForwarder::GetRowHandles() const
	{
		return ParentContext.GetRowHandles();
	}

	void FQueryContextForwarder::RemoveRow(RowHandle Row)
	{
		ParentContext.RemoveRow(Row);
	}

	void FQueryContextForwarder::RemoveRows(TConstArrayView<RowHandle> Rows)
	{
		ParentContext.RemoveRows(Rows);
	}

	void FQueryContextForwarder::AddColumns(RowHandle Row, TConstArrayView<const UScriptStruct*> ColumnTypes)
	{
		ParentContext.AddColumns(Row, ColumnTypes);
	}

	void FQueryContextForwarder::AddColumns(TConstArrayView<RowHandle> Rows, TConstArrayView<const UScriptStruct*> ColumnTypes)
	{
		ParentContext.AddColumns(Rows, ColumnTypes);
	}

	void FQueryContextForwarder::RemoveColumns(RowHandle Row, TConstArrayView<const UScriptStruct*> ColumnTypes)
	{
		ParentContext.RemoveColumns(Row, ColumnTypes);
	}

	void FQueryContextForwarder::RemoveColumns(TConstArrayView<RowHandle> Rows, TConstArrayView<const UScriptStruct*> ColumnTypes)
	{
		ParentContext.RemoveColumns(Rows, ColumnTypes);
	}

	FQueryResult FQueryContextForwarder::RunQuery(QueryHandle Query)
	{
		return ParentContext.RunQuery(Query);
	}
	
	FQueryResult FQueryContextForwarder::RunSubquery(int32 SubqueryIndex)
	{
		return ParentContext.RunSubquery(SubqueryIndex);
	}

	FQueryResult FQueryContextForwarder::RunSubquery(int32 SubqueryIndex,
		SubqueryCallbackRef Callback)
	{
		return ParentContext.RunSubquery(SubqueryIndex, Callback);
	}

	FQueryResult FQueryContextForwarder::RunSubquery(int32 SubqueryIndex,
		RowHandle Row, SubqueryCallbackRef Callback)
	{
		return ParentContext.RunSubquery(SubqueryIndex, Row, Callback);
	}

	void FQueryContextForwarder::PushCommand(void(* CommandFunction)(void*), void* CommandData)
	{
		return ParentContext.PushCommand(CommandFunction, CommandData);
	}

	//
	// FCachedQueryContext
	//
	
	template<typename... Dependencies>
	FCachedQueryContext<Dependencies...>::FCachedQueryContext(
		const FQueryDescription& InDescription,
		IQueryContext& InParentContext)
		: FQueryContextForwarder(InDescription, InParentContext)
	{}

	template<typename... Dependencies>
	void FCachedQueryContext<Dependencies...>::Register(FQueryDescription& Query)
	{
		Query.DependencyTypes.Reserve(sizeof...(Dependencies));
		Query.DependencyFlags.Reserve(sizeof...(Dependencies));
		
		( Query.DependencyTypes.Emplace(Dependencies::StaticClass()), ... );
		( Query.DependencyFlags.Emplace(Private::GetDependencyFlags<Dependencies>()), ... );
		
		Query.CachedDependencies.AddDefaulted(sizeof...(Dependencies));
	}

	template<typename... Dependencies>
	template<typename Dependency>
	Dependency& FCachedQueryContext<Dependencies...>::GetCachedMutableDependency()
	{
		// Don't allow a dependency registered as const to be found.
		constexpr uint32 Index = Private::GetVarArgIndex<std::remove_const_t<Dependency>, Dependencies...>();
		static_assert(Index < sizeof...(Dependencies), "Requested dependency isn't part of the query context cache.");
		return *static_cast<Dependency*>(Description.CachedDependencies[Index].Get());
	}

	template<typename... Dependencies>
	template<typename Dependency>
	const Dependency& FCachedQueryContext<Dependencies...>::GetCachedDependency() const
	{
		// Allow access to dependencies registered with and without const.
		constexpr uint32 Index = Private::GetVarArgIndex<Dependency, Dependencies...>();
		if constexpr (Index < sizeof...(Dependencies))
		{
			return *static_cast<Dependency*>(Description.CachedDependencies[Index].Get());
		}
		else
		{
			constexpr uint32 ConstIndex = Private::GetVarArgIndex<std::add_const_t<Dependency>, Dependencies...>();
			static_assert(ConstIndex < sizeof...(Dependencies), "Requested dependency isn't part of the query context cache.");
			return *static_cast<Dependency*>(Description.CachedDependencies[ConstIndex].Get());
		}
	}
	
	//
	// Select
	//
	namespace Private
	{
		template<typename T>
		concept HasStaticStructMethod = requires
		{
			{ T::StaticStruct() } -> UE::same_as<UScriptStruct*>;
		};

		template<typename Context>
		concept SourceQueryContext = requires(Context& Ctx, int32 TypeCount, char** RetrievedAddresses,
			const TWeakObjectPtr<const UScriptStruct>*ColumnTypes, const EQueryAccessType * AccessTypes)
		{
			{ Ctx.GetColumnsUnguarded(TypeCount, RetrievedAddresses, ColumnTypes, AccessTypes) };
			{ Ctx.GetRowHandles() } -> UE::convertible_to<TConstArrayView<RowHandle>>;
			{ Ctx.GetRowCount() } -> UE::convertible_to<uint32>;
		};

		template<typename Source, typename Target>
		concept IsCompatibleTargetContextType = 
			SourceQueryContext<Source> && 
			UE::derived_from<std::remove_const_t<std::remove_reference_t<Target>>, Source>;

		template<typename... Columns>
		constexpr bool AreAllColumnsReferences()
		{
			if constexpr (sizeof...(Columns) > 0)
			{
				return (std::is_reference_v<Columns> && ...);
			}
			else
			{
				return true;
			}
		}

		template<typename... Columns>
		constexpr bool AreAllColumnsPointers()
		{
			if constexpr (sizeof...(Columns) > 0)
			{
				return (std::is_pointer_v<Columns> && ...);
			}
			else
			{
				return true;
			}
		}

		template<typename Column>
		using BaseColumnType = std::remove_pointer_t<std::remove_reference_t<Column>>;

		template<typename Column>
		using UndecoratedColumnType = std::remove_cv_t<BaseColumnType<Column>>;

		template<typename Column>
		constexpr bool IsValidColumnType()
		{
			// Do not combine these two into a single statement as that would also allow invalid arguments like "MyColumn*&"
			if constexpr (std::is_reference_v<Column>)
			{
				using BaseType = typename std::remove_const_t<std::remove_reference_t<Column>>;
				return HasStaticStructMethod<BaseType>;
			}
			else if constexpr (std::is_pointer_v<Column>)
			{
				using BaseType = typename std::remove_const_t<std::remove_pointer_t<Column>>;
				return HasStaticStructMethod<BaseType>;
			}
			else
			{
				return false;
			}
		}

		template<typename Column>
		const UScriptStruct* GetColumnType()
		{
			return UndecoratedColumnType<Column>::StaticStruct();
		}

		template<typename RowType>
		concept IsRowHandleType = std::is_same_v<std::remove_cv_t<std::remove_pointer_t<RowType>>, RowHandle>;

		template<typename RowType, typename... Columns>
		constexpr bool IsRowTypeCompatibleWithColumns()
		{
			if constexpr (std::is_pointer_v<RowType> && std::is_const_v<std::remove_pointer_t<RowType>>)
			{
				return AreAllColumnsPointers<Columns...>();
			}
			else
			{
				return AreAllColumnsReferences<Columns...>();
			}
		}

		template<typename T> concept IsFunctor = requires{ &T::operator(); };

		template<SourceQueryContext SourceContext, bool ValidateColumns, typename... ColumnTypes>
		struct FFunctionColumnInfo
		{
			std::tuple<BaseColumnType<ColumnTypes>*...> Columns;
			static constexpr bool bArePointerColumns = AreAllColumnsPointers<ColumnTypes...>();

			template<typename Column>
			bool CheckValidity(const FQueryDescription& Description)
			{
				const UScriptStruct* ColumnType = Column::StaticStruct();
				if (const TWeakObjectPtr<const UScriptStruct>* Match = Description.SelectionTypes.FindByPredicate(
					[ColumnType](const TWeakObjectPtr<const UScriptStruct>& Entry)
					{
						return Entry == ColumnType;
					}); Match != nullptr)
				{
					size_t Index = Match - Description.SelectionTypes.GetData();
					if (Description.SelectionAccessTypes[Index] == EQueryAccessType::ReadOnly)
					{
						if (std::is_const_v<Column>)
						{
							return true;
						}
						else
						{
							checkf(false, 
								TEXT("Provided query callback couldn't be bound because target read-only column of type '%s' is bound to a mutable reference."),
								*ColumnType->GetName());
							return false;
						}
					}
					else
					{
						return true;
					}
				}
				else
				{
					checkf(false, 
						TEXT("Provided query callback couldn't be bound because target query doesn't contain a column of type '%s'."), 
						*ColumnType->GetName());
					return false;
				}
			}

			FFunctionColumnInfo(const FQueryDescription& Description, SourceContext& Context)
			{
				if constexpr (ValidateColumns)
				{
					if (!(CheckValidity<BaseColumnType<ColumnTypes>>(Description) && ...))
					{
						return;
					}
				}

				char* ColumnAddresses[sizeof...(ColumnTypes)];
				Context.GetColumnsUnguarded(sizeof...(ColumnTypes), ColumnAddresses, 
					Description.SelectionTypes.GetData(), Description.SelectionAccessTypes.GetData());
				std::apply([ColumnAddresses](auto&&... Column)
					{
						int Index = 0;
						((Column = reinterpret_cast<BaseColumnType<ColumnTypes>*>(ColumnAddresses[Index++])), ...);
					}, Columns);
			}
		};

		template<SourceQueryContext SourceContext, bool ValidateColumns>
		struct FFunctionColumnInfo<SourceContext, ValidateColumns>
		{
			FFunctionColumnInfo(const FQueryDescription&, SourceContext&) {}
			static constexpr bool bArePointerColumns = false;
		};

		template<SourceQueryContext SourceType, typename TargetType> 
			requires IsCompatibleTargetContextType<SourceType, TargetType>
		struct FContextInfo
		{
			using BaseContextType = std::remove_reference_t<TargetType>;
			BaseContextType ContextWrapper;

			FContextInfo(const FQueryDescription& Description, SourceType& Context)
				: ContextWrapper(Description, Context)
			{}
		};

		template<SourceQueryContext SourceType>
		struct FContextInfo<SourceType, const SourceType&>
		{
			const SourceType& ContextWrapper;

			FContextInfo(const FQueryDescription& Description, SourceType& Context)
				: ContextWrapper(Context)
			{}
		};

		template<SourceQueryContext SourceType>
		struct FContextInfo<SourceType, SourceType&>
		{
			SourceType& ContextWrapper;

			FContextInfo(const FQueryDescription& Description, SourceType& Context)
				: ContextWrapper(Context)
			{}
		};

		template<SourceQueryContext SourceContext, bool ValidateColumns, typename TargetContext, typename RowHandleType, typename... Args>
			requires IsCompatibleTargetContextType<SourceContext, TargetContext>
		struct FContextRowHandleColumnsFunction 
			: public FFunctionColumnInfo<SourceContext, ValidateColumns, Args...>
			, public FContextInfo<SourceContext, TargetContext>
		{
			using SuperColumn = FFunctionColumnInfo<SourceContext, ValidateColumns, Args...>;
			using SuperContext = FContextInfo<SourceContext, TargetContext>;

			FContextRowHandleColumnsFunction(const FQueryDescription& Description, SourceContext& Context)
				: SuperColumn(Description, Context)
				, SuperContext(Description, Context)
			{}

			template<typename CallerType>
			AUTORTFM_INFER void Call(SourceContext& Context, const CallerType& Caller)
			{
				TConstArrayView<RowHandle> Rows = Context.GetRowHandles();
				if constexpr (SuperColumn::bArePointerColumns || std::is_pointer_v<RowHandleType>)
				{
					if constexpr (sizeof...(Args) > 0)
					{
						std::apply([this, Rows = Rows.GetData(), &Caller] AUTORTFM_ENABLE (auto&&... Column)
							{
								if constexpr (AUTORTFM_MODE_OF_CALL(Caller(this->ContextWrapper, Rows, Column...)) == autortfm_mode_disable) //-V501
								{
									AutoRTFM::UnreachableIfClosed();
								}
								Caller(this->ContextWrapper, Rows, Column...);
							}, this->Columns);
					}
					else
					{
						Caller(this->ContextWrapper, Rows.GetData());
					}
				}
				else
				{
					for (RowHandle Row : Rows)
					{
						if constexpr (sizeof...(Args) > 0)
						{
							std::apply([this, Row, &Caller] AUTORTFM_ENABLE (auto&&... Column)
								{
									if constexpr (AUTORTFM_MODE_OF_CALL(Caller(this->ContextWrapper, Row, *Column...)) == autortfm_mode_disable) //-V501
									{
										AutoRTFM::UnreachableIfClosed();
									}
									Caller(this->ContextWrapper, Row, *Column...);
									((++Column), ...);
								}, this->Columns);
						}
						else
						{
							Caller(this->ContextWrapper, Row);
						}
					}
				}
			}
		};

		template<SourceQueryContext SourceContext, bool ValidateColumns, typename TargetContext, typename... Args>
			requires IsCompatibleTargetContextType<SourceContext, TargetContext>
		struct FContextColumnsFunction 
			: public FFunctionColumnInfo<SourceContext, ValidateColumns, Args...>
			, public FContextInfo<SourceContext, TargetContext>
		{
			using SuperColumn = FFunctionColumnInfo<SourceContext, ValidateColumns, Args...>;
			using SuperContext = FContextInfo<SourceContext, TargetContext>;

			FContextColumnsFunction(const FQueryDescription& Description, SourceContext& Context)
				: SuperColumn(Description, Context)
				, SuperContext(Description, Context)
			{}

			template<typename CallerType>
			void Call(SourceContext& Context, const CallerType& Caller)
			{
				if constexpr (sizeof...(Args) > 0)
				{
					if constexpr (SuperColumn::bArePointerColumns)
					{
						std::apply(
							[this, &Caller](auto&&... Column)
							{
								Caller(this->ContextWrapper, Column...);
							}, this->Columns);
					}
					else
					{
						auto CallForwarder = [this, &Caller](auto&&... Column)
						{
							Caller(this->ContextWrapper, *Column...);
							((++Column), ...);
						};
						const uint32 RowCount = Context.GetRowCount();
						for (uint32 Index = 0; Index < RowCount; ++Index)
						{
							std::apply(CallForwarder, this->Columns);
						}
					}
				}
				else
				{
					Caller(this->ContextWrapper);
				}
			}
		};

		template<SourceQueryContext SourceContext, bool ValidateColumns, typename RowHandleType, typename... Columns>
		struct FRowHandleColumnsFunction : public FFunctionColumnInfo<SourceContext, ValidateColumns, Columns...>
		{
			using Super = FFunctionColumnInfo<SourceContext, ValidateColumns, Columns...>;

			FRowHandleColumnsFunction(const FQueryDescription& Description, SourceContext& Context)
				: Super(Description, Context)
			{}

			template<typename CallerType>
			void Call(SourceContext& Context, const CallerType& Caller)
			{
				TConstArrayView<RowHandle> Rows = Context.GetRowHandles();
				if constexpr (Super::bArePointerColumns || std::is_pointer_v<RowHandleType>)
				{
					if constexpr (sizeof...(Columns) > 0)
					{
						std::apply([Rows = Rows.GetData(), &Caller](auto&&... Column)
							{
								Caller(Rows, Column...);
							}, this->Columns);
					}
					else
					{
						Caller(Rows.GetData());
					}
				}
				else
				{
					for (RowHandle Row : Rows)
					{
						if constexpr (sizeof...(Columns) > 0)
						{
							std::apply([Row, &Caller](auto&&... Column)
								{
									Caller(Row, *Column...);
									((++Column), ...);
								}, this->Columns);
						}
						else
						{
							Caller(Row);
						}
					}
				}
			}
		};

		template<SourceQueryContext SourceContext, bool ValidateColumns, typename... Args>
		struct FColumnsFunction : public FFunctionColumnInfo<SourceContext, ValidateColumns, Args...>
		{
			using Super = FFunctionColumnInfo<SourceContext, ValidateColumns, Args...>;
			
			FColumnsFunction(const FQueryDescription& Description, SourceContext& Context)
				: Super(Description, Context)
			{}

			template<typename CallerType>
			void Call(SourceContext& Context, const CallerType& Caller)
			{
				if constexpr (sizeof...(Args) > 0)
				{
					if constexpr (Super::bArePointerColumns)
					{
						std::apply(
							[&Caller](auto&&... Column)
							{
								Caller(Column...);
							}, this->Columns);
					}
					else
					{
						auto CallForwarder = [&Caller](auto&&... Column)
						{
							Caller(*Column...);
							((++Column), ...);
						};
						const uint32 RowCount = Context.GetRowCount();
						for (uint32 Index = 0; Index < RowCount; ++Index)
						{
							std::apply(CallForwarder, this->Columns);
						}
					}
				}
				else
				{
					const uint32 RowCount = Context.GetRowCount();
					for (uint32 Index = 0; Index < RowCount; ++Index)
					{
						Caller();
					}
				}
			}
		};

		template<SourceQueryContext SourceContext, bool ValidateColumns, typename... Args>
		struct FFunctionInfoHelper
		{
			using BaseClass = FColumnsFunction<SourceContext, ValidateColumns, Args...>;
		};

		template<SourceQueryContext SourceContext, bool ValidateColumns, typename Arg0, typename... Args>
			requires IsCompatibleTargetContextType<SourceContext, Arg0>
		struct FFunctionInfoHelper<SourceContext, ValidateColumns, Arg0, Args...>
		{
			using BaseClass = FContextColumnsFunction<SourceContext, ValidateColumns, Arg0, Args...>;
		};

		template<SourceQueryContext SourceContext, bool ValidateColumns, typename Arg0, typename... Args>
			requires IsRowHandleType<Arg0>
		struct FFunctionInfoHelper<SourceContext, ValidateColumns, Arg0, Args...>
		{
			using BaseClass = FRowHandleColumnsFunction<SourceContext, ValidateColumns, Arg0, Args...>;
		};

		template<SourceQueryContext SourceContext, bool ValidateColumns, typename Arg0, typename Arg1, typename... Args>
			requires IsCompatibleTargetContextType<SourceContext, Arg0> && IsRowHandleType<Arg1>
		struct FFunctionInfoHelper<SourceContext, ValidateColumns, Arg0, Arg1, Args...>
		{
			using BaseClass = FContextRowHandleColumnsFunction<SourceContext, ValidateColumns, Arg0, Arg1, Args...>;
		};

		template<SourceQueryContext SourceContext, bool ValidateColumns, typename... Args>
		struct FFunctionInfo final : public FFunctionInfoHelper<SourceContext, ValidateColumns, Args...>::BaseClass
		{
			using Super = typename FFunctionInfoHelper<SourceContext, ValidateColumns, Args...>::BaseClass;

			FFunctionInfo(const FQueryDescription& Description, SourceContext& Context)
				: Super(Description, Context)
			{}
		};

		template<SourceQueryContext SourceContext, bool ValidateColumns, typename... Args>
		void BindQueryFunction(QueryCallback& Function,
			void (*Callback)(Args...))
		{
			Function = [Callback](const FQueryDescription& Description, SourceContext& Context)
			{
				FFunctionInfo<SourceContext, ValidateColumns, Args...>(Description, Context).Call(Context, Callback);
			};
		}

		template<SourceQueryContext SourceContext, bool ValidateColumns, typename Class, typename... Args>
		void BindQueryFunction(QueryCallback& Function, Class* Target,
			void (Class::*Callback)(Args...))
		{
			Function = [Target, Callback](const FQueryDescription& Description, SourceContext& Context)
			{
				FFunctionInfo<SourceContext, ValidateColumns, Args...>(Description, Context).Call(Context, (Target->*Callback));
			};
		}

		template<SourceQueryContext SourceContext, bool ValidateColumns, typename Class, typename... Args>
		void BindQueryFunction(QueryCallback& Function, Class* Target,
			void (Class::*Callback)(Args...) const)
		{
			Function = [Target, Callback](const FQueryDescription& Description, SourceContext& Context)
			{
				FFunctionInfo<SourceContext, ValidateColumns, Args...>(Description, Context).Call(Context, (Target->*Callback));
			};
		}

		template<SourceQueryContext SourceContext, bool ValidateColumns, typename Functor, typename Class, typename... Args>
		void BindQueryFunction_Expand(QueryCallback& Function, Functor&& CallbackObject,
			void (Class::*Callback)(Args...) const)
		{
			Function = [CallbackObject = std::forward<Functor>(CallbackObject)](const FQueryDescription& Description, SourceContext& Context)
			{
				FFunctionInfo<SourceContext, ValidateColumns, Args...>(Description, Context).Call(Context, CallbackObject);
			};
		}

		template<SourceQueryContext SourceContext, bool ValidateColumns, typename Functor, typename Class, typename... Args>
		void BindQueryFunction_Expand(QueryCallback& Function, Functor&& CallbackObject,
			void (Class::*Callback)(Args...))
		{
			Function = [CallbackObject = std::forward<Functor>(CallbackObject)](const FQueryDescription& Description, SourceContext& Context)
			{
				FFunctionInfo<SourceContext, ValidateColumns, Args...>(Description, Context).Call(Context, CallbackObject);
			};
		}

		template<SourceQueryContext SourceContext, bool ValidateColumns, typename Functor>
		void BindQueryFunction(QueryCallback& Function, Functor&& Callback)
		{
			BindQueryFunction_Expand<SourceContext, ValidateColumns>(
				Function,
				std::forward<Functor>(Callback),
				&std::remove_cv_t<std::remove_reference_t<Functor>>::operator());
		}

		
		template<bool ValidateColumns, SourceQueryContext SourceContext, typename... Args>
		AUTORTFM_INFER void CallQueryFunction(const FQueryDescription& Description, SourceContext& Context,
			QueryCallback& Function, void (*Callback)(Args...))
		{
			FFunctionInfo<SourceContext, ValidateColumns, Args...>(Description, Context).Call(Context, Callback);
		}

		template<bool ValidateColumns, SourceQueryContext SourceContext, typename Class, typename... Args>
		AUTORTFM_INFER void CallQueryFunction(const FQueryDescription& Description, SourceContext& Context,
			Class* Target, void (Class::* Callback)(Args...))
		{
			FFunctionInfo<SourceContext, ValidateColumns, Args...>(Description, Context).Call(Context, (Target->*Callback));
		}

		template<bool ValidateColumns, SourceQueryContext SourceContext, typename Class, typename... Args>
		AUTORTFM_INFER void CallQueryFunction(const FQueryDescription& Description, SourceContext& Context,
			Class* Target, void (Class::* Callback)(Args...) const)
		{
			FFunctionInfo<SourceContext, ValidateColumns, Args...>(Description, Context).Call(Context, (Target->*Callback));
		}

		template<bool ValidateColumns, SourceQueryContext SourceContext, typename Functor, typename Class, typename... Args>
		AUTORTFM_INFER void CallQueryFunction_Expand(const FQueryDescription& Description, SourceContext& Context,
			const Functor& CallbackObject, void (Class::* Callback)(Args...) const)
		{
			FFunctionInfo<SourceContext, ValidateColumns, Args...>(Description, Context).Call(Context, CallbackObject);
		}

		template<bool ValidateColumns, SourceQueryContext SourceContext, typename Functor, typename Class, typename... Args>
		AUTORTFM_INFER void CallQueryFunction_Expand(const FQueryDescription& Description, SourceContext& Context,
			const Functor& CallbackObject, void (Class::* Callback)(Args...))
		{
			FFunctionInfo<SourceContext, ValidateColumns, Args...>(Description, Context).Call(Context, CallbackObject);
		}

		template<bool ValidateColumns, SourceQueryContext SourceContext, typename Functor>
		AUTORTFM_INFER void CallQueryFunction(const FQueryDescription& Description, SourceContext& Context,
			const Functor& Callback)
		{
			CallQueryFunction_Expand<ValidateColumns>(Description, Context, Callback, &Functor::operator());
		}
		

		template<typename Arg>
		void AddColumnToSelect(Select& Target)
		{
			if constexpr (std::is_const_v<BaseColumnType<Arg>>)
			{
				Target.ReadOnly(UndecoratedColumnType<Arg>::StaticStruct());
			}
			else
			{
				Target.ReadWrite(UndecoratedColumnType<Arg>::StaticStruct());
			}
		}

		template<typename ContextType> void RegisterDependencies(FQueryDescription& Query)
		{
			using BaseType = typename TRemoveReference<ContextType>::Type;
			if constexpr (TIsDerivedFrom<BaseType, FQueryContextForwarder>::Value)
			{
				BaseType::Register(Query);
			}
		}

		template<SourceQueryContext SourceContext, typename... Args>
		struct RegisterFunctionArgumentsHelper
		{
			void Register(FQueryDescription&, Select& Target)
			{
				(AddColumnToSelect<Args>(Target), ...);
			}
		};

		template<SourceQueryContext SourceContext>
		struct RegisterFunctionArgumentsHelper<SourceContext>
		{
			static void Register(FQueryDescription&, Select&) {}
		};

		template<SourceQueryContext SourceContext, typename Arg0>
		struct RegisterFunctionArgumentsHelper<SourceContext, Arg0>
		{
			static void Register(FQueryDescription& Query, Select& Target)
			{
				if constexpr (IsCompatibleTargetContextType<SourceContext, Arg0>)
				{
					RegisterDependencies<Arg0>(Query);
				}
				else if constexpr (IsValidColumnType<Arg0>())
				{
					AddColumnToSelect<Arg0>(Target);
				}
				// The third option is a row handle, which doesn't need to be registered.
			}
		};

		template<SourceQueryContext SourceContext, typename Arg0, typename Arg1, typename... Args>
		struct RegisterFunctionArgumentsHelper<SourceContext, Arg0, Arg1, Args...>
		{
			static void Register(FQueryDescription& Query, Select& Target)
			{
				if constexpr (IsCompatibleTargetContextType<SourceContext, Arg0>)
				{
					RegisterDependencies<Arg0>(Query);
				}
				else if constexpr (IsValidColumnType<Arg0>())
				{
					AddColumnToSelect<Arg0>(Target);
				}
				// The third option is a row handle, which doesn't need to be registered.

				if constexpr (IsValidColumnType<Arg1>())
				{
					AddColumnToSelect<Arg1>(Target);
				}
				// The second option is a row handle, which doesn't need to be registered.

				(AddColumnToSelect<Args>(Target), ...);
			}
		};

		template<SourceQueryContext SourceContext, typename... Args>
		void RegisterFunctionArguments(FQueryDescription& Query, Select& Target, void (*)(Args...))
		{
			RegisterFunctionArgumentsHelper<SourceContext, Args...>::Register(Query, Target);
		}

		template<SourceQueryContext SourceContext, typename Class, typename... Args>
		void RegisterFunctionArguments(FQueryDescription& Query, Select& Target, void (Class::*)(Args...))
		{
			RegisterFunctionArgumentsHelper<SourceContext, Args...>::Register(Query, Target);
		}

		template<SourceQueryContext SourceContext, typename Class, typename... Args>
		void RegisterFunctionArguments(FQueryDescription& Query, Select& Target, void (Class::*)(Args...) const)
		{
			RegisterFunctionArgumentsHelper<SourceContext, Args...>::Register(Query, Target);
		}

		template<SourceQueryContext SourceContext, typename Functor>
		void RegisterFunctionArguments(FQueryDescription& Query, Select& Target, Functor)
		{
			RegisterFunctionArguments<SourceContext>(Query, Target, &Functor::operator());
		}

		template<SourceQueryContext SourceContext, typename... Args>
		struct IsValidSelectFunctionSignatureImpl2
		{
			constexpr static bool HasValidColumnTypes = (IsValidColumnType<Args>() && ...);
			static_assert(HasValidColumnTypes, "One or more of the provided columns is not compatible with query callback.");

			constexpr static bool Value = HasValidColumnTypes;
		};

		template<SourceQueryContext SourceContext, typename Arg0, typename... Args>
		struct IsValidSelectFunctionSignatureImpl2<SourceContext, Arg0, Args...>
		{
			constexpr static bool Arg0IsContext = IsCompatibleTargetContextType<SourceContext, Arg0>;
			constexpr static bool Arg0IsRowHandle = IsRowHandleType<Arg0>;
			constexpr static bool Arg0IsArgument = IsValidColumnType<Arg0>();
			constexpr static bool Arg0IsValidRowHandle = Arg0IsRowHandle && IsRowTypeCompatibleWithColumns<Arg0, Args...>();
			constexpr static bool Arg0IsValid = (Arg0IsContext || Arg0IsValidRowHandle || Arg0IsArgument);
			
			static_assert(!Arg0IsRowHandle || Arg0IsValidRowHandle, "Row handles need to taken by value when the columns are requested by "
				"reference or by const pointer if the columns are taken by reference.");
			static_assert(Arg0IsValid, "The first provided argument has to be a reference to a compatible query reference, a row handle"
				" or a (const) reference/pointer to a column.");
			
			constexpr static bool HasValidColumnTypes = (IsValidColumnType<Args>() && ...);
			static_assert(HasValidColumnTypes, "One or more of the provided columns is not compatible with query callback.");

			constexpr static bool Value = Arg0IsValid && HasValidColumnTypes;
		};

		template<SourceQueryContext SourceContext, typename Arg0, typename Arg1, typename... Args>
		struct IsValidSelectFunctionSignatureImpl2<SourceContext, Arg0, Arg1, Args...>
		{
			constexpr static bool Arg0IsContext = IsCompatibleTargetContextType<SourceContext, Arg0>;
			constexpr static bool Arg0IsRowHandle = IsRowHandleType<Arg0>;
			constexpr static bool Arg0IsArgument = IsValidColumnType<Arg0>();
			constexpr static bool Arg0IsValidRowHandle = Arg0IsRowHandle && IsRowTypeCompatibleWithColumns<Arg0, Args...>();
			constexpr static bool Arg0IsValid = (Arg0IsContext || Arg0IsValidRowHandle || Arg0IsArgument);
			
			static_assert(!Arg0IsRowHandle || Arg0IsValidRowHandle, "Row handles need to taken by value when the columns are requested by "
				"reference or by const pointer if the columns are taken by reference.");
			static_assert(Arg0IsValid, "The first argument to a query callback has to be a reference to a compatible query reference, a "
				"row handle or a (const) reference/pointer to a column.");
			
			constexpr static bool Arg1IsRowHandle = IsRowHandleType<Arg1>;
			constexpr static bool Arg1IsArgument = IsValidColumnType<Arg1>();
			constexpr static bool Arg1IsValidRowHandle = Arg1IsRowHandle && IsRowTypeCompatibleWithColumns<Arg1, Args...>();
			constexpr static bool Arg1IsValid = (Arg1IsValidRowHandle || Arg1IsArgument);

			static_assert(!Arg1IsRowHandle || Arg1IsValidRowHandle, "Row handles need to taken by value when the columns are requested by "
				"reference or by const pointer if the columns are taken by reference.");
			static_assert(Arg1IsValid, "The second argument to a query callback has to be a row handle or a (const) reference/pointer "
				"to a column.");
			
			constexpr static bool HasValidColumnTypes = (IsValidColumnType<Args>() && ...);
			static_assert(HasValidColumnTypes, "One or more of the provided columns is not compatible with query callback.");

			constexpr static bool Value = Arg0IsValid && Arg1IsValid && HasValidColumnTypes;
		};

		template<SourceQueryContext SourceContext>
		struct IsValidSelectFunctionSignatureImpl2<SourceContext>
		{
			constexpr static bool Value = false;
		};
				
		template<SourceQueryContext SourceContext, typename T> 
		struct IsValidSelectFunctionSignatureImpl
		{ 
			constexpr static bool Value = false; 
		};

		template<SourceQueryContext SourceContext, typename... Args> 
		struct IsValidSelectFunctionSignatureImpl<SourceContext, void (*)(Args...)>
		{ 
			constexpr static bool Value = IsValidSelectFunctionSignatureImpl2<SourceContext, Args...>::Value;
		};
		
		template<SourceQueryContext SourceContext, typename Class, typename... Args> 
		struct IsValidSelectFunctionSignatureImpl<SourceContext, void (Class::*)(Args...)>
		{ 
			constexpr static bool Value = IsValidSelectFunctionSignatureImpl2<SourceContext, Args...>::Value;
		};

		template<SourceQueryContext SourceContext, typename Class, typename... Args>
		struct IsValidSelectFunctionSignatureImpl<SourceContext, void (Class::*)(Args...) const>
		{ 
			constexpr static bool Value = IsValidSelectFunctionSignatureImpl2<SourceContext, Args...>::Value;
		};

		template<SourceQueryContext SourceContext, typename T> constexpr bool IsValidSelectFunctionSignature()
		{ 
			if constexpr (IsFunctor<T>)
			{
				return IsValidSelectFunctionSignatureImpl<SourceContext, decltype(&T::operator())>::Value;
			}
			else
			{
				return IsValidSelectFunctionSignatureImpl<SourceContext, T>::Value;
			}
		};
		
		inline void PrepareForQueryBinding(FQueryDescription& Query, const FProcessor& Processor)
		{
			Query.Callback.Type = EQueryCallbackType::Processor;
			Query.Callback.Phase = Processor.Phase;
			Query.Callback.Group = Processor.Group;
			Query.Callback.ActivationName = Processor.ActivationName;
			Query.Callback.Active = Processor.ActivationName.IsNone();
			if (!Processor.BeforeGroup.IsNone())
			{
				Query.Callback.BeforeGroups.Add(Processor.BeforeGroup);
			}
			if (!Processor.AfterGroup.IsNone())
			{
				Query.Callback.AfterGroups.Add(Processor.AfterGroup);
			}
			Query.Callback.ExecutionMode = Processor.ExecutionMode;

			Query.bShouldBatchModifications = Processor.bBatchModifications;
		}

		inline void PrepareForQueryBinding(FQueryDescription& Query, const FObserver& Observer)
		{
			switch (Observer.Event)
			{
			case FObserver::EEvent::Add:
				Query.Callback.Type = EQueryCallbackType::ObserveAdd;
				break;
			case FObserver::EEvent::Remove:
				Query.Callback.Type = EQueryCallbackType::ObserveRemove;
				break;
			}
			Query.Callback.ActivationName = Observer.ActivationName;
			Query.Callback.Active = Observer.ActivationName.IsNone();
			Query.Callback.MonitoredType = Observer.Monitor;
			Query.Callback.ExecutionMode = Observer.ExecutionMode;
		}

		inline void PrepareForQueryBinding(FQueryDescription& Query, const FPhaseAmble& PhaseAmble)
		{
			switch (PhaseAmble.Location)
			{
			case FPhaseAmble::ELocation::Preamble:
				Query.Callback.Type = EQueryCallbackType::PhasePreparation;
				break;
			case FPhaseAmble::ELocation::Postamble:
				Query.Callback.Type = EQueryCallbackType::PhaseFinalization;
				break;
			}
			Query.Callback.ActivationName = PhaseAmble.ActivationName;
			Query.Callback.Active = PhaseAmble.ActivationName.IsNone();
			Query.Callback.Phase = PhaseAmble.Phase;
			Query.Callback.ExecutionMode = PhaseAmble.ExecutionMode;
		}
		
		template<typename CallbackType, typename Function>
		void PrepareForQueryBinding(Select& Target, FQueryDescription& Query, FName Name,
			const CallbackType& Type, Function Callback)
		{
			static_assert(TIsDerivedFrom<CallbackType, FQueryCallbackType>::Value, "The callback type provided isn't one of the available "
				"classes derived from FQueryCallbackType.");
			static_assert(IsValidSelectFunctionSignature<IQueryContext, Function>(),
				R"(The function provided to the Query Builder's Select call wasn't invocable or doesn't contain a supported combination of arguments.
The following options are supported:
- void([const]Column&...) 
- void(RowHandle, [const]Column&...) 
- void(<Context>&, [const]Column&...) 
- void(<Context>&, RowHandle, [const]Column&...) 
- void(<Context>&, [const]Column*...) 
- void(<Context>&, const RowHandle*, [const]Column*...) 
Where <Context> is IQueryContext or FCachedQueryContext<...>
e.g. void(FCachedQueryContext<Subsystem1, const Subsystem2>& Context, RowHandle Row, ColumnType0& ColumnA, const ColumnType1& ColumnB) {...}
)");
			RegisterFunctionArguments<IQueryContext>(Query, Target, Callback);
			PrepareForQueryBinding(Query, Type);
			Query.Callback.Name = Name;
		}
	} // namespace Private

	template<typename CallbackType, typename Function>
	Select::Select(FName Name, const CallbackType& Type, Function&& Callback)
		: Select()
	{
		static constexpr bool ValidateColumns = false;
		Private::PrepareForQueryBinding(*this, Query, Name, Type, Callback);
		Private::BindQueryFunction<IQueryContext, ValidateColumns>(Query.Callback.Function, std::forward<Function>(Callback));
	}

	template<typename CallbackType, typename Class, typename Function>
	Select::Select(FName Name, const CallbackType& Type, Class* Instance, Function&& Callback)
		: Select()
	{
		static constexpr bool ValidateColumns = false;
		Private::PrepareForQueryBinding(*this, Query, Name, Type, Callback);
		Private::BindQueryFunction<IQueryContext, ValidateColumns>(Query.Callback.Function, Instance, Callback);
	}

	template<TDataColumnType... TargetTypes>
	Select& Select::ReadOnly()
	{
		ReadOnly({ TargetTypes::StaticStruct()... });
		return *this;
	}

	template<TDataColumnType... TargetTypes>
	Select& Select::ReadOnly(EOptional Optional)
	{
		ReadOnly({ TargetTypes::StaticStruct()... }, Optional);
		return *this;
	}

	template<TDynamicColumnTemplate Target>
	Select& Select::ReadOnly(const FName& Identifier)
	{
		ReadOnly(FDynamicColumnDescription
			{
				.TemplateType = Target::StaticStruct(),
				.Identifier = Identifier
			});
		return *this;
	}

	template<TDataColumnType... TargetTypes>
	Select& Select::ReadWrite()
	{
		ReadWrite({ TargetTypes::StaticStruct()... });
		return *this;
	}

	template<TDynamicColumnTemplate Target>
	Select& Select::ReadWrite(const FName& Identifier)
	{
		ReadWrite(FDynamicColumnDescription
			{
				.TemplateType = Target::StaticStruct(),
				.Identifier = Identifier
			});
		return *this;
	}

	//
	// FSimpleQuery
	//

	template<TColumnType... TargetTypes>
	FSimpleQuery& FSimpleQuery::All()
	{
		All({ TargetTypes::StaticStruct()... });
		return *this;
	}

	template<TColumnType... TargetTypes>
	FSimpleQuery& FSimpleQuery::Any()
	{
		Any({ TargetTypes::StaticStruct()... });
		return *this;
	}

	template <TDynamicColumnTemplate DynamicColumnTemplate>
	FSimpleQuery& FSimpleQuery::Any(const FName& Identifier)
	{
		return Any(FDynamicColumnDescription
		{
			.TemplateType = DynamicColumnTemplate::StaticStruct(),
			.Identifier = Identifier
		});
	}

	template<TColumnType... TargetTypes>
	FSimpleQuery& FSimpleQuery::None()
	{
		None({ TargetTypes::StaticStruct()... });
		return *this;
	}

	template<TValueTagType>
	FSimpleQuery& FSimpleQuery::All(const FName& Tag)
	{
		return All(FValueTag(Tag));
	}

	template<TValueTagType>
	FSimpleQuery& FSimpleQuery::All(const FName& Tag, const FName& Value)
	{
		return All(FValueTag(Tag), Value);
	}
	
	template <TDynamicColumnTemplate DynamicColumnTemplate>
	FSimpleQuery& FSimpleQuery::None(const FName& Identifier)
	{
		return None(FDynamicColumnDescription
			{
				.TemplateType = DynamicColumnTemplate::StaticStruct(),
				.Identifier = Identifier
			});
	}

	template <TDynamicColumnTemplate DynamicColumnTemplate>
	FSimpleQuery& FSimpleQuery::All(const FName& Identifier)
	{
		return All(FDynamicColumnDescription
			{
				.TemplateType = DynamicColumnTemplate::StaticStruct(),
				.Identifier = Identifier
			});
	}

	template <TEnumType EnumT>
    FSimpleQuery& FSimpleQuery::All()
    {
    	return All(static_cast<const UEnum&>(*StaticEnum<EnumT>()));
    }

    template <TEnumType EnumT>
    FSimpleQuery& FSimpleQuery::All(EnumT EnumValue)
    {
    	return All(static_cast<const UEnum&>(*StaticEnum<EnumT>()), static_cast<int64>(EnumValue));
    }
	
	template<auto Value, TEnumType EnumT = decltype(Value)>
	FSimpleQuery& All()
	{
		return All<EnumT>(Value);
	}

	//
	// External query bindings
	//
	template<typename Function>
	DirectQueryCallback CreateDirectQueryCallbackBinding(Function&& Callback)
	{
		static_assert(Private::IsValidSelectFunctionSignature<IDirectQueryContext, Function>(),
			R"(The function provided to the Query Builder's CreateDirectQueryCallbackBinding call wasn't invocable or doesn't contain a supported combination of arguments.
The following options are supported:
- void([const]Column&...) 
- void(RowHandle, [const]Column&...) 
- void(<Context>&, [const]Column&...) 
- void(<Context>&, RowHandle, [const]Column&...) 
- void(<Context>&, [const]Column*...) 
- void(<Context>&, const RowHandle*, [const]Column*...) 
Where <Context> is IDirectQueryContext
e.g. void(IDirectQueryContext& Context, RowHandle Row, ColumnType0& ColumnA, const ColumnType1& ColumnB) {...}
)");

		return [Callback = Forward<Function>(Callback)](
			const FQueryDescription& Description,
			IDirectQueryContext& Context)
		{
			static constexpr bool ValidateColumns = true;
			Private::CallQueryFunction<ValidateColumns>(Description, Context, Callback);
		};
	}

	template<typename Function>
	SubqueryCallback CreateSubqueryCallbackBinding(Function&& Callback)
	{
		static_assert(Private::IsValidSelectFunctionSignature<ISubqueryContext, Function>(),
			R"(The function provided to the Query Builder's CreateSubqueryCallbackBinding call wasn't invocable or doesn't contain a supported combination of arguments.
The following options are supported:
- void([const]Column&...) 
- void(RowHandle, [const]Column&...) 
- void(<Context>&, [const]Column&...) 
- void(<Context>&, RowHandle, [const]Column&...) 
- void(<Context>&, [const]Column*...) 
- void(<Context>&, const RowHandle*, [const]Column*...) 
Where <Context> is ISubqueryContext
e.g. void(ISubqueryContext& Context, RowHandle Row, ColumnType0& ColumnA, const ColumnType1& ColumnB) {...}
)");

		return [Callback = Forward<Function>(Callback)](
			const FQueryDescription& Description,
			ISubqueryContext& Context)
		{
			static constexpr bool ValidateColumns = true;
			Private::CallQueryFunction<ValidateColumns>(Description, Context, Callback);
		};
	}
} // namespace UE::Editor::DataStorage::Queries

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Templates/IsConst.h"
#endif
