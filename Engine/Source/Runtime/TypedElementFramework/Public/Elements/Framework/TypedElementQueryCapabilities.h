// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "Concepts/SameAs.h"
#include "Containers/ContainersFwd.h"
#include "DataStorage/CommonTypes.h"
#include "DataStorage/Handles.h"
#include "DataStorage/Queries/Conditions.h"
#include "Elements/Framework/TypedElementQueryFunctionArgumentTypes.h"
#include "Elements/Framework/TypedElementRowHandleArrayView.h"
#include "Misc/EnumClassFlags.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"


class UScriptStruct;

namespace UE::Editor::DataStorage::Queries
{
	enum class EContextCapabilityFlags
	{
		SupportsSingle = 1 << 0,
		SupportsBatch = 1 << 1
	};
	ENUM_CLASS_FLAGS(EContextCapabilityFlags)

	/** 
	 * Class fragments used to composite a context.
	 * A query context is created by combining various context capabilities together. Users can request a query context
	 * being created with one or more of these capabilities and based on the requested capabilities they get connected
	 * with an implementation. Each capability represents a subset of functionality of TEDS such as access to column data,
	 * creating new rows or access to subqueries.
	 */
	struct IContextCapability
	{
		virtual ~IContextCapability() = default;
	};

	template<typename T>
	concept ContextCapability = std::is_base_of_v<IContextCapability, T> && !std::is_same_v<IContextCapability, T>;

	template<ContextCapability Capability>
	struct ImplementsContextCapability 
	{
		static constexpr int32 Id = Capability::Id;
		inline static FName Name = Capability::Name;
	};

	template<typename ReturnType>
	class TQueryFunction;
	template<typename ReturnType>
	class TConstQueryFunction;

	struct IContextContract;
	struct IQueryFunctionResponse;

	template<typename Return, FunctionType Function>
	TQueryFunction<Return> BuildQueryFunction(Function&& Callback); // forward declaration

	template<typename Return, FunctionType Function>
	TConstQueryFunction<Return> BuildConstQueryFunction(Function&& Callback); // forward declaration

	// Here only to make sure that the argument shows up in compiler errors.

	template <typename T>
	constexpr T& VarType(T& Value);
	
	template <typename T>
	constexpr const T& VarType(const T& Value);
	
	template <typename T>
	constexpr T&& VarType(T&& Value);

#define ArgType(Type, Name) Type
#define ArgName(Type, Name) Name
#define ArgTypeName(Type, Name) Type Name

	// Fill out the FCapabilityInfo

	template<typename Implementation, typename Capability>
	struct FCapabilityCompatibilityCheck
	{
	};

#define CapabilityStart(InCapability, InFlags) \
	template<typename T> \
	struct I##InCapability ; \
	\
	template<typename Implementation> \
	struct FCapabilityCompatibilityCheck<Implementation, I##InCapability <IContextCapability> > \
	{ \
		static constexpr bool Value =

#define Function0(Capability, Return, Function) \
	requires(Implementation& obj) \
	{ \
		{ obj. Function () } -> UE::CSameAs< Return >; \
	} &&  
#define Function1(Capability, Return, Function, Arg1) \
	requires(Implementation& obj, ArgTypeName Arg1) \
	{ \
		{ obj. Function ( VarType< ArgType Arg1 >( ArgName Arg1 ) ) } -> UE::CSameAs< Return >; \
	} && 
#define Function2(Capability, Return, Function, Arg1, Arg2) \
	requires(Implementation& obj, ArgTypeName Arg1, ArgTypeName Arg2) \
	{ \
		{ obj. Function ( \
			VarType< ArgType Arg1 >( ArgName Arg1 ), \
			VarType< ArgType Arg2 >( ArgName Arg2 ) ) } -> UE::CSameAs< Return >; \
	} && 
#define Function3(Capability, Return, Function, Arg1, Arg2, Arg3) \
	requires(Implementation& obj, ArgTypeName Arg1, ArgTypeName Arg2, ArgTypeName Arg3) \
	{ \
		{ obj. Function ( \
			VarType< ArgType Arg1 >(ArgName Arg1), \
			VarType< ArgType Arg2 >(ArgName Arg2), \
			VarType< ArgType Arg3 >(ArgName Arg3)) } -> UE::CSameAs< Return >; \
	} &&
#define Function4(Capability, Return, Function, Arg1, Arg2, Arg3, Arg4) \
	requires(Implementation& obj, ArgTypeName Arg1, ArgTypeName Arg2, ArgTypeName Arg3, ArgTypeName Arg4) \
	{ \
		{ obj. Function ( \
			VarType< ArgType Arg1 >(ArgName Arg1), \
			VarType< ArgType Arg2 >(ArgName Arg2), \
			VarType< ArgType Arg3 >(ArgName Arg3), \
			VarType< ArgType Arg4 >(ArgName Arg4)) } -> UE::CSameAs< Return >; \
	} &&

#define ConstFunction0(Capability, Return, Function) \
	requires(const Implementation& obj) \
	{ \
		{ obj. Function () } -> UE::CSameAs< Return >; \
	} && 
#define ConstFunction1(Capability, Return, Function, Arg1) \
	requires(const Implementation& obj, ArgTypeName Arg1) \
	{ \
		{ obj. Function ( VarType< ArgType Arg1 >( ArgName Arg1 ) ) } -> UE::CSameAs< Return >; \
	} && 
#define ConstFunction2(Capability, Return, Function, Arg1, Arg2) \
	requires(const Implementation& obj, ArgTypeName Arg1, ArgTypeName Arg2) \
	{ \
		{ obj. Function ( \
			VarType< ArgType Arg1 >( ArgName Arg1 ), \
			VarType< ArgType Arg2 >( ArgName Arg2 ) ) } -> UE::CSameAs< Return >; \
	} && 
#define ConstFunction3(Capability, Return, Function, Arg1, Arg2, Arg3) \
	requires(const Implementation& obj, ArgTypeName Arg1, ArgTypeName Arg2, ArgTypeName Arg3) \
	{ \
		{ obj. Function ( \
			VarType< ArgType Arg1 >(ArgName Arg1), \
			VarType< ArgType Arg2 >(ArgName Arg2), \
			VarType< ArgType Arg3 >(ArgName Arg3)) } -> UE::CSameAs< Return >; \
	} &&
#define ConstFunction4(Capability, Return, Function, Arg1, Arg2, Arg3, Arg4) \
	requires(const Implementation& obj, ArgTypeName Arg1, ArgTypeName Arg2, ArgTypeName Arg3, ArgTypeName Arg4) \
	{ \
		{ obj. Function ( \
			VarType< ArgType Arg1 >(ArgName Arg1), \
			VarType< ArgType Arg2 >(ArgName Arg2), \
			VarType< ArgType Arg3 >(ArgName Arg3), \
			VarType< ArgType Arg4 >(ArgName Arg4)) } -> UE::CSameAs< Return >; \
	} &&

#define CapabilityEnd(InCapability) \
			true; \
	};

#define DeprecatedFunction(Version, Msg)

#include "Elements/Framework/TypedElementQueryCapabilities.inl"

#undef DeprecatedFunction
#undef Function0
#undef Function1
#undef Function2
#undef Function3
#undef Function4
#undef ConstFunction0
#undef ConstFunction1
#undef ConstFunction2
#undef ConstFunction3
#undef ConstFunction4
#undef CapabilityStart
#undef CapabilityEnd

	// Fill in the ability interface.

	// Assign a unique id to each capability, starting at 0. Since __COUNTER__ is a preprocessor defined macro other headers are free
	// to use it as well, leading to the starting address not being 0, hence recording the first occurrence and subtracting it from
	// the ids.
	static constexpr int32 CapabilityBaseId = __COUNTER__;

#define WithWrappers 1	

#define CapabilityStart(InCapability, InFlags) \
	template<typename Base> \
	struct I##InCapability : Base \
	{ \
		inline static const FName Name = #InCapability ; \
		static constexpr int32 Id = __COUNTER__ - CapabilityBaseId - 1; \
		static constexpr EContextCapabilityFlags Flags = InFlags; \
		virtual ~I##InCapability () = default; \

#define Function0(Capability, Return, Function) virtual Return Function() = 0;
#define Function1(Capability, Return, Function, Arg1) virtual Return Function(ArgTypeName Arg1 ) = 0;
#define Function2(Capability, Return, Function, Arg1, Arg2) virtual Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 ) = 0;
#define Function3(Capability, Return, Function, Arg1, Arg2, Arg3) virtual Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 , ArgTypeName Arg3 ) = 0;
#define Function4(Capability, Return, Function, Arg1, Arg2, Arg3, Arg4) virtual Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 , ArgTypeName Arg3 , ArgTypeName Arg4 ) = 0;

#define ConstFunction0(Capability, Return, Function) virtual Return Function() const = 0;
#define ConstFunction1(Capability, Return, Function, Arg1) virtual Return Function(ArgTypeName Arg1 ) const = 0;
#define ConstFunction2(Capability, Return, Function, Arg1, Arg2) virtual Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 ) const = 0;
#define ConstFunction3(Capability, Return, Function, Arg1, Arg2, Arg3) virtual Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 , ArgTypeName Arg3 ) const = 0;
#define ConstFunction4(Capability, Return, Function, Arg1, Arg2, Arg3, Arg4) virtual Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 , ArgTypeName Arg3 , ArgTypeName Arg4 ) const = 0;

#define DeprecatedFunction(Version, Msg) UE_DEPRECATED(Version, Msg)

#define CapabilityEnd(InCapability) \
	}; \
	using InCapability = I##InCapability <IContextCapability>; \


#include "Elements/Framework/TypedElementQueryCapabilities.inl"

#undef ArgTypeName
#undef ArgType
#undef ArgName
#undef DeprecatedFunction
#undef Function0
#undef Function1
#undef Function2
#undef Function3
#undef Function4
#undef ConstFunction0
#undef ConstFunction1
#undef ConstFunction2
#undef ConstFunction3
#undef ConstFunction4
#undef CapabilityStart
#undef CapabilityEnd
#undef WithWrappers
} // namespace UE::Editor::DataStorage::Queries