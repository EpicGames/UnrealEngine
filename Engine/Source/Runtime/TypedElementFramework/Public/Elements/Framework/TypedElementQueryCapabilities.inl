// Copyright Epic Games, Inc. All Rights Reserved.

// Mechanic to declare TEDS capabilities.
// Do not include #pragma once here or any other guards against repeated inclusion as this file is meant to be included multiple times.
// This file contains the definitions of all available query capabilities. The macros get expended in various ways to generate interfaces,
// function forwarders, contexts, etc.

/**
 * Capability to provide information about tables.
 */
CapabilityStart(TableInfo, EContextCapabilityFlags::SupportsSingle | EContextCapabilityFlags::SupportsBatch)
	/** Checks if the provided table handle is pointing to a valid table. A table handle of value `InvalidTableHandle` is considered invalid. */
	ConstFunction1(TableInfo, bool, IsTableValid, (TableHandle, Table))
	/**
	 * Returns a previously registered table with the provided name or InvalidTableHandle if not found.
	 * This function will always return the registered version of the table and never any of the dynamically created variants.
	 */
	ConstFunction1(TableInfo, TableHandle, FindTable, (const FName&, TableName))
	/** Finds the table the row is stored in. */
	ConstFunction1(TableInfo, TableHandle, FindTable, (RowHandle, Row))
	/** Lists the handles for all tables. */
	ConstFunction1(TableInfo, void, ListTables, (TFunctionRef<void(TableHandle)>, Callback))
	/** Lists the handles for all tables of a specific type. */
	ConstFunction2(TableInfo, void, ListTables, (ETableType, TableType), (TFunctionRef<void(TableHandle)>, Callback))
	/** Lists all the columns in a table. */
	ConstFunction2(TableInfo, void, ListTableColumns, (TableHandle, Table), (TFunctionRef<bool(const UScriptStruct*)>, Callback))
	/** Returns true if the table contains the requested columns, otherwise false. */
	ConstFunction2(TableInfo, bool, TableHasColumns, (TableHandle, Table), (TConstArrayView<const UScriptStruct*>, Columns))
	/** Returns various bits of information about the table. */
	ConstFunction1(TableInfo, FTableInfoView, GetTableInfo, (TableHandle, Table))
	/** Lists all rows in a table. The callback may be called multiple times, once for each chunk in the table. */
	ConstFunction2(TableInfo, void, ListTableRows, (TableHandle, Table), (TFunctionRef<void(FRowHandleArrayView)>, Callback))
	/** Lists all the domain names for foreign keys that have been registered with this table. */
	ConstFunction3(TableInfo, void, ListTableForeignKeyDomains, (TableHandle, Table), (bool, bIncludeParents), (TFunctionRef<void(const FName&)>, Callback))
	/** Get the unique tag that identifies the requested table. If the table is not found a nullptr will be returned. */
	ConstFunction1(TableInfo, const UScriptStruct*, GetTagForTable, (TableHandle, Table))

#if defined(WithWrappers)
	/** Returns true if the table contains the requested columns, otherwise false. */
	template<TColumnType... ColumnTypes>
	bool TableHasColumns(TableHandle Table) const
	{
		return this->TableHasColumns(Table, TConstArrayView<const UScriptStruct*>({ ColumnTypes::StaticStruct()... }));
	}
#endif
CapabilityEnd(TableInfo)

/**
 * Capability to provide information about the currently active table.
 */
CapabilityStart(CurrentTableInfo, EContextCapabilityFlags::SupportsSingle | EContextCapabilityFlags::SupportsBatch)
	/** Returns the handle to the table that's currently being used in the context. */
	ConstFunction0(CurrentTableInfo, TableHandle, GetCurrentTable)
	/** Returns true if the current table contains the requested columns, otherwise false. */
	ConstFunction1(CurrentTableInfo, bool, CurrentTableHasColumns, (TConstArrayView<const UScriptStruct*>, Columns))
	/** Get the unique tag that identifies the current table. */
	ConstFunction0(CurrentTableInfo, const UScriptStruct*, GetTagForCurrentTable)

#if defined(WithWrappers)
	/** Returns true if the current table contains the requested columns, otherwise false. */
	template<TColumnType... ColumnTypes>
	bool CurrentTableHasColumns() const
	{
		return this->CurrentTableHasColumns(TConstArrayView<const UScriptStruct*>({ ColumnTypes::StaticStruct()... }));
	}
#endif
CapabilityEnd(CurrentTableInfo)

/**
 * Capability to allow managing tables.
 */
CapabilityStart(TableManagement, EContextCapabilityFlags::SupportsSingle | EContextCapabilityFlags::SupportsBatch)
	/** Register a table by name. */
	Function2(TableManagement, TableHandle, RegisterTable, (TConstArrayView<const UScriptStruct*>, Columns), (const FName&, TableName))
	/** Register a table by name with additional configuration options. */
	Function3(TableManagement, TableHandle, RegisterTable, \
		(TConstArrayView<const UScriptStruct*>, Columns), (const FName&, TableName), (FTableRegistrationOptions, Options))
	/**
	 * Register a function that's used to generate a foreign key for a particular row. Foreign keys create a unique identifier that
	 * represents the row for a specific domain.
	 */
	Function3(TableManagement, void, RegisterTableForeignKey, \
		(TableHandle, Table), (const FName&, Domain), (const Queries::TConstQueryFunction<FForeignKey>&, KeyConstructor))
	/**
	 * Register a function that's used to generate a foreign key for a particular row. Foreign keys create a unique identifier that
	 * represents the row for a specific domain.
	 */
	Function3(TableManagement, void, RegisterTableForeignKey, \
		(TableHandle, Table), (const FName&, Domain), (Queries::TConstQueryFunction<FForeignKey>&&, KeyConstructor))
	
	/** Creates a foreign key for the provided row using the table the row is stored in and the assigned domain. */
	ConstFunction2(TableManagement, FForeignKey, BuildForeignKey, (const FName&, Domain), (RowHandle, Row))

#if defined(WithWrappers)
	/** Register a table by name. */
	template<TColumnType... ColumnTypes>
	TableHandle RegisterTable(const FName& TableName)
	{
		return this->RegisterTable(TConstArrayView<const UScriptStruct*>({ ColumnTypes::StaticStruct()... }), TableName);
	}

	/** Register a table by name with additional configuration options. */
	template<TColumnType... ColumnTypes>
	TableHandle RegisterTable(const FName& TableName, const FTableRegistrationOptions& Options)
	{
		return this->RegisterTable(TConstArrayView<const UScriptStruct*>({ ColumnTypes::StaticStruct()... }), TableName, Options);
	}

	/**
	 * Register a function that's used to generate a foreign key for a particular row. Foreign keys create a unique identifier that
	 * represents the row for a specific domain.
	 */
	template<FunctionType Function>
	void RegisterTableForeignKey(TableHandle Table, const FName& Domain, Function&& Callback)
	{
		RegisterTableForeignKey(Table, Domain, BuildConstQueryFunction<FForeignKey>(Forward<Function>(Callback)));
	}

	UE_DEPRECATED(5.8, "Use the version of `RegisterTable` that takes a `FTableRegistrationOptions`.")
	inline TableHandle RegisterTable(TableHandle SourceTable, TConstArrayView<const UScriptStruct*> ColumnList, const FName& TableName)
	{
		return this->RegisterTable(ColumnList, TableName, FTableRegistrationOptions{ .SourceTable = SourceTable });
	}
	
	template<TColumnType... ColumnTypes>
	UE_DEPRECATED(5.8, "Use the version of `RegisterTable` that takes a `FTableRegistrationOptions`.")
	TableHandle RegisterTable(TableHandle SourceTable, const FName& TableName)
	{
		return this->RegisterTable(TConstArrayView<const UScriptStruct*>({ ColumnTypes::StaticStruct()... }), 
			TableName, FTableRegistrationOptions{ .SourceTable = SourceTable });
	}
#endif
CapabilityEnd(TableManagement)

/**
 * Capability to provide information about the currently active row.
 */
CapabilityStart(SingleRowInfo, EContextCapabilityFlags::SupportsSingle)
	/**
	 * Returns whether a column matches the requested type or not. This version only applies to current row. This version is faster than
	 * querying for an arbitrary row.
	 */
	DeprecatedFunction(5.8, "Use CurrentTableHasColumns instead. Make sure to add CurrentTableInfo to the query's context capabilities.")
	ConstFunction1(SingleRowInfo, bool, CurrentRowHasColumns, (TConstArrayView<const UScriptStruct*>, Columns))
	/** Returns the currently active row. */
	ConstFunction0(SingleRowInfo, RowHandle, GetCurrentRow)

#if defined(WithWrappers)
	/**
	 * Returns whether the current row's table has the requested columns.
	 * @deprecated Use CurrentTableHasColumns instead. Make sure to add CurrentTableInfo to the query's context capabilities.
	 */
	template<TColumnType... ColumnTypes>
	DeprecatedFunction(5.8, "Use CurrentTableHasColumns instead. Make sure to add CurrentTableInfo to the query's context capabilities.")
	bool CurrentRowHasColumns() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return this->CurrentRowHasColumns(TConstArrayView<const UScriptStruct*>({ ColumnTypes::StaticStruct()... }));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif
CapabilityEnd(SingleRowInfo)

/**
 * Capability to provide information about the currently active batch.
 */
CapabilityStart(RowBatchInfo, EContextCapabilityFlags::SupportsBatch)
	/** Returns the number of rows in the current batch. */
	ConstFunction0(RowBatchInfo, uint32, GetBatchRowCount)
	/** Returns an view with the rows used by this batch. */
	ConstFunction0(RowBatchInfo, FRowHandleArrayView, GetBatchRowHandles)
	/** Checks if the rows in the current batch have the requested columns. */
	DeprecatedFunction(5.8, "Use CurrentTableHasColumns instead. Make sure to add CurrentTableInfo to the query's context capabilities.")
	ConstFunction1(RowBatchInfo, bool, CurrentBatchTableHasColumns, (TConstArrayView<const UScriptStruct*>, Columns))
	/** Direct access to the const addresses of a column or null if not found in this query. */
	ConstFunction1(RowBatchInfo, const void*, GetColumnBatchAddress, (const UScriptStruct*, ColumnType))
	/** Direct access to the mutable addresses of a column or null if not found in this query. */
	Function1(RowBatchInfo, void*, GetMutableColumnBatchAddress, (const UScriptStruct*, ColumnType))

#if defined(WithWrappers)
	/**
	 * Checks if the rows in the current batch have the requested columns.
	 * @deprecated Use CurrentTableHasColumns instead. Make sure to add CurrentTableInfo to the query's context capabilities.
	 */
	template<TColumnType... ColumnTypes>
	DeprecatedFunction(5.8, "Use CurrentTableHasColumns instead. Make sure to add CurrentTableInfo to the query's context capabilities.")
	bool CurrentBatchTableHasColumns() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return this->CurrentBatchTableHasColumns(TConstArrayView<const UScriptStruct*>({ ColumnTypes::StaticStruct()... }));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Utility function to iterate over each individual cell in the rows in this batch for the requested columns.
	 * Example: ForEachRow([](RowHandle Row, const ColumnA&, const ColumnB&), ArgA, ArgB); 
	 *		where ArgA is a TBatch<ColumnA> or TConstBatch<ColumnA> (a.k.a. TBatch<const ColumnA>) and 
	 *		ArgB is a TBatch<ColumnB> or TConstBatch<ColumnB> (a.k.a. TBatch<const ColumnB>) argument in the query callback function.
	 */
	template<typename CallbackType, typename... Args>
	void ForEachRow(CallbackType&& Callback, TBatch<Args>... Batches) const
	{
		TTuple<Args*...> AddressTuple = MakeTuple(Batches.GetData()...);
		for (RowHandle Row : GetBatchRowHandles())
		{
			AddressTuple.ApplyBefore([&](std::add_const_t<Args>*&... Pointers)
				{
					Callback(Row, (*Pointers++)...);
				});
		}
	}

	/**
	 * Utility function to iterate over each individual cell in the rows in this batch for the requested columns.
	 * Example: ForEachRow([](RowHandle Row, ColumnA&, const ColumnB&), ArgA, ArgB);
	 *		where ArgA is a TBatch<ColumnA> and ArgB is a TConstBatch<ColumnB> (a.k.a. TBatch<const ColumnB>) 
	 *		or TBatch<ColumnB> argument in the query callback function.
	 */
	template<typename CallbackType, typename... Args>
	void ForEachRow(CallbackType&& Callback, TBatch<Args>... Batches)
	{
		TTuple<Args*...> AddressTuple = MakeTuple(Batches.GetData()...);
		for (RowHandle Row : GetBatchRowHandles())
		{
			AddressTuple.ApplyBefore([&](Args*&... Pointers)
				{
					Callback(Row, (*Pointers++)...);
				});
		}
	}
#endif
CapabilityEnd(RowBatchInfo)

/**
 * Capability to work with dynamic columns
 */
CapabilityStart(DynamicColumnInfo, EContextCapabilityFlags::SupportsSingle | EContextCapabilityFlags::SupportsBatch)
	/**
	 * Finds the type information for a dynamic column.
	 * If the dynamic column has not been generated, then return nullptr
	 * The TemplateType may be a type derived from either FColumn or FTag, anything else will return nullptr
	 */
	ConstFunction2(DynamicColumnInfo, const UScriptStruct*, FindDynamicColumnType, (const UScriptStruct&, TemplateType), (const FName&, Identifier))
	
	/**
	 * Executes the given callback for each known dynamic column that derives from the base template provided.
	 */
	ConstFunction2(DynamicColumnInfo, void, ForEachDynamicColumnType, (const UScriptStruct&, TemplateType), (TFunctionRef<void(const UScriptStruct& Type)>, Callback))

#if defined(WithWrappers)
	/**
	 * Finds the type information for a dynamic column.
	 * If the dynamic column has not been generated, then return nullptr
	 * The DynamicColumnTemplateType may be a type derived from either FColumn or FTag, anything else will result in a compile error.
	 */
	template<TDynamicColumnTemplate DynamicColumnTemplateType>
	const UScriptStruct* FindDynamicColumnType(const FName& Identifier) const
	{
		return this->FindDynamicColumnType(*DynamicColumnTemplateType::StaticStruct(), Identifier);
	}
	
	/**
	 * Finds the type information for a dynamic column.
	 * If the dynamic column has not been generated, then return nullptr
	 * The DynamicColumnTemplateType may be a type derived from either FColumn or FTag, anything else will result in a compile error.
	 */
	template<TDynamicColumnTemplate DynamicColumnTemplateType, TVariantName Identifier>
	const UScriptStruct* FindDynamicColumnType() const
	{
		return this->FindDynamicColumnType(*DynamicColumnTemplateType::StaticStruct(),
			[]()
			{
				static FName Name = Identifier.GetName();
				return Name;
			}());
	}

	/**
	 * Finds the type information for a dynamic column.
	 * If the dynamic column has not been generated, then return nullptr
	 * The ColumnType can be any TColumn but TColumns without a name or dynamic column type will return a nullptr.
	 */
	template<TColumnDescType ColumnType>
	const UScriptStruct* FindDynamicColumnType(const ColumnType& Column) const
	{
		return Column.TypeInfo
			? this->FindDynamicColumnType(*Column.TypeInfo, Column.Identifier)
			: nullptr;
	}

	/**
	 * Executes the given callback for each known dynamic column that derives from the base template provided.
	 * The DynamicColumnTemplateType may be a type derived from either FColumn or FTag, anything else will result in a compile error. 
	 */
	template<TDynamicColumnTemplate DynamicColumnTemplateType>
	void ForEachDynamicColumnType(TFunctionRef<void(const UScriptStruct& Type)> Callback) const
	{
		this->ForEachDynamicColumnType(*DynamicColumnTemplateType::StaticStruct(), Callback);
	}

	UE_DEPRECATED(5.8, "Please one of the new versions of FindDynamicColumnType.")
	const UScriptStruct* FindDynamicColumnType(const FDynamicColumnDescription& Description) const
	{
		return Description.TemplateType ? this->FindDynamicColumnType(*Description.TemplateType, Description.Identifier) : nullptr;
	}

	UE_DEPRECATED(5.8, "Please one of the new versions of FindDynamicColumnType.")
	const UScriptStruct* FindDynamicColumn(const FDynamicColumnDescription& Description) const
	{
		return Description.TemplateType ? this->FindDynamicColumnType(*Description.TemplateType, Description.Identifier) : nullptr;
	}
#endif
CapabilityEnd(DynamicColumnInfo)
