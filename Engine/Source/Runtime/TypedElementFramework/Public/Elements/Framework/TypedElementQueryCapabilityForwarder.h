// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "Elements/Framework/TypedElementQueryCapabilities.h"
#include "Elements/Framework/TypedElementQueryContract.h"
#include "Elements/Framework/TypedElementQueryFunctionArgumentTypes.h"

namespace UE::Editor::DataStorage::Queries::Private
{
	// Class fragments use to composite a class that forwards its calls to the query context contract.
	// If the forwarder contains any virtual calls, the forwarder won't be fully inlined and will add an additional pointer dereference
	// to each call. To avoid this, the forwarder doesn't directly inherit from the capability interfaces.
	
	struct FQueryContext {};
	struct FConstQueryContext {};

	template<bool bIsConst, typename... Capabilities> struct TForwarder
	{
		static_assert(sizeof...(Capabilities) < 0,
			"Specialization for query context forwards missing. The most likely reason is the addition of a new query "
			"context capability that wasn't specialized.");
	};

	template<bool bIsConst, typename Base, typename...Capabilities>
	struct TForwarder<bIsConst, Base, Capabilities...> : Base, TForwarder<bIsConst, Capabilities...>{};

	template<bool bIsConst>
	struct TForwarder<bIsConst> : std::conditional_t<bIsConst, FConstQueryContext, FQueryContext>
	{
		TForwarder() = default;
		explicit TForwarder(IContextContract& Contract) : Contract(&Contract) {}
		explicit TForwarder(const IContextContract& Contract) requires (bIsConst) : Contract(&Contract) {}
	
	protected:
		std::conditional_t<bIsConst, const IContextContract*, IContextContract*> Contract = nullptr;
	};

#define CapabilityStart(Capability, Flags) \
	template<bool bIsConst, typename... Capabilities> \
	struct TForwarder<bIsConst, Capability , Capabilities...> : TForwarder<bIsConst, Capabilities...> \
	{ \
		TForwarder() = default; \
		explicit TForwarder(IContextContract& Contract) : TForwarder<bIsConst, Capabilities...>(Contract) {} \
		explicit TForwarder(const IContextContract& Contract) requires (bIsConst) : TForwarder<bIsConst, Capabilities...>(Contract) {}

#define ArgTypeName(Type, Name) Type Name
#define ArgName(Type, Name) Name
#define ForwardArg(Type, Name) Forward< Type >( Name )

#define Function0(Capability, Return, Function) Return Function() requires (!bIsConst) \
	{ return this->Contract-> Function (); }
#define Function1(Capability, Return, Function, Arg1) Return Function(ArgTypeName Arg1 ) requires (!bIsConst) \
	{ return this->Contract-> Function (ForwardArg Arg1 ); }
#define Function2(Capability, Return, Function, Arg1, Arg2) Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 ) requires (!bIsConst) \
	{ return this->Contract-> Function (ForwardArg Arg1 , ForwardArg Arg2 ); }
#define Function3(Capability, Return, Function, Arg1, Arg2, Arg3) Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 , ArgTypeName Arg3 ) requires (!bIsConst) \
	{ return this->Contract-> Function (ForwardArg Arg1 , ForwardArg Arg2 , ForwardArg Arg3); }
#define Function4(Capability, Return, Function, Arg1, Arg2, Arg3, Arg4) Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 , ArgTypeName Arg3 , ArgTypeName Arg4 ) requires (!bIsConst) \
	{ return this->Contract-> Function (ForwardArg Arg1 , ForwardArg Arg2 , ForwardArg Arg3 , ForwardArg Arg4); }

#define ConstFunction0(Capability, Return, Function) Return Function() const \
	{ return this->Contract-> Function (); }
#define ConstFunction1(Capability, Return, Function, Arg1) Return Function(ArgTypeName Arg1 ) const \
	{ return this->Contract-> Function (ForwardArg Arg1 ); }
#define ConstFunction2(Capability, Return, Function, Arg1, Arg2) Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 ) const \
	{ return this->Contract-> Function (ForwardArg Arg1 , ForwardArg Arg2 ); }
#define ConstFunction3(Capability, Return, Function, Arg1, Arg2, Arg3) Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 , ArgTypeName Arg3 ) const \
	{ return this->Contract-> Function (ForwardArg Arg1 , ForwardArg Arg2 , ForwardArg Arg3); }
#define ConstFunction4(Capability, Return, Function, Arg1, Arg2, Arg3, Arg4) Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 , ArgTypeName Arg3 , ArgTypeName Arg4 ) const \
	{ return this->Contract-> Function (ForwardArg Arg1 , ForwardArg Arg2 , ForwardArg Arg3 , ForwardArg Arg4); }

#define CapabilityEnd(Capability) \
	};

#define DeprecatedFunction(Version, Msg) UE_DEPRECATED(Version, Msg)

#define WithWrappers 1

#include "Elements/Framework/TypedElementQueryCapabilities.inl"

#undef ArgTypeName
#undef ArgName
#undef ForwardArg
#undef DeprecatedFunction
#undef Function0
#undef Function1
#undef Function2
#undef Function3
#undef Function4
#undef ConstFunction0
#undef ConstFunction1
#undef ConstFunction2
#undef ConstFunction3
#undef ConstFunction4
#undef CapabilityStart
#undef CapabilityEnd
#undef WithWrappers
} // namespace UE::Editor::DataStorage::Queries::Private