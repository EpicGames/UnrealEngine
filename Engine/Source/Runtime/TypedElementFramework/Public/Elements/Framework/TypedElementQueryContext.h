// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "Elements/Framework/TypedElementQueryCapabilities.h"
#include "Elements/Framework/TypedElementQueryCapabilityForwarder.h"

namespace UE::Editor::DataStorage::Queries
{
	struct IContextContract;

	template<ContextCapability... Capabilities>
	struct TCapabilityStore : Capabilities... 
	{
		static constexpr bool bIsSingle = (EnumHasAnyFlags(Capabilities::Flags, EContextCapabilityFlags::SupportsSingle) && ...);
		static constexpr bool bIsBatch = (EnumHasAnyFlags(Capabilities::Flags, EContextCapabilityFlags::SupportsBatch) && ...);
	};

	/** 
	 * Template to composite a query context using context capabilities.
	 * Any query callback that requires interacting with the editor data storage requires a context to get access. Each context capability
	 * provides access to a different kind of functionality. Some of the capabilities are mutually exclusive and not all places that accept
	 * a query callback support the same capabilities. It's therefore recommended to only include the capabilities that are needed by the
	 * query callback to improve re-usability or use a predefined query context if the query callback doesn't need to be reused.
	 */
	template<ContextCapability... CapabilityTypes>
	struct TQueryContext final : Private::TForwarder<false, CapabilityTypes...>
	{
		using Capabilities = TCapabilityStore<CapabilityTypes...>;

		static_assert(Capabilities::bIsSingle || Capabilities::bIsBatch,
			"One or more capabilities that only support single or batch processing were mixed.");

		template<ContextCapability... InCapabilityTypes>
		friend struct TForwardingQueryContext;
		
		template<ContextCapability... InCapabilityTypes>
		friend struct TForwardingConstQueryContext;

		TQueryContext() = default;
		TQueryContext(IContextContract& Contract);
	};
	
	/**
	 * Template to composite a query context using context capabilities.
	 * Any query callback that requires interacting with the editor data storage requires a context to get access. Each context capability
	 * provides access to a different kind of functionality. Some of the capabilities are mutually exclusive and not all places that accept
	 * a query callback support the same capabilities. It's therefore recommended to only include the capabilities that are needed by the
	 * query callback to improve re-usability or use a predefined query context if the query callback doesn't need to be reused.
	 */
	template<ContextCapability... CapabilityTypes>
	struct TConstQueryContext final : Private::TForwarder<true, CapabilityTypes...>
	{
		using Capabilities = TCapabilityStore<CapabilityTypes...>;

		static_assert(Capabilities::bIsSingle || Capabilities::bIsBatch,
			"One or more capabilities that only support single or batch processing were mixed.");

		template<ContextCapability... InCapabilityTypes>
		friend struct TForwardingConstQueryContext;

		TConstQueryContext() = default;
		TConstQueryContext(const IContextContract& Contract);
	};

	template<ContextCapability... CapabilityTypes>
	struct TForwardingQueryContext final : Private::TForwarder<false, CapabilityTypes...>
	{
		using Capabilities = TCapabilityStore<CapabilityTypes...>;

		static_assert(Capabilities::bIsSingle || Capabilities::bIsBatch,
			"One or more capabilities that only support single or batch processing were mixed.");

		TForwardingQueryContext() = default;
		TForwardingQueryContext(IContextContract& Contract);

		template<ContextCapability... InCapabilityTypes>
		TForwardingQueryContext(TQueryContext<InCapabilityTypes...> QueryContext);

		operator IContextContract& ();
		operator const IContextContract& () const;
	};

	template<ContextCapability... CapabilityTypes>
	struct TForwardingConstQueryContext final : Private::TForwarder<true, CapabilityTypes...>
	{
		using Capabilities = TCapabilityStore<CapabilityTypes...>;

		static_assert(Capabilities::bIsSingle || Capabilities::bIsBatch,
			"One or more capabilities that only support single or batch processing were mixed.");

		TForwardingConstQueryContext() = default;
		TForwardingConstQueryContext(const IContextContract& Contract);

		template<ContextCapability... InCapabilityTypes>
		TForwardingConstQueryContext(TQueryContext<InCapabilityTypes...> QueryContext);
		template<ContextCapability... InCapabilityTypes>
		TForwardingConstQueryContext(TConstQueryContext<InCapabilityTypes...> QueryContext);

		operator const IContextContract& () const;
	};


	//
	// Implementations.
	//
	
	//
	// TQueryContext
	//

	template<ContextCapability... CapabilityTypes>
	TQueryContext<CapabilityTypes...>::TQueryContext(IContextContract& Contract)
		: Private::TForwarder<false, CapabilityTypes...>(Contract)
	{
	}

	//
	// TConstQueryContext
	//

	template<ContextCapability... CapabilityTypes>
	TConstQueryContext<CapabilityTypes...>::TConstQueryContext(const IContextContract& Contract)
		: Private::TForwarder<true, CapabilityTypes...>(Contract)
	{
	}

	//
	// TForwardingQueryContext
	// 

	template<ContextCapability... CapabilityTypes>
	TForwardingQueryContext<CapabilityTypes...>::TForwardingQueryContext(IContextContract& Contract)
		: Private::TForwarder<false, CapabilityTypes...>(Contract)
	{
	}

	template<ContextCapability... CapabilityTypes>
	template<ContextCapability... InCapabilityTypes>
	TForwardingQueryContext<CapabilityTypes...>::TForwardingQueryContext(TQueryContext<InCapabilityTypes...> QueryContext)
		: Private::TForwarder<false, CapabilityTypes...>(*QueryContext.Contract)
	{
		static_assert(
			(std::is_base_of_v<CapabilityTypes, TCapabilityStore<InCapabilityTypes...>> && ...),
			"Forwarding query context requires more or different capabilities than the input query context provides.");
	}

	template<ContextCapability... CapabilityTypes>
	TForwardingQueryContext<CapabilityTypes...>::operator IContextContract& ()
	{
		return *this->Contract;
	}

	template<ContextCapability... CapabilityTypes>
	TForwardingQueryContext<CapabilityTypes...>::operator const IContextContract& () const
	{ 
		return *this->Contract;
	}

	//
	// TForwardingConstQueryContext
	// 

	template<ContextCapability... CapabilityTypes>
	TForwardingConstQueryContext<CapabilityTypes...>::TForwardingConstQueryContext(const IContextContract& Contract)
		: Private::TForwarder<true, CapabilityTypes...>(Contract)
	{
	}

	template<ContextCapability... CapabilityTypes>
	template<ContextCapability... InCapabilityTypes>
	TForwardingConstQueryContext<CapabilityTypes...>::TForwardingConstQueryContext(TQueryContext<InCapabilityTypes...> QueryContext)
		: Private::TForwarder<true, CapabilityTypes...>(*QueryContext.Contract)
	{
		static_assert(
			(std::is_base_of_v<CapabilityTypes, TCapabilityStore<InCapabilityTypes...>> && ...),
			"Forwarding const query context requires more or different capabilities than the input query context provides.");
	}

	template<ContextCapability... CapabilityTypes>
	template<ContextCapability... InCapabilityTypes>
	TForwardingConstQueryContext<CapabilityTypes...>::TForwardingConstQueryContext(TConstQueryContext<InCapabilityTypes...> QueryContext)
		: Private::TForwarder<true, CapabilityTypes...>(*QueryContext.Contract)
	{
		static_assert(
			(std::is_base_of_v<CapabilityTypes, TCapabilityStore<InCapabilityTypes...>> && ...),
			"Forwarding const query context requires more or different capabilities than the input query context provides.");
	}

	template<ContextCapability... CapabilityTypes>
	TForwardingConstQueryContext<CapabilityTypes...>::operator const IContextContract& () const
	{
		return *this->Contract;
	}
} // namespace UE::Editor::DataStorage::Queries
