// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "Elements/Framework/TypedElementQueryCapabilities.h"
#include "Elements/Framework/TypedElementQueryContract.h"
#include "Elements/Framework/TypedElementQueryContext.h"
#include "Elements/Framework/TypedElementQueryFunctions.h"
#include "Elements/Framework/TypedElementQueryFunctionArgumentTypes.h"

#define ArgTypeName(Type, Name) Type Name

#define CapabilityStart(Capability, Flags)

#define Function0(Capability, Return, Function) virtual Return Function() override;
#define Function1(Capability, Return, Function, Arg1) virtual Return Function(ArgTypeName Arg1 ) override;
#define Function2(Capability, Return, Function, Arg1, Arg2) virtual Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 ) override;
#define Function3(Capability, Return, Function, Arg1, Arg2, Arg3) virtual Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 , ArgTypeName Arg3 ) override;
#define Function4(Capability, Return, Function, Arg1, Arg2, Arg3, Arg4) virtual Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 , ArgTypeName Arg3 , ArgTypeName Arg4 ) override;

#define ConstFunction0(Capability, Return, Function) virtual Return Function() const override;
#define ConstFunction1(Capability, Return, Function, Arg1) virtual Return Function(ArgTypeName Arg1 ) const override;
#define ConstFunction2(Capability, Return, Function, Arg1, Arg2) virtual Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 ) const override;
#define ConstFunction3(Capability, Return, Function, Arg1, Arg2, Arg3) virtual Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 , ArgTypeName Arg3 ) const override;
#define ConstFunction4(Capability, Return, Function, Arg1, Arg2, Arg3, Arg4) virtual Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 , ArgTypeName Arg3 , ArgTypeName Arg4 ) const override;

#define CapabilityEnd(Capability)

#define DeprecatedFunction(Version, Msg) UE_DEPRECATED(Version, Msg)

#define WithWrappers 1

namespace UE::Editor::DataStorage::Queries
{
	/** 
	 * Base used for context implementations. This will verify if a context is correctly implemented.
	 * Classes that implement capabilities should use this template to automatically satisfy the requirements
	 * of a query contract. Any missing functionality in a capability implementation will be reported as a compilation 
	 * failure. This template will satisfy the IContextContract, but will implement placeholder functions for the
	 * functions of capabilities that are not supported.
	 */
	template<bool bIsConst, typename EnvironmentType, typename... ImplementationTypes>
	class TQueryContextImpl final : public IContextContract
	{
	public:
		explicit TQueryContextImpl(EnvironmentType& Environment);

		virtual ~TQueryContextImpl() override = default;

#include "Elements/Framework/TypedElementQueryCapabilities.inl"

		// Generic access

		static constexpr uint64 GetCapabilityMask();
		template<ContextCapability RequestedCapability>
		static constexpr bool SupportsCapability();
		static bool SupportsCapability(const FName& Capability);
		static constexpr bool SupportsCapability(int32 CapabilityId);
		static constexpr bool SupportsCapabilities(uint64 CapabilityMask);
		
		static void StaticReportCompatibility(uint64 CapabilityMask);
		static TConstArrayView<FName> StaticGetSupportedCapabilities();

		virtual constexpr bool CheckCompatibility(uint64 CapabilityMask) const override;
		virtual void ReportCompatibility(uint64 CapabilityMask) const override;
		virtual TConstArrayView<FName> GetSupportedCapabilities() const override;
		
		std::conditional_t<bIsConst, const EnvironmentType&, EnvironmentType&> Environment;
	};
}
// namespace UE::Editor::DataStorage::Queries

#undef ArgTypeName
#undef DeprecatedFunction
#undef Function0
#undef Function1
#undef Function2
#undef Function3
#undef Function4
#undef ConstFunction0
#undef ConstFunction1
#undef ConstFunction2
#undef ConstFunction3
#undef ConstFunction4
#undef CapabilityStart
#undef CapabilityEnd
#undef WithWrappers

#include "Elements/Framework/TypedElementQueryContextImplementation.inl"
