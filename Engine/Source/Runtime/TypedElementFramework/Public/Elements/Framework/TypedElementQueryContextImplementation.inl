// Copyright Epic Games, Inc. All Rights Reserved.

#include <type_traits>
#include "Containers/StringFwd.h"
#include "DataStorage/CommonTypes.h"
#include "Math/UnrealMathUtility.h"

namespace UE::Editor::DataStorage::Queries
{
	namespace Private
	{
		template<ContextCapability RequestedCapability, typename... ImplementationTypes>
		struct FindImplementationType
		{ 
			using Type = void;
		};

		template<ContextCapability RequestedCapability, typename ImplementationType, typename... ImplementationTypes>
		struct FindImplementationType<RequestedCapability, ImplementationType, ImplementationTypes...> {
			using Type = std::conditional_t<
				std::is_base_of_v<ImplementsContextCapability<RequestedCapability>, ImplementationType>,
				ImplementationType,
				typename FindImplementationType<RequestedCapability, ImplementationTypes...>::Type
			>;
		};

		template<ContextCapability RequestedCapability, typename... ImplementationTypes>
		using ImplementationType = typename FindImplementationType<RequestedCapability, ImplementationTypes...>::Type;
	}

	template<bool bIsConst, typename EnvironmentType, typename... ImplementationTypes>
	TQueryContextImpl<bIsConst, EnvironmentType, ImplementationTypes...>::TQueryContextImpl(EnvironmentType& Environment)
		: Environment(Environment)
	{
	}

	template<bool bIsConst, typename EnvironmentType, typename... ImplementationTypes>
	constexpr uint64 TQueryContextImpl<bIsConst, EnvironmentType, ImplementationTypes...>::GetCapabilityMask()
	{
		return ((uint64(1) << ImplementationTypes::Id) | ...);
	}

	template<bool bIsConst, typename EnvironmentType, typename... ImplementationTypes>
	template<ContextCapability RequestedCapability>
	constexpr bool TQueryContextImpl<bIsConst, EnvironmentType, ImplementationTypes...>::SupportsCapability()
	{
		return (std::is_base_of_v<ImplementsContextCapability<RequestedCapability>, ImplementationTypes> || ...);
	}

	template<bool bIsConst, typename EnvironmentType, typename... ImplementationTypes>
	bool TQueryContextImpl<bIsConst, EnvironmentType, ImplementationTypes...>::SupportsCapability(const FName& Capability)
	{
		return ((Capability == ImplementationTypes::Name) || ...);
	}

	template<bool bIsConst, typename EnvironmentType, typename... ImplementationTypes>
	constexpr bool TQueryContextImpl<bIsConst, EnvironmentType, ImplementationTypes...>::SupportsCapability(int32 CapabilityId)
	{
		return ((CapabilityId == ImplementationTypes::Id) || ...);
	}

	template<bool bIsConst, typename EnvironmentType, typename... ImplementationTypes>
	constexpr bool TQueryContextImpl<bIsConst, EnvironmentType, ImplementationTypes...>::SupportsCapabilities(uint64 CapabilityMask)
	{
		return (GetCapabilityMask() & CapabilityMask) == CapabilityMask;
	}

	// Use a macro for this since it's impossible to do with a template since the function pointer that would be needed might not exist.
	// The macro also has the added benefit that it can print the function name for better reporting.
	// CheckArgs and CallArgs are both parenthesized argument packs. CheckArgs uses lvalue argument names for the
	// requires() expression while CallArgs uses Forward<> for the actual call. Keeping them separate incorrect evaluated requires() as 
	// false when Forward<T> produces an rvalue for types that are not move-constructible (e.g. TFunctionRef).
#define ExpandArgPack(...) __VA_ARGS__
#define PrependCommaIfNeeded(...) __VA_OPT__(,) __VA_ARGS__
#define CallFunction(ConstFunction, Capability, ReturnValue, FunctionName, CheckArgs, CallArgs) \
	if constexpr (SupportsCapability<Capability>()) \
	{ \
		using ImplementationType = Private::ImplementationType<Capability, ImplementationTypes...>; \
		constexpr bool bFunctionFound = requires() \
				{ \
					{ ImplementationType:: FunctionName (Environment PrependCommaIfNeeded(ExpandArgPack CheckArgs) ) } -> UE::CSameAs< ReturnValue >; \
				}; \
		if constexpr (!bIsConst || (bIsConst && ConstFunction)) \
		{ \
			static_assert(bFunctionFound, "Function '" #FunctionName "' in capability '" #Capability "' is missing from the implementation. This is required by context implementations." ); \
			PRAGMA_DISABLE_DEPRECATION_WARNINGS \
			return ImplementationType:: FunctionName ( Environment PrependCommaIfNeeded(ExpandArgPack CallArgs) ); \
			PRAGMA_ENABLE_DEPRECATION_WARNINGS \
		} \
		else \
		{ \
			static_assert(!bFunctionFound, "Non-const function '" #FunctionName "' in capability '" #Capability "' is implemented for a const environment." ); \
			checkf(false, TEXT("Function '" #FunctionName "' in capability '" #Capability "' cannot be called from a const environment.")); \
			return UE::MakeDefault< ReturnValue >(); \
		} \
	} \
	else \
	{ \
		checkf(false, TEXT("Function '" #FunctionName "' in capability '" #Capability "' is not supported by the current query context implementation.")); \
		return UE::MakeDefault< ReturnValue >(); \
	}

#define ArgTypeName(Type, Name) Type Name
#define ArgName(Type, Name) Name
#define ForwardArg(Type, Name) Forward< Type >( Name )

#define FunctionCommon(ReturnType) \
	template<bool bIsConst, typename EnvironmentType, typename... ImplementationTypes> \
	ReturnType TQueryContextImpl<bIsConst, EnvironmentType, ImplementationTypes...>::

#define CapabilityStart(Capability, Flags)

#define Function0(Capability, Return, Function) \
	FunctionCommon(Return) Function() \
	{ \
		CallFunction(false, Capability, Return, Function, (), ()) \
	}
#define Function1(Capability, Return, Function, Arg1) \
	FunctionCommon(Return) Function(ArgTypeName Arg1) \
	{ \
		CallFunction(false, Capability, Return, Function, (ArgName Arg1), (ForwardArg Arg1)) \
	}
#define Function2(Capability, Return, Function, Arg1, Arg2) \
	FunctionCommon(Return) Function(ArgTypeName Arg1, ArgTypeName Arg2) \
	{ \
		CallFunction(false, Capability, Return, Function, (ArgName Arg1, ArgName Arg2), (ForwardArg Arg1, ForwardArg Arg2)) \
	}
#define Function3(Capability, Return, Function, Arg1, Arg2, Arg3) \
	FunctionCommon(Return) Function(ArgTypeName Arg1, ArgTypeName Arg2, ArgTypeName Arg3) \
	{ \
		CallFunction(false, Capability, Return, Function, (ArgName Arg1, ArgName Arg2, ArgName Arg3), (ForwardArg Arg1, ForwardArg Arg2, ForwardArg Arg3)) \
	}
#define Function4(Capability, Return, Function, Arg1, Arg2, Arg3, Arg4) \
	FunctionCommon(Return) Function(ArgTypeName Arg1, ArgTypeName Arg2, ArgTypeName Arg3, ArgTypeName Arg4) \
	{ \
		CallFunction(false, Capability, Return, Function, (ArgName Arg1, ArgName Arg2, ArgName Arg3, ArgName Arg4), (ForwardArg Arg1, ForwardArg Arg2, ForwardArg Arg3, ForwardArg Arg4)) \
	}

#define ConstFunction0(Capability, Return, Function) \
	FunctionCommon(Return) Function() const \
	{ \
		CallFunction(true, Capability, Return, Function, (), ()) \
	}
#define ConstFunction1(Capability, Return, Function, Arg1) \
	FunctionCommon(Return) Function(ArgTypeName Arg1) const \
	{ \
		CallFunction(true, Capability, Return, Function, (ArgName Arg1), (ForwardArg Arg1)) \
	}
#define ConstFunction2(Capability, Return, Function, Arg1, Arg2) \
	FunctionCommon(Return) Function(ArgTypeName Arg1, ArgTypeName Arg2) const \
	{ \
		CallFunction(true, Capability, Return, Function, (ArgName Arg1, ArgName Arg2), (ForwardArg Arg1, ForwardArg Arg2)) \
	}
#define ConstFunction3(Capability, Return, Function, Arg1, Arg2, Arg3) \
	FunctionCommon(Return) Function(ArgTypeName Arg1, ArgTypeName Arg2, ArgTypeName Arg3) const \
	{ \
		CallFunction(true, Capability, Return, Function, (ArgName Arg1, ArgName Arg2, ArgName Arg3), (ForwardArg Arg1, ForwardArg Arg2, ForwardArg Arg3)) \
	}
#define ConstFunction4(Capability, Return, Function, Arg1, Arg2, Arg3, Arg4) \
	FunctionCommon(Return) Function(ArgTypeName Arg1, ArgTypeName Arg2, ArgTypeName Arg3, ArgTypeName Arg4) const \
	{ \
		CallFunction(true, Capability, Return, Function, (ArgName Arg1, ArgName Arg2, ArgName Arg3, ArgName Arg4), (ForwardArg Arg1, ForwardArg Arg2, ForwardArg Arg3, ForwardArg Arg4)) \
	}

#define CapabilityEnd(Capability)

#define DeprecatedFunction(Version, Msg)

#include "Elements/Framework/TypedElementQueryCapabilities.inl"

#undef ArgTypeName
#undef ArgName
#undef ForwardArg
#undef ExpandArgPack
#undef PrependCommaIfNeeded
#undef FunctionCommon
#undef DeprecatedFunction
#undef Function0
#undef Function1
#undef Function2
#undef Function3
#undef Function4
#undef ConstFunction0
#undef ConstFunction1
#undef ConstFunction2
#undef ConstFunction3
#undef ConstFunction4
#undef CapabilityStart
#undef CapabilityEnd
#undef CallFunction

	//
	// Generic access
	
	template<bool bIsConst, typename EnvironmentType, typename... ImplementationTypes>
	void TQueryContextImpl<bIsConst, EnvironmentType, ImplementationTypes...>::StaticReportCompatibility(uint64 CapabilityMask)
	{
		uint64 MissingCapabilitiesMask = CapabilityMask & ~GetCapabilityMask();
		if (MissingCapabilitiesMask != 0)
		{
			TStringBuilder<256> Builder;

			while (MissingCapabilitiesMask)
			{
				uint64 CapabilityIndex = FMath::CountTrailingZeros64(MissingCapabilitiesMask);
				MissingCapabilitiesMask &= MissingCapabilitiesMask - 1;

				Builder.Append(TEXT("Requested query context '"));
				Builder.Append(IContextContract::GetCapabilityName(static_cast<int32>(CapabilityIndex)).ToString());
				Builder.Append(TEXT("' is not supported by query context implementation.\n"));
			}

			Builder.Append(TEXT("The following types are supported by this query context: \n"));
			for (const FName& CapabilityName : StaticGetSupportedCapabilities())
			{
				Builder.Append(TEXT("    "));
				Builder.Append(CapabilityName.ToString());
				Builder.AppendChar(TEXT('\n'));
			}
			checkf(false, TEXT("%s"), *Builder);
		}
	}

	template<bool bIsConst, typename EnvironmentType, typename... ImplementationTypes>
	TConstArrayView<FName> TQueryContextImpl<bIsConst, EnvironmentType, ImplementationTypes...>::StaticGetSupportedCapabilities()
	{
		static_assert(sizeof...(ImplementationTypes) > 0,
			"At least one context capability needs to be implemented to work with a query context contract.");

		static TArray<FName> Result = { ImplementationTypes::Name... };
		return Result;
	}

	template<bool bIsConst, typename EnvironmentType, typename... ImplementationTypes>
	constexpr bool TQueryContextImpl<bIsConst, EnvironmentType, ImplementationTypes...>::CheckCompatibility(uint64 CapabilityMask) const
	{
		return SupportsCapabilities(CapabilityMask);
	}

	template<bool bIsConst, typename EnvironmentType, typename... ImplementationTypes>
	void TQueryContextImpl<bIsConst, EnvironmentType, ImplementationTypes...>::ReportCompatibility(uint64 CapabilityMask) const
	{
		StaticReportCompatibility(CapabilityMask);
	}

	template<bool bIsConst, typename EnvironmentType, typename... ImplementationTypes>
	TConstArrayView<FName> TQueryContextImpl<bIsConst, EnvironmentType, ImplementationTypes...>::GetSupportedCapabilities() const
	{
		return StaticGetSupportedCapabilities();
	}
}
// namespace UE::Editor::DataStorage::Queries
