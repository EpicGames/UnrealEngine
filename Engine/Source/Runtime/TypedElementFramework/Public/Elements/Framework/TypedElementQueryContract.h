// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Queries/Types.h"
#include "Elements/Framework/TypedElementQueryCapabilities.h"

namespace UE::Editor::DataStorage::Queries
{
	namespace Private
	{
		template<template<typename> typename Base, template<typename> typename... Bases>
		struct IContextContractCombinerImpl : Base<IContextContractCombinerImpl<Bases>...>
		{
			virtual ~IContextContractCombinerImpl() override = default;
		};

		template<template<typename> typename Base>
		struct IContextContractCombinerImpl<Base> : Base<IContextCapability>
		{
			virtual ~IContextContractCombinerImpl() override = default;
		};

		template<template<typename> typename... Bases>
		struct IContextContractCombiner : IContextContractCombinerImpl<Bases>...
		{
			virtual ~IContextContractCombiner() override = default;

		private:
			// Work around for MSVC struggling to correctly expand fold expressions when providing the implementation for a template with 
			// templated types.
			template<typename Implementation, template<typename> typename Base>
			static bool ImplementsCapability(const FName& Capability)
			{
				return 
					FCapabilityCompatibilityCheck<Implementation, Base<IContextCapability>>::Value && 
					Base<IContextCapability>::Name == Capability;
			}

			template<typename Implementation, template<typename> typename Base>
			constexpr static bool ImplementsCapability(int32 CapabilityId)
			{
				return 
					FCapabilityCompatibilityCheck<Implementation, Base<IContextCapability>>::Value && 
					Base<IContextCapability>::Id == CapabilityId;
			}

		public:
			static FName GetCapabilityName(int32 CapabilityId)
			{
				FName Result;
				(void)(((Bases<IContextCapability>::Id == CapabilityId && (Result = Bases<IContextCapability>::Name, true)) || ...));
				return Result;
			}

			template<typename Implementation, ContextCapability RequestedCapability>
			constexpr static bool SupportsCapability()
			{
				return FCapabilityCompatibilityCheck<Implementation, RequestedCapability>::Value;
			}

			template<typename Implementation>
			static bool SupportsCapability(const FName& Capability)
			{
				return (ImplementsCapability<Implementation, Bases>(Capability) || ...);
			}

			template<typename Implementation>
			constexpr static bool SupportsCapability(int32 CapabilityId)
			{
				return (ImplementsCapability<Implementation, Bases>(CapabilityId) || ...);
			}

			template<typename Implementation>
			constexpr static int32 CountSupportedCapabilities()
			{
				return 0 + ((SupportsCapability<Implementation, Bases<IContextCapability>>() ? 1 : 0) + ...);
			}

			template<typename Implementation>
			static TConstArrayView<FName> GetSupportedCapabilityNameList()
			{
				static TConstArrayView<FName> Result = []()
					{
						constexpr int32 CapabilityCount = CountSupportedCapabilities<Implementation>();
						static_assert(CapabilityCount > 0, "At least one context capability needs to be implemented to work with a query context contract.");
						static FName Capabilities[CapabilityCount] = {};

						int32 Index = 0;
						(
							[&]
							{
								if constexpr (SupportsCapability<Implementation, Bases<IContextCapability>>())
								{
									Capabilities[Index++] = Bases<IContextCapability>::Name;
								}
							}(), ...);

						return TConstArrayView<FName>(Capabilities, CapabilityCount);
					}();
				return Result;
			}

			template<typename Implementation>
			static TConstArrayView<int32> GetSupportedCapabilityIdList()
			{
				static TConstArrayView<int32> Result = []()
					{
						constexpr int32 CapabilityCount = CountSupportedCapabilities<Implementation>();
						static_assert(CapabilityCount > 0, "At least one context capability needs to be implemented to work with a query context contract.");
						static int32 CapabilityIds[CapabilityCount] = {};

						int32 Index = 0;
						(
							[&]
							{
								if constexpr (SupportsCapability<Implementation, Bases<IContextCapability>>())
								{
									CapabilityIds[Index++] = Bases<IContextCapability>::Id;
								}
							}(), ...);

						return TConstArrayView<int32>(CapabilityIds, CapabilityCount);
					}();
				return Result;
			}

			template<typename Implementation>
			static constexpr uint64 GetSupportedCapabilityMask()
			{
				uint64 Result = 0;
				([&]
				{
					if constexpr (SupportsCapability<Implementation, Bases<IContextCapability>>())
					{
						Result = Result | (uint64(1) << Bases<IContextCapability>::Id);
					}
				}(), ...);
				return Result;
			}
		};

		//Used to remove the first template argument as that's a placeholder to allow the X-macros to work with the template.
		template<typename, template<typename> typename... Bases>
		struct IPreContextContractCombiner : IContextContractCombiner<Bases...>
		{
			virtual ~IPreContextContractCombiner() override = default;
		};
} // namespace Private

#define Function0(Capability, Return, Function)
#define Function1(Capability, Return, Function, Arg1)
#define Function2(Capability, Return, Function, Arg1, Arg2)
#define Function3(Capability, Return, Function, Arg1, Arg2, Arg3)
#define Function4(Capability, Return, Function, Arg1, Arg2, Arg3, Arg4)
#define ConstFunction0(Capability, Return, Function)
#define ConstFunction1(Capability, Return, Function, Arg1)
#define ConstFunction2(Capability, Return, Function, Arg1, Arg2)
#define ConstFunction3(Capability, Return, Function, Arg1, Arg2, Arg3)
#define ConstFunction4(Capability, Return, Function, Arg1, Arg2, Arg3, Arg4)

#define DeprecatedFunction(Version, Msg)
#define CapabilityStart(Capability, Flags) , I##Capability
#define CapabilityEnd(Capability)

	/**
	 * Contract between context and implementation to be able to communicate.
	 * The contract contains all functions for all capabilities that are available. Based on the supported capabilities
	 * an implementation may opt to only partially implement the contract, with the remaining functions asserting. On 
	 * the opposite side a context may restrict what functions on the contract can be called based on the requested
	 * capabilities. Through the query function the capabilities on both sides are kept aligned, resulting in no
	 * function on the contract being callable if they're not implemented.
	 */
	struct IContextContract : Private::IPreContextContractCombiner<int
#include "Elements/Framework/TypedElementQueryCapabilities.inl"
	>
	{
		virtual ~IContextContract() override = default;

		virtual bool constexpr CheckCompatibility(uint64 CapabilityMask) const = 0;
		template<ContextCapability... Capabilities>
		constexpr bool CheckCompatibility() const
		{
			constexpr uint64 CapabilityMask = ((uint64(1) << Capabilities::Id) | ...);
			return CheckCompatibility(CapabilityMask);
		}

		virtual void ReportCompatibility(uint64 CapabilityMask) const = 0;
		template<ContextCapability... Capabilities>
		void ReportCompatibility() const
		{
			constexpr uint64 CapabilityMask = ((uint64(1) << Capabilities::Id) | ...);
			ReportCompatibility(CapabilityMask);
		}

		virtual TConstArrayView<FName> GetSupportedCapabilities() const = 0;
	};

#undef DeprecatedFunction
#undef Function0
#undef Function1
#undef Function2
#undef Function3
#undef Function4
#undef ConstFunction0
#undef ConstFunction1
#undef ConstFunction2
#undef ConstFunction3
#undef ConstFunction4
#undef CapabilityStart
#undef CapabilityEnd
} // namespace UE::Editor::DataStorage::Queries