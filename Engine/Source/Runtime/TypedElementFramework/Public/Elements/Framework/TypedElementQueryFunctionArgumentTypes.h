// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "DataStorage/Handles.h"

namespace UE::Editor::DataStorage::Queries
{
	/**
	 * This file contains the arguments that can be used in functions for Query Callbacks. In addition
	 * to the options in this file, the following arguments can be added as well:
	 *	- TQueryContext - Addition is typically required to interact with TEDS, e.g. to add/remove columns.
	 *	- Columns - When using the single row mode any column with data can be added by reference and it will
	 *		automatically be bound. Note that the use of const is important as it will determine if read-only
	 *		or read-write access is requested from TEDS, which can have performance implications as read-only 
	 *		generally provides better parallelization.
	 */
	 
	 // Forward declarations.
	namespace Private
	{
		template<typename... Args>
		struct TArgumentInfo;
	}

	/** 
	 * Optional argument that allows controlling the flow of the query callback.
	 * This can be used to early out if for example a searched for value is found or if the number of
	 * processed rows has reached a predetermined limit.
	 */
	enum class EFlowControl
	{
		Continue, /** Continue processing with the next row/batch. */
		Break /** Stop processing at the earliest possible breakpoint. */
	};

	/**
	 * Functions that return anything other than void need to provide an interface to collect results into.
	 * It'll be up to the implementation to determine what to do with the results, such as track each registered
	 * value or accumulate all results together. Query Callback functions that operate in single mode can return
	 * the result type from the function and the result will automatically be added. Query Callback functions that
	 * work in batch mode require having a TResult<...>& argument where results are written to.
	 */
	template<typename ResultType>
	struct TResult
	{
		virtual void Add(RowHandle Row, ResultType ResultValue) = 0;
	};
	template<> struct TResult<void>{}; // Used as a placeholder for internal use.

	/** 
	 * Storage for batch of objects used in a query.
	 * Batches provide the following guarantees:
	 *	- Contiguous in memory up to the number of entries in the batch.
	 *	- All values apply to the same table.
	 */
	template<typename T>
	class TBatch
	{
		template<typename... Args>
		friend struct Private::TArgumentInfo;

	public:
		TBatch() = default;

		/** Provides direct access to the stored objects. */
		TArrayView<T> GetView(int32 BatchSize) { return TArrayView<T>(Values, BatchSize); };
		
		/** Returns a pointer to the stored column. */
		T* GetData() { return Values; }
		/** Returns a pointer to the stored column. */
		const T* GetData() const { return Values; }

	private:
		TBatch(T* Values) : Values(Values) {}
		TBatch& operator=(T* InValues) { Values = InValues; return *this; }

		T* Values = nullptr;
	};

	template<typename T>
	using TConstBatch = TBatch<const T>;
} // namespace UE::Editor::DataStorage::Queries
