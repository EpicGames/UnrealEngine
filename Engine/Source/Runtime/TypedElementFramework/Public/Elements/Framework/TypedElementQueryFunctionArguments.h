// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "Containers/ContainersFwd.h"
#include "DataStorage/CommonTypes.h"
#include "Elements/Framework/TypedElementQueryCapabilityForwarder.h"
#include "Elements/Framework/TypedElementQueryFunctionArgumentTypes.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Tuple.h"

class UScriptStruct;

namespace UE::Editor::DataStorage::Queries::Private
{
	template<typename T>
	using UnreferencedType = std::remove_reference_t<std::remove_pointer_t<T>>;
	template<typename T>
	using FoundationalType = std::remove_cv_t<UnreferencedType<T>>;
	template<typename T> concept ContextType = std::is_base_of_v<FQueryContext, T> || std::is_base_of_v<FConstQueryContext, T>;

	enum class EArgumentFlags
	{
		IsConst = 1 << 0,			// Indicates the data is immutable.
		IsMutable = 1 << 1,			// Indicates that the data can change.
		SingleArgument = 1 << 2,	// The argument can be used when a single row is being processed.
		BatchArgument = 1 << 3,		// The argument can be used when a batch of rows is being processed.

		Type_Result = 1 << 4,		// The argument is used to pass back results.
		Type_Context = 1 << 5,		// The argument contains contextual information about the current row or batch.
		Type_RowHandle = 1 << 6,	// The argument stores a row handle.
		Type_Column = 1 << 7,		// The argument stores a column, including data columns and tag columns.
		Type_FlowControl = 1 << 8,	// The argument can be used to control the next iteration step, including stopping work.
	};
	ENUM_CLASS_FLAGS(EArgumentFlags)

	// Wrapper around a pointer to act as a reference.
	template<typename T>
	struct TPointerForwarder
	{
		TPointerForwarder() = default;
		TPointerForwarder(T* Value) : Value(Value) {}
		TPointerForwarder& operator=(T* InValue) { Value = InValue; return *this; }

		operator T& () const { return *Value; }
		void Increment() { Value++; }
		
		T* Value = nullptr;
	};

	// Base descriptor for query callback function arguments. Not directly usable and requires specializations.
	template<typename T>
	struct TArgument
	{
		static_assert(!std::is_same_v<T, void>, "Unsupported argument type for query callback.");
	};

	// Void type (used as a placeholder)
	template<>
	struct TArgument<void>
	{
		constexpr static EArgumentFlags Flags = EArgumentFlags(0);
		using BaseType = void;
		using ArgumentType = void;
		using ColumnType = void;
		using PointerCastType = void;
	};

	// Result type
	template<typename T>
	struct ResultTypeInfo
	{
		static constexpr bool bIsValid = false;
		using BaseType = void;
	};
	template<typename T>
	struct ResultTypeInfo<TResult<T>>
	{
		static constexpr bool bIsValid = true;
		using BaseType = T;
	};

	template<typename T> requires ResultTypeInfo<FoundationalType<T>>::bIsValid
	struct TArgument<T>
	{
		static_assert(std::is_reference_v<T>, "Results can only be accessed by reference.");
		static_assert(!std::is_const_v<T>, "Results requires write access to function.");

		constexpr static EArgumentFlags Flags =
			EArgumentFlags::SingleArgument | EArgumentFlags::BatchArgument | EArgumentFlags::IsMutable | EArgumentFlags::Type_Result;
		using BaseType = typename ResultTypeInfo<FoundationalType<T>>::BaseType;
		using ArgumentType = TPointerForwarder<FoundationalType<T>>;
		using ColumnType = void;
		using PointerCastType = void*;
	};

	// Context type
	template<typename T> requires ContextType<FoundationalType<T>>
	struct TArgument<T>
	{
		static_assert(!std::is_reference_v<T> && !std::is_pointer_v<T>, "The context can only be passed in by value.");
		
		using BaseType = FoundationalType<T>;
		using ContextCapabilities = BaseType::Capabilities;

		constexpr static EArgumentFlags Flags =
			(ContextCapabilities::bIsSingle ? EArgumentFlags::SingleArgument : EArgumentFlags(0)) |
			(ContextCapabilities::bIsBatch ? EArgumentFlags::BatchArgument : EArgumentFlags(0)) |
			(std::is_base_of_v<FConstQueryContext, BaseType> ? EArgumentFlags::IsConst : EArgumentFlags::IsMutable) |
			EArgumentFlags::Type_Context; 
		using ArgumentType = BaseType; // Take by value because the context is a forwarder that only stores a single pointer.
		using ColumnType = void;
		using PointerCastType = void*;
	};

	// Column reference types
	template<typename T> requires TColumnType<FoundationalType<T>>
	struct TArgument<T>
	{
		static_assert(!std::is_pointer_v<T>, "Pointers to columns to work with batches are no longer supported. Use TBatch instead.");
		static_assert(std::is_reference_v<T>, "Columns can only be accessed by reference.");

		static constexpr bool bIsConst = std::is_const_v<UnreferencedType<T>>;

		constexpr static EArgumentFlags Flags =
			EArgumentFlags::SingleArgument |
			(bIsConst ? EArgumentFlags::IsConst : EArgumentFlags::IsMutable) |
			EArgumentFlags::Type_Column;
		using BaseType = FoundationalType<T>;
		using ArgumentType = TPointerForwarder<UnreferencedType<T>>;
		using ColumnType = BaseType;
		using PointerCastType = std::conditional_t<bIsConst, const BaseType*, BaseType*>;
	};

	// Column batch types
	template<typename T> requires TColumnType<T>
	struct TArgument<TBatch<T>>
	{
		static constexpr bool bIsConst = std::is_const_v<UnreferencedType<T>>;
		
		constexpr static EArgumentFlags Flags =
			EArgumentFlags::BatchArgument | 
			(bIsConst ? EArgumentFlags::IsConst : EArgumentFlags::IsMutable) |
			EArgumentFlags::Type_Column;
		using BaseType = T;
		using ArgumentType = TBatch<T>;
		using ColumnType = BaseType;
		using PointerCastType = std::conditional_t<bIsConst, const BaseType*, BaseType*>;
	};

	// Flow control
	template<typename T> requires std::is_same_v<FoundationalType<T>, EFlowControl>
	struct TArgument<T>
	{
		static_assert(std::is_reference_v<T>, "Flow control can only be accessed by reference.");
		static_assert(!std::is_const_v<T>, "Flow control requires write access to function.");

		constexpr static EArgumentFlags Flags = 
			EArgumentFlags::SingleArgument | EArgumentFlags::BatchArgument | EArgumentFlags::IsMutable | EArgumentFlags::Type_FlowControl;
		using BaseType = FoundationalType<T>;
		using ArgumentType = TPointerForwarder<BaseType>;
		using ColumnType = void;
		using PointerCastType = BaseType*;
	};

	template<int32 Index, typename... Args>
	struct TIndexToArgInfoImpl {};

	template<int32 Index, typename Front, typename... Args>
	struct TIndexToArgInfoImpl<Index, Front, Args...>
	{
		using Type = 
			std::conditional_t<Index < 0, TArgument<void>,
				std::conditional_t<Index == 0, TArgument<Front>,
					typename TIndexToArgInfoImpl<Index - 1, Args...>::Type>>;
	};

	template<int32 Index>
	struct TIndexToArgInfoImpl<Index> // Empty argument list.
	{
		using Type = TArgument<void>;
	};

	template<int32 Index, typename... Args>
	using TIndexToArgInfo = typename TIndexToArgInfoImpl<Index, Args...>::Type;

	template<typename... Args>
	struct TArgumentInfo
	{
	private:
		template<EArgumentFlags TypeFlag>
		constexpr static int32 FirstIndexOfType();

		template<EArgumentFlags Flags>
		constexpr static int32 CountFlags();

		template<int32 Index, EArgumentFlags Flags>
		constexpr static bool HasFlags();

		template<EArgumentFlags Flags>
		static TConstArrayView<const UScriptStruct*> ListColumns();
	
	public:
		static constexpr bool bIsSingle = (EnumHasAnyFlags(TArgument<Args>::Flags, EArgumentFlags::SingleArgument) && ...);
		static constexpr bool bIsBatch = (EnumHasAnyFlags(TArgument<Args>::Flags, EArgumentFlags::BatchArgument) && ...);

		static_assert(bIsSingle || bIsBatch, "One or more query callback arguments for single or batch processing were mixed.");

		using ArgumentList = TTuple<typename TArgument<Args>::ArgumentType...>;

		constexpr static int32 ResultIndex = FirstIndexOfType<EArgumentFlags::Type_Result>();
		using ResultType = typename TIndexToArgInfo<ResultIndex, Args...>::BaseType;
		template<typename T>
		static void SetResult(ArgumentList& Arguments, TResult<T>& Result);
		constexpr static int32 CountResults();

		constexpr static int32 ContextIndex = FirstIndexOfType<EArgumentFlags::Type_Context>();
		using ContextType = typename TIndexToArgInfo<ContextIndex, Args...>::BaseType;
		static void SetContext(ArgumentList& Arguments, IContextContract& Contract);
		static void SetContext(ArgumentList& Arguments, const IContextContract& Contract);
		constexpr static bool IsContextConst();
		constexpr static int32 CountContexts();

		constexpr static int32 FlowIndex = FirstIndexOfType<EArgumentFlags::Type_FlowControl>();
		static void SetFlowControl(ArgumentList& Arguments, EFlowControl& FlowControl);
		constexpr static int32 CountFlowControls();

		constexpr static int32 CountConstColumns();
		constexpr static int32 CountMutableColumns();
		static TConstArrayView<const UScriptStruct*> ListConstColumns();
		static TConstArrayView<const UScriptStruct*> ListMutableColumns();
		static void SetConstColumns(ArgumentList& Arguments, TConstArrayView<const void*> Columns);
		static void SetMutableColumns(ArgumentList& Arguments, TConstArrayView<void*> Columns);
		static void IncrementColumns(ArgumentList& Arguments);

	private:
		template<int32 Index, typename T>
		static void SetResultUnguarded(ArgumentList& Arguments, TResult<T>& Result);
		template<int32 Index>
		static void SetContextUnguarded(ArgumentList& Arguments, IContextContract& Contract);
		template<int32 Index>
		static void SetContextUnguarded(ArgumentList& Arguments, const IContextContract& Contract);
		template<int32 Index>
		static void SetFlowControlUnguarded(ArgumentList& Arguments, EFlowControl& FlowControl);
		template<typename ArgumentType>
		static void IncrementColumnUnguarded(ArgumentList& Arguments);
	};
} // namespace UE::Editor::DataStorage::Queries::Private

#include "Elements/Framework/TypedElementQueryFunctionArguments.inl"
