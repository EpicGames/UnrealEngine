// Copyright Epic Games, Inc. All Rights Reserved.

#include <functional>
#include <type_traits>

namespace UE::Editor::DataStorage::Queries::Private
{
	template<EArgumentFlags TypeFlag, int32 Index>
	constexpr int32 FirstIndexOfTypeImpl()
	{
		return -1;
	}

	template<EArgumentFlags TypeFlag, int32 Index, typename Arg, typename... Args>
	constexpr int32 FirstIndexOfTypeImpl()
	{
		if constexpr (EnumHasAnyFlags(TArgument<Arg>::Flags, TypeFlag))
		{
			return Index;
		}
		else
		{
			return FirstIndexOfTypeImpl<TypeFlag, Index + 1, Args...>();
		}
	}

	template<typename... Args>
	template<EArgumentFlags TypeFlag>
	constexpr int32 TArgumentInfo<Args...>::FirstIndexOfType()
	{
		if constexpr (sizeof...(Args) > 0)
		{
			return FirstIndexOfTypeImpl<TypeFlag, 0, Args...>();
		}
		else
		{
			return INDEX_NONE;
		}
	}

	template<typename... Args>
	template<EArgumentFlags Flags>
	constexpr int32 TArgumentInfo<Args...>::CountFlags()
	{
		return (0 + ... + (EnumHasAllFlags(TArgument<Args>::Flags, Flags) ? 1 : 0));
	}

	template<typename... Args>
	template<int32 Index, EArgumentFlags Flags>
	constexpr bool TArgumentInfo<Args...>::HasFlags()
	{
		using NthArg = typename TTupleElement<Index, TTuple<Args...>>::Type;
		return EnumHasAllFlags(TArgument<NthArg>::Flags, Flags);
	}

	template<typename... Args>
	template<typename T>
	void TArgumentInfo<Args...>::SetResult(ArgumentList& Arguments, TResult<T>& Result)
	{
		if constexpr (ResultIndex >= 0)
		{
			SetResultUnguarded<ResultIndex>(Arguments, Result);
		}
	}

	template<typename... Args>
	constexpr int32 TArgumentInfo<Args...>::CountResults()
	{
		return CountFlags<EArgumentFlags::Type_Result>();
	}

	template<typename... Args>
	void TArgumentInfo<Args...>::SetContext(ArgumentList& Arguments, IContextContract& Contract)
	{
		if constexpr (ContextIndex >= 0)
		{
			SetContextUnguarded<ContextIndex>(Arguments, Contract);
		}
	}

	template<typename... Args>
	void TArgumentInfo<Args...>::SetContext(ArgumentList& Arguments, const IContextContract& Contract)
	{
		if constexpr (ContextIndex >= 0)
		{
			SetContextUnguarded<ContextIndex>(Arguments, Contract);
		}
	}

	template<typename... Args>
	constexpr bool TArgumentInfo<Args...>::IsContextConst()
	{
		if constexpr (ContextIndex >= 0)
		{
			return HasFlags<ContextIndex, EArgumentFlags::IsConst>();
		}
		else
		{
			return false;
		}
	}

	template<typename... Args>
	constexpr int32 TArgumentInfo<Args...>::CountContexts()
	{
		return CountFlags<EArgumentFlags::Type_Context>();
	}
	
	template<typename... Args>
	void TArgumentInfo<Args...>::SetFlowControl(ArgumentList& Arguments, EFlowControl& FlowControl)
	{
		if constexpr (FlowIndex >= 0)
		{
			SetFlowControlUnguarded<FlowIndex>(Arguments, FlowControl);
		}
	}

	template<typename... Args>
	constexpr int32 TArgumentInfo<Args...>::CountFlowControls()
	{
		return CountFlags<EArgumentFlags::Type_FlowControl>();
	}

	template<typename... Args>
	constexpr int32 TArgumentInfo<Args...>::CountConstColumns()
	{
		return CountFlags<EArgumentFlags::IsConst | EArgumentFlags::Type_Column>();
	}
			
	template<typename... Args>
	constexpr int32 TArgumentInfo<Args...>::CountMutableColumns()
	{
		return CountFlags<EArgumentFlags::IsMutable | EArgumentFlags::Type_Column>();
	}

	template<typename... Args>
	template<EArgumentFlags Flags>
	TConstArrayView<const UScriptStruct*> TArgumentInfo<Args...>::ListColumns()
	{
		constexpr int32 ColumnCount = CountFlags<Flags>();
		if constexpr (ColumnCount > 0)
		{
			static TConstArrayView<const UScriptStruct*> Result =
				[&]()
				{
					static const UScriptStruct* ResultColumns[ColumnCount];
					int32 Index = 0;
					(
						[&]
						{
							if constexpr (EnumHasAllFlags(TArgument<Args>::Flags, Flags))
							{
								ResultColumns[Index++] = TArgument<Args>::ColumnType::StaticStruct();
							}
						}(), ...
					);

					return TConstArrayView<const UScriptStruct*>(ResultColumns, ColumnCount);
				}();
			return Result;
		}
		else
		{
			return TConstArrayView<const UScriptStruct*>();
		}
	}

	template<typename... Args>
	TConstArrayView<const UScriptStruct*> TArgumentInfo<Args...>::ListConstColumns()
	{
		return ListColumns<EArgumentFlags::IsConst | EArgumentFlags::Type_Column>();
	}

	template<typename... Args>
	TConstArrayView<const UScriptStruct*> TArgumentInfo<Args...>::ListMutableColumns()
	{
		return ListColumns<EArgumentFlags::IsMutable | EArgumentFlags::Type_Column>();
	}

	template<typename... Args>
	void TArgumentInfo<Args...>::SetConstColumns(ArgumentList& Arguments, TConstArrayView<const void*> Columns)
	{
		const void* const* ColumnIt = Columns.GetData();
		([&]{
				using ArgumentType = TArgument<Args>;
				if constexpr (EnumHasAllFlags(ArgumentType::Flags, EArgumentFlags::IsConst | EArgumentFlags::Type_Column))
				{
					Arguments.template Get<typename ArgumentType::ArgumentType>() = static_cast<typename ArgumentType::PointerCastType>(*ColumnIt++);
				}
			}(), ...);
	}

	template<typename... Args>
	void TArgumentInfo<Args...>::SetMutableColumns(ArgumentList& Arguments, TConstArrayView<void*> Columns)
	{
		void* const* ColumnIt = Columns.GetData();
		([&]{
				using ArgumentType = TArgument<Args>;
				if constexpr (EnumHasAllFlags(ArgumentType::Flags, EArgumentFlags::IsMutable | EArgumentFlags::Type_Column))
				{
					Arguments.template Get<typename ArgumentType::ArgumentType>() = static_cast<typename ArgumentType::PointerCastType>(*ColumnIt++);
				}
			}(), ...);
	}

	template<typename... Args>
	void TArgumentInfo<Args...>::IncrementColumns(ArgumentList& Arguments)
	{
		([&] {
			using ArgumentType = TArgument<Args>;
			if constexpr (EnumHasAllFlags(ArgumentType::Flags, EArgumentFlags::SingleArgument | EArgumentFlags::Type_Column))
			{
				IncrementColumnUnguarded<typename ArgumentType::ArgumentType>(Arguments);
			}
			}(), ...);
	}

	template<typename... Args>
	template<int32 Index, typename T>
	void TArgumentInfo<Args...>::SetResultUnguarded(ArgumentList& Arguments, TResult<T>& Result)
	{
		static_assert(std::is_same_v<T, typename TIndexToArgInfo<Index, Args...>::BaseType>, 
			"Result type used doesn't match the result type in the function's arguments.");
		Arguments.template Get<Index>() = &Result;
	}

	template<typename... Args>
	template<int32 Index>
	void TArgumentInfo<Args...>::SetContextUnguarded(ArgumentList& Arguments, IContextContract& Contract)
	{
		Arguments.template Get<Index>() = typename TTupleElement<Index, ArgumentList>::Type(Contract);
	}

	template<typename... Args>
	template<int32 Index>
	void TArgumentInfo<Args...>::SetContextUnguarded(ArgumentList& Arguments, const IContextContract& Contract)
	{
		Arguments.template Get<Index>() = typename TTupleElement<Index, ArgumentList>::Type(Contract);
	}

	template<typename... Args>
	template<int32 Index>
	void TArgumentInfo<Args...>::SetFlowControlUnguarded(ArgumentList& Arguments, EFlowControl& FlowControl)
	{
		Arguments.template Get<Index>() = &FlowControl;
	}

	template<typename... Args>
	template<typename ArgumentType>
	void TArgumentInfo<Args...>::IncrementColumnUnguarded(ArgumentList& Arguments)
	{
		Arguments.template Get<ArgumentType>().Increment();
	}
} // namespace UE::Editor::DataStorage::Queries::Private
