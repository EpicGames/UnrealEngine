// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "Containers/ContainersFwd.h"
#include "Elements/Framework/TypedElementQueryCapabilities.h"
#include "Elements/Framework/TypedElementQueryFunctionArgumentTypes.h"
#include "Elements/Framework/TypedElementRowHandleArrayView.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"

class UScriptStruct;

namespace UE::Editor::DataStorage::Queries
{
	struct IContextContract;

	struct IQueryFunctionResponse
	{
		virtual ~IQueryFunctionResponse() = default;
		
		virtual FRowHandleArrayView GetBatchRows() const = 0;
		virtual void GetConstColumns(TArrayView<const void*> ColumnsData, TConstArrayView<const UScriptStruct*> ColumnTypes) const = 0;
		// Not used in const queries.
		virtual void GetMutableColumns(TArrayView<void*> ColumnsData, TConstArrayView<const UScriptStruct*> ColumnTypes) = 0;
		// Only called when a batch is broken up in individual rows for the current active call.
		virtual void SetCurrentRow(RowHandle Row) = 0;
	};

	enum class EFunctionCallConfig
	{
		None = 0,
		/** Verifies if the function is compatible with the provided context. */
		VerifyCapabilityCompatibility = 1 << 0,
		/** If the function is not compatible with the provided context it will assert. Requires `VerifyCapabilityCompatibility`. */
		ReportCapabilityCompatibility = 1 << 1,
		/** If not all columns could be retrieved, return the default value for the return type, if applicable, and don't call the function. */
		VerifyColumns = 1 << 2,
		/** Continues to evaluate all input when Verify Columns fails. Requires `VerifyColumns`. */
		ContinueOnIncompleteRow = 1 << 3,
	};
	ENUM_CLASS_FLAGS(EFunctionCallConfig);

	/** Storage for a function that can be used as part of a query.  */
	template<typename ReturnType, bool bIsConst>
	class TQueryFunctionBase
	{
	public:
		using ContractType = std::conditional_t<bIsConst, const IContextContract&, IContextContract&>;
		using FunctionResponseType = IQueryFunctionResponse&;
		using FunctionSpecializationCallback = std::conditional_t<bIsConst,
			bool(*)(FunctionResponseType Response, TArrayView<const void*> ConstColumns),
			bool(*)(FunctionResponseType Response, TArrayView<const void*> ConstColumns, TArrayView<void*> MutableColumns)>;
	
		using WrapperFunctionType = TFunction<EFlowControl(TResult<ReturnType>& Result, ContractType Contract,
				FunctionResponseType Response, FunctionSpecializationCallback Specialization, EFlowControl IncompleteRowFlowControl)>;

		TConstArrayView<const UScriptStruct*> ConstColumnTypes;
		TConstArrayView<const UScriptStruct*> MutableColumnTypes;
		WrapperFunctionType Function;
		uint64 CapabilityMask = 0;
		bool bIsSingleRowProcessor = false;

	protected:
		template<EFunctionCallConfig Config>
		EFlowControl CallInternal(TResult<ReturnType>& Result, ContractType Contract, FunctionResponseType Response) const;
	};

	template<typename ReturnType>
	class TQueryFunction final : public TQueryFunctionBase<ReturnType, false>
	{
	public:
		using typename TQueryFunctionBase<ReturnType, false>::FunctionResponseType;

		template<EFunctionCallConfig Config>
		EFlowControl Call(TResult<ReturnType>& Result, IContextContract& Contract, FunctionResponseType Response) const
		{
			return this->template CallInternal<Config>(Result, Contract, Response);
		}

		EFlowControl Call(TResult<ReturnType>& Result, IContextContract& Contract, FunctionResponseType Response) const
		{
			return this->template CallInternal<EFunctionCallConfig::None>(Result, Contract, Response);
		}
	};

	template<>
	class TQueryFunction<void> final : public TQueryFunctionBase<void, false>
	{
	public:
		using typename TQueryFunctionBase<void, false>::FunctionResponseType;

		template<EFunctionCallConfig Config>
		EFlowControl Call(IContextContract& Contract, FunctionResponseType Response) const
		{
			TResult<void> Dummy;
			return this->template CallInternal<Config>(Dummy, Contract, Response);
		}

		EFlowControl Call(IContextContract& Contract, FunctionResponseType Response) const
		{
			return this->Call<EFunctionCallConfig::None>(Contract, Response);
		}
	};

	template<typename ReturnType>
	class TConstQueryFunction final : public TQueryFunctionBase<ReturnType, true>
	{
	public:
		using typename TQueryFunctionBase<ReturnType, true>::FunctionResponseType;

		template<EFunctionCallConfig Config>
		EFlowControl Call(TResult<ReturnType>& Result, const IContextContract& Contract, FunctionResponseType Response) const
		{
			return this->template CallInternal<Config>(Result, Contract, Response);
		}

		EFlowControl Call(TResult<ReturnType>& Result, const IContextContract& Contract, FunctionResponseType Response) const
		{
			return this->template CallInternal<EFunctionCallConfig::None>(Result, Contract, Response);
		}
	};

	template<>
	class TConstQueryFunction<void> final : public TQueryFunctionBase<void, true>
	{
	public:
		using typename TQueryFunctionBase<void, true>::FunctionResponseType;

		template<EFunctionCallConfig Config>
		EFlowControl Call(const IContextContract& Contract, FunctionResponseType Response) const
		{
			TResult<void> Dummy;
			return this->template CallInternal<Config>(Dummy, Contract, Response);
		}

		EFlowControl Call(const IContextContract& Contract, FunctionResponseType Response) const
		{
			return this->Call<EFunctionCallConfig::None>(Contract, Response);
		}
	};

	template<typename Return, FunctionType Function>
	TQueryFunction<Return> BuildQueryFunction(Function&& Callback);

	template<typename Return, FunctionType Function>
	TConstQueryFunction<Return> BuildConstQueryFunction(Function&& Callback);

} // namespace UE::Editor::DataStorage::Queries

#include "Elements/Framework/TypedElementQueryFunctions.inl"
