// Copyright Epic Games, Inc. All Rights Reserved.

#include <functional>
#include <type_traits>
#include "DataStorage/CommonTypes.h"
#include "DataStorage/Handles.h"
#include "Elements/Framework/TypedElementQueryCapabilities.h"
#include "Elements/Framework/TypedElementQueryCapabilityForwarder.h"
#include "Elements/Framework/TypedElementQueryContract.h"
#include "Elements/Framework/TypedElementQueryFunctionArguments.h"

namespace UE::Editor::DataStorage::Queries
{
	namespace Private
	{
		template<typename T> concept FunctorType = requires{ &T::operator(); };
		
		template<typename T>
		struct TFunctionInfoImpl
		{
			using ArgumentInfo = TArgumentInfo<>;

			static constexpr bool bIsValidQueryFunction = false;
			using ReturnType = void;
		};

		template<typename Return, typename... Args>
		struct TFunctionInfoImpl<Return(*)(Args...)>
		{
			using ArgumentInfo = TArgumentInfo<Args...>;

			static constexpr bool bIsValidQueryFunction = true;
			using ReturnType = Return;			
		};

		template<typename Class, typename Return, typename... Args>
		struct TFunctionInfoImpl<Return(Class::*)(Args...)>
		{
			using ArgumentInfo = TArgumentInfo<Args...>;

			static constexpr bool bIsValidQueryFunction = true;
			using ReturnType = Return;
		};

		template<typename Class, typename Return, typename... Args>
		struct TFunctionInfoImpl<Return(Class::*)(Args...) const>
		{
			using ArgumentInfo = TArgumentInfo<Args...>;

			static constexpr bool bIsValidQueryFunction = true;
			using ReturnType = Return;
		};

		template<bool IsFunctor, typename FunctionType>
		struct TFunctionInfoSelection {};

		template<typename FunctionType>
		struct TFunctionInfoSelection<true, FunctionType>
		{
			using Type = TFunctionInfoImpl<decltype(&FunctionType::operator())>;
		};

		template<typename FunctionType>
		struct TFunctionInfoSelection<false, FunctionType>
		{
			using Type = TFunctionInfoImpl<FunctionType>;
		};

		template<typename FunctionType>
		struct TFunctionInfo
		{
			using Base = TFunctionInfoSelection<FunctorType<FunctionType>, FunctionType>::Type;

			using ReturnType = typename Base::ReturnType;
			using ArgumentInfo = typename Base::ArgumentInfo;
			using ContextType = typename ArgumentInfo::ContextType;

			static constexpr bool bIsValidQueryFunction =
				Base::bIsValidQueryFunction &&
				(ArgumentInfo::bIsSingle || ArgumentInfo::bIsBatch);
			static constexpr bool bIsSingleRowProcessor = ArgumentInfo::bIsSingle;

			template<typename RequestedReturnType, bool bIsConst>
			static constexpr void StaticValidation()
			{
				static_assert(bIsValidQueryFunction, "The provided function is not compatible with a query callback. "
					"One or more arguments are possibly incompatible.");

				static_assert(ArgumentInfo::CountContexts() <= 1, "Only zero or one context can be added as an argument.");
				static_assert(ArgumentInfo::CountFlowControls() <= 1, "Only zero or one flow control arguments can be added.");
				
				if constexpr (bIsConst)
				{
					static_assert(ArgumentInfo::IsContextConst(), "Const functions require a const query context (TConstQueryContext).");
					static_assert(ArgumentInfo::CountMutableColumns() == 0, "Const functions can not bind to mutable columns.");
				}

				if constexpr (std::is_same_v<RequestedReturnType, void>)
				{
					static_assert(ArgumentInfo::ResultIndex == INDEX_NONE,
						"The query function being build has no return value and can't therefore accept a TResult.");
					static_assert(std::is_same_v<ReturnType, void>,
						"The return type of the provided query is expected to be void.");
				}
				else
				{
					if constexpr (bIsSingleRowProcessor)
					{
						if constexpr (ArgumentInfo::ResultIndex >= 0)
						{
							static_assert(ArgumentInfo::CountResults() == 1, "Only one TResult can be present.");
							static_assert(std::is_same_v<typename ArgumentInfo::ResultType, RequestedReturnType>,
								"The type used for TResult is not compatible with the result type expected for the query function.");
							static_assert(std::is_same_v<ReturnType, void>,
								"The return type of the provided query is expected to be void when a TResult is provided.");
						}
						else
						{
							static_assert(std::is_convertible_v<ReturnType, RequestedReturnType>,
								"The type returned is not compatible with the result type expected for the query function.");
						}
					}
					else
					{
						static_assert(ArgumentInfo::ResultIndex >= 0,
							"Batch processing a function that expects a result it's required that a TResult is used.");
						static_assert(ArgumentInfo::CountResults() == 1, "Only one TResult can be present.");
						static_assert(std::is_same_v<typename ArgumentInfo::ResultType, RequestedReturnType>,
							"The type used for TResult is not compatible with the result type expected for the query function.");
						static_assert(std::is_same_v<ReturnType, void>,
							"The return type of the provided query is expected to be void for batch processing functions.");
					}
				}
			}

			template<typename RequestedReturnType>
			static constexpr void SetupResult(TQueryFunction<RequestedReturnType>& Result)
			{
				if constexpr (!std::is_same_v<void, ContextType>)
				{
					Result.CapabilityMask = IContextContract::GetSupportedCapabilityMask<typename ContextType::Capabilities>();
				}
				else
				{
					Result.CapabilityMask = 0;
				}
				Result.ConstColumnTypes = ArgumentInfo::ListConstColumns();
				Result.MutableColumnTypes = ArgumentInfo::ListMutableColumns();
				Result.bIsSingleRowProcessor = bIsSingleRowProcessor;
			}

			template<typename RequestedReturnType>
			static constexpr void SetupResult(TConstQueryFunction<RequestedReturnType>& Result)
			{
				if constexpr (!std::is_same_v<void, ContextType>)
				{
					Result.CapabilityMask = IContextContract::GetSupportedCapabilityMask<typename ContextType::Capabilities>();
				}
				else
				{
					Result.CapabilityMask = 0;
				}
				Result.ConstColumnTypes = ArgumentInfo::ListConstColumns();
				Result.bIsSingleRowProcessor = bIsSingleRowProcessor;
			}
		};

		template<bool bIsConst, typename FunctionInfo, typename ArgumentInfo, typename ReturnType, typename FunctionType>
		EFlowControl CallBody(
			TResult<ReturnType>& QueryResult,
			typename TQueryFunctionBase<ReturnType, bIsConst>::ContractType Contract,
			typename TQueryFunctionBase<ReturnType, bIsConst>::FunctionResponseType Response,
			typename TQueryFunctionBase<ReturnType, bIsConst>::FunctionSpecializationCallback Specialization,
			FunctionType&& Callback,
			EFlowControl IncompleteRowFlowControl)
		{
			using ArgumentList = typename ArgumentInfo::ArgumentList;

			TConstArrayView<const UScriptStruct*> ConstColumnTypes = ArgumentInfo::ListConstColumns();
			const void** ConstColumnsData = static_cast<const void**>(FMemory_Alloca(sizeof(void*) * ConstColumnTypes.Num()));
			TArrayView<const void*> ConstColumns(ConstColumnsData, ConstColumnTypes.Num());

			TConstArrayView<const UScriptStruct*> MutableColumnTypes;
			TArrayView<void*> MutableColumns;
			if constexpr (!bIsConst)
			{
				MutableColumnTypes = ArgumentInfo::ListMutableColumns();
				void** MutableColumnsData = static_cast<void**>(FMemory_Alloca(sizeof(void*) * MutableColumnTypes.Num()));
				MutableColumns = TArrayView<void*>(MutableColumnsData, MutableColumnTypes.Num());
			}

			EFlowControl Flow = EFlowControl::Continue;

			ArgumentList Arguments;
			ArgumentInfo::SetResult(Arguments, QueryResult);
			ArgumentInfo::SetContext(Arguments, Contract);
			ArgumentInfo::SetFlowControl(Arguments, Flow);

			FRowHandleArrayView Rows = Response.GetBatchRows();
			if (!Rows.IsEmpty())
			{
				bool bSpecializationResult;
				if constexpr (bIsConst)
				{
					Response.GetConstColumns(ConstColumns, ConstColumnTypes);
					bSpecializationResult = Specialization(Response, ConstColumns);
				}
				else
				{
					Response.GetConstColumns(ConstColumns, ConstColumnTypes);
					Response.GetMutableColumns(MutableColumns, MutableColumnTypes);
					bSpecializationResult = Specialization(Response, ConstColumns, MutableColumns);
				}

				if (bSpecializationResult)
				{
					ArgumentInfo::SetConstColumns(Arguments, ConstColumns);
					if constexpr (!bIsConst)
					{
						ArgumentInfo::SetMutableColumns(Arguments, MutableColumns);
					}

					if constexpr (FunctionInfo::bIsSingleRowProcessor)
					{
						const RowHandle* It = Rows.GetData();
						const RowHandle* End = Rows.GetData() + Rows.Num();
						for (; It != End && Flow == EFlowControl::Continue; ++It)
						{
							Response.SetCurrentRow(*It);
							if constexpr (ArgumentInfo::ResultIndex >= 0 || std::is_same_v<ReturnType, void>)
							{
								Arguments.ApplyBefore(Callback);
							}
							else
							{
								QueryResult.Add(*It, Arguments.ApplyBefore(Callback));
							}
							ArgumentInfo::IncrementColumns(Arguments);
						}
					}
					else
					{
						Arguments.ApplyBefore(Callback);
					}
					return Flow;
				}
				
				return IncompleteRowFlowControl;
			}
			return EFlowControl::Break;
		}
	} // namespace Private



	//
	// TQueryFunctionBase
	//

	template<typename ReturnType, bool bIsConst>
	template<EFunctionCallConfig Config>
	EFlowControl TQueryFunctionBase<ReturnType, bIsConst>::CallInternal(TResult<ReturnType>& Result, ContractType Contract, FunctionResponseType Response) const
	{
		if constexpr (EnumHasAnyFlags(Config, EFunctionCallConfig::VerifyCapabilityCompatibility))
		{
			if (!Contract.CheckCompatibility(CapabilityMask))
			{
				if constexpr (EnumHasAnyFlags(Config, EFunctionCallConfig::ReportCapabilityCompatibility))
				{
					Contract.ReportCompatibility(CapabilityMask);
				}
				return EFlowControl::Break;
			}
		}
		
		constexpr EFlowControl IncompleteRowFlowControl = EnumHasAnyFlags(Config, EFunctionCallConfig::ContinueOnIncompleteRow) ? EFlowControl::Continue : EFlowControl::Break;

		if constexpr (bIsConst)
		{
			auto Specialization = [](FunctionResponseType Response, TArrayView<const void*> ConstColumns)
				{
					if constexpr (EnumHasAnyFlags(Config, EFunctionCallConfig::VerifyColumns))
					{
						bool bMissingColumn = false;
						for (const void* Column : ConstColumns)
						{
							bMissingColumn = bMissingColumn || Column == nullptr;
						}
						return !bMissingColumn;
					}
					else
					{
						return true;
					}
				};
			return Function(Result, Contract, Response, Specialization, IncompleteRowFlowControl);
		}
		else
		{
			auto Specialization = [](FunctionResponseType Response, TArrayView<const void*> ConstColumns, TArrayView<void*> MutableColumns)
				{
					if constexpr (EnumHasAnyFlags(Config, EFunctionCallConfig::VerifyColumns))
					{
						bool bMissingColumn = false;
						for (const void* Column : ConstColumns)
						{
							bMissingColumn = bMissingColumn || Column == nullptr;
						}
						for (void* Column : MutableColumns)
						{
							bMissingColumn = bMissingColumn || Column == nullptr;
						}
						return !bMissingColumn;
					}
					else
					{
						return true;
					}
				};
			return Function(Result, Contract, Response, Specialization, IncompleteRowFlowControl);
		}
	}

	template<typename Return, FunctionType Function>
	TQueryFunction<Return> BuildQueryFunction(Function&& Callback)
	{
		using FunctionInfo = Private::TFunctionInfo<Function>;
		using ArgumentInfo = typename FunctionInfo::ArgumentInfo;
		
		TQueryFunction<Return> Result;
		FunctionInfo::template StaticValidation<Return, false>();
		FunctionInfo::SetupResult(Result);

		Result.Function =
			[LocalCallback = Forward<Function>(Callback)](
				TResult<Return>& QueryResult,
				TQueryFunction<Return>::ContractType Contract,
				TQueryFunction<Return>::FunctionResponseType Response,
				TQueryFunction<Return>::FunctionSpecializationCallback Specialization,
				EFlowControl IncompleteRowFlowControl)
			{
				return Private::CallBody<false, FunctionInfo, ArgumentInfo, Return>(QueryResult, Contract, Response, Specialization, LocalCallback, IncompleteRowFlowControl);
			};
		return Result;
	}

	template<typename Return, FunctionType Function>
	TConstQueryFunction<Return> BuildConstQueryFunction(Function&& Callback)
	{
		using FunctionInfo = Private::TFunctionInfo<Function>;
		using ArgumentInfo = typename FunctionInfo::ArgumentInfo;

		TConstQueryFunction<Return> Result;
		FunctionInfo::template StaticValidation<Return, true>();
		FunctionInfo::SetupResult(Result);

		Result.Function =
			[LocalCallback = Forward<Function>(Callback)](
				TResult<Return>& QueryResult,
				TConstQueryFunction<Return>::ContractType Contract,
				TConstQueryFunction<Return>::FunctionResponseType Response,
				TConstQueryFunction<Return>::FunctionSpecializationCallback Specialization,
				EFlowControl IncompleteRowFlowControl)
			{
				return Private::CallBody<true, FunctionInfo, ArgumentInfo, Return>(QueryResult, Contract, Response, Specialization, LocalCallback, IncompleteRowFlowControl);
			};
		return Result;
	}
} // namespace UE::Editor::DataStorage::Queries
