// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Set.h"
#include "Containers/SortedMap.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Elements/Framework/TypedElementData.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "Elements/Framework/TypedElementId.h"
#include "Elements/Framework/TypedElementLimits.h"
#include "Elements/Framework/TypedElementList.h"
#include "Elements/Framework/TypedElementListFwd.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformCrt.h"
#include "Logging/LogVerbosity.h"
#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Templates/Tuple.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/Interface.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptInterface.h"
#include "UObject/Stack.h"
#include "UObject/UObjectGlobals.h"

#include <type_traits>

#include "TypedElementRegistry.generated.h"

/**
 * Registry of element types and their associated interfaces, along with the elements that represent their instances.
 */
UCLASS(Transient, MinimalAPI)
class UTypedElementRegistry : public UObject
{
	GENERATED_BODY()

public:
	TYPEDELEMENTFRAMEWORK_API UTypedElementRegistry();

	//~ UObject interface
	TYPEDELEMENTFRAMEWORK_API virtual void FinishDestroy() override;
	static TYPEDELEMENTFRAMEWORK_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	/**
	 * Initialize the singleton instance of the registry used in most cases.
	 */
	static TYPEDELEMENTFRAMEWORK_API void Private_InitializeInstance();

	/**
	 * Shutdown the singleton instance of the registry used in most cases.
	 */
	static TYPEDELEMENTFRAMEWORK_API void Private_ShutdownInstance();

	/**
	 * Get the singleton instance of the registry used in most cases.
	 */
	UFUNCTION(BlueprintPure, DisplayName="Get Default Typed Element Registry", Category = "TypedElementFramework|Registry", meta=(ScriptName="GetDefaultTypedElementRegistry"))
	static TYPEDELEMENTFRAMEWORK_API UTypedElementRegistry* GetInstance();

	/**
	 * Event fired when references to one element should be replaced with a reference to a different element.
	 */
	using FOnElementReplaced_Payload = TArrayView<const TTuple<FTypedElementHandle, FTypedElementHandle>>; // Alias for the macro below
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnElementReplaced, FOnElementReplaced_Payload /*ReplacedElements*/);
	FOnElementReplaced& OnElementReplaced()
	{
		return OnElementReplacedDelegate;
	}

	/**
	 * Event fired when an element has been internally updated and data cached from it should be refreshed.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnElementUpdated, TArrayView<const FTypedElementHandle> /*UpdatedElements*/);
	FOnElementUpdated& OnElementUpdated()
	{
		return OnElementUpdatedDelegate;
	}

	/**
	 * Event fired prior to processing any elements that were previously marked for deferred destruction.
	 * @see ProcessDeferredElementsToDestroy.
	 */
	FSimpleMulticastDelegate& OnProcessingDeferredElementsToDestroy()
	{
		return OnProcessingDeferredElementsToDestroyDelegate;
	}

	/**
	 * Get the element type ID for the associated element type name, if any.
	 * @return The element type ID, or 0 if the given name wasn't registered.
	 */
	inline FTypedHandleTypeId GetRegisteredElementTypeId(const FName InElementTypeName) const
	{
		const FRegisteredElementType* RegisteredElementType = GetRegisteredElementTypeFromName(InElementTypeName);
		return RegisteredElementType ? RegisteredElementType->TypeId : 0;
	}

	/**
	 * Get the element type name for the associated element type ID, if any.
	 * @return The element type name, or None if the given ID wasn't registered.
	 */
	inline FName GetRegisteredElementTypeName(const FTypedHandleTypeId InElementTypeId) const
	{
		const FRegisteredElementType* RegisteredElementType = GetRegisteredElementTypeFromId(InElementTypeId);
		return RegisteredElementType ? RegisteredElementType->TypeName : FName();
	}

	/**
	 * Register an element type that doesn't require any additional payload data.
	 * @param bSupportScriptHandles Does this element type support script handles. If true the elements of this type can only be destroyed from the game thread
	 */
	inline void RegisterElementType(const FName InElementTypeName, bool bSupportScriptHandles)
	{
		if (bSupportScriptHandles)
		{
			RegisterElementTypeImpl(InElementTypeName, MakeUnique<TRegisteredElementType<void, true>>());
		}
		else
		{
			RegisterElementTypeImpl(InElementTypeName, MakeUnique<TRegisteredElementType<void, false>>());
		}
	}

	/**
	 * Register an element type that has additional payload data.
	 * Note if bSupportScriptHandles is true, elements of this type should only be destroyed from the game thread
	 */
	template <typename ElementDataType, bool bSupportScriptHandles>
	inline void RegisterElementType(const FName InElementTypeName)
	{
		RegisterElementTypeImpl(InElementTypeName, MakeUnique<TRegisteredElementType<ElementDataType, bSupportScriptHandles>>());
	}

	/**
	 * Register that an element interface is supported for the given type, which must have previously been registered via RegisterElementType.
	 */
	template <typename BaseInterfaceType>
	inline void RegisterElementInterface(const FName InElementTypeName, UObject* InElementInterface, const bool InAllowOverride = false)
	{
		checkf(InElementInterface->GetClass()->ImplementsInterface(BaseInterfaceType::UClassType::StaticClass()), TEXT("The InElementInterface pass must implement the interface to register."));
		RegisterElementInterfaceImpl(InElementTypeName, InElementInterface, BaseInterfaceType::UClassType::StaticClass(), InAllowOverride);
	}

	/**
	 * Get the element interface supported by the given type, or null if there is no support for this interface.
	 */
	template <typename BaseInterfaceType>
	inline BaseInterfaceType* GetElementInterface(const FTypedHandleTypeId InElementTypeId) const
	{
		return Cast<BaseInterfaceType>(GetElementInterfaceImpl(InElementTypeId, BaseInterfaceType::UClassType::StaticClass()));
	}

	/**
	 * Get the element interface supported by the given handle, or null if there is no support for this interface.
	 */
	template <typename BaseInterfaceType>
	inline BaseInterfaceType* GetElementInterface(const FTypedElementHandle& InElementHandle) const
	{
		return Cast<BaseInterfaceType>(GetElementInterfaceImpl(InElementHandle.GetId().GetTypeId(), BaseInterfaceType::UClassType::StaticClass()));
	}

	/**
	 * Get the element interface supported by the given handle, or null if there is no support for this interface.
	 */
	inline UObject* GetElementInterface(const FTypedElementHandle& InElementHandle, const TSubclassOf<UInterface> InBaseInterfaceType) const
	{
		return GetElementInterfaceImpl(InElementHandle.GetId().GetTypeId(), InBaseInterfaceType);
	}

	/**
	 * Get the element interface supported by the given handle, or null if there is no support for this interface or if the handle is invalid.
	 */
	UFUNCTION(BlueprintPure, Category="TypedElementFramework|Registry")
	inline UObject* GetElementInterface(const FScriptTypedElementHandle& InElementHandle, const TSubclassOf<UInterface> InBaseInterfaceType) const
	{
		return GetElementInterfaceImpl(InElementHandle.GetId().GetTypeId(), InBaseInterfaceType);
	}

	/**
	 * Create an element that doesn't require any additional payload data.
	 * @note The associated handle ID should be something that can externally be used to uniquely identify this element, until DestroyElementHandle is called on this handle.
	 */
	inline FTypedElementOwner CreateElement(const FName InElementTypeName, const FTypedHandleElementId InElementId)
	{
		return CreateElementImpl<void>(InElementTypeName, InElementId);
	}

	/**
	 * Create an element that has additional payload data.
	 * @note Allocation of the payload data and the associated handle ID are managed internally, and the data will remain valid until DestroyElementHandle is called on this handle.
	 */
	template <typename ElementDataType>
	inline TTypedElementOwner<ElementDataType> CreateElement(const FName InElementTypeName)
	{
		return CreateElementImpl<ElementDataType>(InElementTypeName, INDEX_NONE);
	}

	/**
	 * Destroy an element.
	 * @note Destruction is deferred until the next call to ProcessDeferredElementsToDestroy.
	 */
	inline void DestroyElement(FTypedElementOwner& InOutElementOwner)
	{
		return DestroyElementImpl<void>(InOutElementOwner);
	}

	/**
	 * Destroy an element.
	 * @note Destruction is deferred until the next call to ProcessDeferredElementsToDestroy.
	 */
	template <typename ElementDataType>
	inline void DestroyElement(TTypedElementOwner<ElementDataType>& InOutElementOwner)
	{
		return DestroyElementImpl<ElementDataType>(InOutElementOwner);
	}

	/**
	 * Process any elements that were previously marked for deferred destruction.
	 * @note This is automatically called at the end of each frame, but may also be called manually.
	 * @note This is automatically called post-GC, unless auto-GC destruction has been disabled (@see FDisableElementDestructionOnGC).
	 */
	TYPEDELEMENTFRAMEWORK_API void ProcessDeferredElementsToDestroy();

	/**
	 * Release an element ID that was previously acquired from an existing handle.
	 */
	TYPEDELEMENTFRAMEWORK_API void ReleaseElementId(FTypedElementId& InOutElementId);

	/**
	 * Get an element handle from its minimal ID.
	 */
	TYPEDELEMENTFRAMEWORK_API FTypedElementHandle GetElementHandle(const FTypedElementId& InElementId) const;

	/**
	 * Get an element that implements the given interface from its minimal ID.
	 */
	inline FTypedElement GetElement(const FTypedElementId& InElementId, const TSubclassOf<UInterface>& InBaseInterfaceType) const
	{
		FTypedElement Element;
		GetElementImpl(InElementId, InBaseInterfaceType, Element);
		return Element;
	}

	/**
	 * Get an element that implements the given interface from its minimal ID.
	 */
	template <typename BaseInterfaceType>
	inline TTypedElement<BaseInterfaceType> GetElement(const FTypedElementId& InElementId) const
	{
		TTypedElement<BaseInterfaceType> Element;
		GetElementImpl(InElementId, BaseInterfaceType::UClassType::StaticClass(), Element);
		return Element;
	}

	/**
	 * Get an element that implements the given interface from its handle.
	 */
	inline FTypedElement GetElement(const FTypedElementHandle& InElementHandle, const TSubclassOf<UInterface>& InBaseInterfaceType) const
	{
		FTypedElement Element;
		GetElementImpl(InElementHandle, InBaseInterfaceType, Element);
		return Element;
	}

	/**
	 * Get an element that implements the given interface from its handle.
	 */
	template <typename BaseInterfaceType>
	inline TTypedElement<BaseInterfaceType> GetElement(const FTypedElementHandle& InElementHandle) const
	{
		TTypedElement<BaseInterfaceType> Element;
		GetElementImpl(InElementHandle, BaseInterfaceType::UClassType::StaticClass(), Element);
		return Element;
	}

	/**
	 * Create an empty list of elements associated with this registry.
	 */
	inline FTypedElementListRef CreateElementList()
	{
		return FTypedElementList::Private_CreateElementList(this);
	}

	/**
	 * Create an empty list of script elements associated with this registry.
	 */
	inline FScriptTypedElementListRef CreateScriptElementList()
	{
		return FScriptTypedElementList::Private_CreateElementList(this);
	}

	/**
	 * Create an empty list of elements associated with this registry, populated from the given minimal IDs that are valid.
	 */
	TYPEDELEMENTFRAMEWORK_API FTypedElementListRef CreateElementList(TArrayView<const FTypedElementId> InElementIds);

	/**
	 * Create an empty list of elements associated with this registry, populated from the given handles that are valid.
	 */
	TYPEDELEMENTFRAMEWORK_API FTypedElementListRef CreateElementList(TArrayView<const FTypedElementHandle> InElementHandles);

	/**
	 * Create an empty list of elements associated with this registry, populated from the given owners that are valid.
	 */
	template <typename ElementDataType>
	inline FTypedElementListRef CreateElementList(const TArray<TTypedElementOwner<ElementDataType>>& InElementOwners)
	{
		return CreateElementList(MakeArrayView(InElementOwners));
	}

	/**
	 * Create an empty list of elements associated with this registry, populated from the given owners that are valid.
	 */
	template <typename ElementDataType>
	FTypedElementListRef CreateElementList(TArrayView<const TTypedElementOwner<ElementDataType>> InElementOwners)
	{
		FTypedElementListRef ElementList = CreateElementList();
		ElementList->Append(InElementOwners);
		return ElementList;
	}

	/**
	 * Extract a string that contains name of the type elements registered.
	 * For each type it also says what interface are registered and the path of the class that implement them
	 */
	TYPEDELEMENTFRAMEWORK_API FString RegistredElementTypesAndInterfacesToString() const;

	/**
	 * Create a script handle. This should only be use for the script exposure apis.
	 */
	inline FScriptTypedElementHandle CreateScriptHandle(const FTypedElementId& InElementId)
	{
		if (InElementId == FTypedElementId::Unset)
		{
			return FScriptTypedElementHandle();
		}

		FRegisteredElementType* RegisteredElementType = GetRegisteredElementTypeFromId(InElementId.GetTypeId());
		if (!RegisteredElementType)
		{
			FFrame::KismetExecutionMessage(TEXT("Element type ID has not been registered to this registry!"), ELogVerbosity::Error);
			return FScriptTypedElementHandle();
		}
	
		FScriptTypedElementHandle ScriptTypedElementHandle;
		ScriptTypedElementHandle.Private_Initialize(RegisteredElementType->GetDataForScriptElement(InElementId.GetElementId()));
		return ScriptTypedElementHandle;
	}

	void Private_OnElementListCreated(FTypedElementList* InElementList)
	{
		FScopeLock ActiveElementListsLock(&ActiveElementListsCS);
		ActiveElementLists.Add(InElementList);
	}

	void Private_OnElementListDestroyed(FTypedElementList* InElementList)
	{
		FScopeLock ActiveElementListsLock(&ActiveElementListsCS);
		ActiveElementLists.Remove(InElementList);
	}

	
	void Private_OnElementListCreated(FScriptTypedElementList* InElementList)
	{
		// Script type element are not thread safe so no need for a critical section
		ActiveScriptElementLists.Add(InElementList);
	}

	void Private_OnElementListDestroyed(FScriptTypedElementList* InElementList)
	{
		// Script type element are not thread safe so no need for a critical section
		ActiveScriptElementLists.Remove(InElementList);
	}

	// Note: Access for FTypedElementList
	inline void Private_GetElementImpl(const FTypedElementHandle& InElementHandle, const TSubclassOf<UInterface>& InBaseInterfaceType, FTypedElement& OutElement) const
	{
		GetElementImpl(InElementHandle, InBaseInterfaceType, OutElement);
	}

	/** Struct to disable auto-GC reference collection within a scope */
	struct FDisableElementDestructionOnGC
	{
	public:
		explicit FDisableElementDestructionOnGC(UTypedElementRegistry* InRegistry)
			: Registry(InRegistry)
		{
			checkf(Registry, TEXT("FDisableElementDestructionOnGC must be used with a valid registry!"));
			Registry->IncrementDisableElementDestructionOnGCCount();
		}

		~FDisableElementDestructionOnGC()
		{
			Registry->DecrementDisableElementDestructionOnGCCount();
		}

		FDisableElementDestructionOnGC(const FDisableElementDestructionOnGC&) = delete;
		FDisableElementDestructionOnGC& operator=(const FDisableElementDestructionOnGC&) = delete;

		FDisableElementDestructionOnGC(FDisableElementDestructionOnGC&&) = delete;
		FDisableElementDestructionOnGC& operator=(FDisableElementDestructionOnGC&&) = delete;

	private:
		UTypedElementRegistry* Registry = nullptr;
	};

private:
	struct FRegisteredElementType
	{
		virtual ~FRegisteredElementType() = default;

		virtual FTypedElementInternalData& AddDataForElement(FTypedHandleElementId& InOutElementId) = 0;
		virtual void RemoveDataForElement(const FTypedHandleElementId InElementId, const FTypedElementInternalData* InExpectedDataPtr, const bool bDefer) = 0;
		virtual const FTypedElementInternalData& GetDataForElement(const FTypedHandleElementId InElementId) const = 0;
		virtual FScriptTypedElementInternalDataPtr GetDataForScriptElement(const FTypedHandleElementId InElementId) = 0;
		virtual void ProcessDeferredElementsToRemove() = 0;
		virtual void SetDataTypeId(const FTypedHandleTypeId InTypeId) = 0;
		virtual FTypedHandleTypeId GetDataTypeId() const = 0;
		virtual FName GetDataTypeName() const = 0;

		FTypedHandleTypeId TypeId = 0;
		FName TypeName;

		TSortedMap<FName, TObjectPtr<UObject>, FDefaultAllocator, FNameFastLess> Interfaces;
	};

	template <typename ElementDataType, bool bSupportScriptHandles>
	struct TRegisteredElementType : public FRegisteredElementType
	{
		virtual ~TRegisteredElementType() = default;
		
		virtual FTypedElementInternalData& AddDataForElement(FTypedHandleElementId& InOutElementId) override
		{
			return HandleDataStore.AddDataForElement(TypeId, InOutElementId);
		}

		virtual void RemoveDataForElement(const FTypedHandleElementId InElementId, const FTypedElementInternalData* InExpectedDataPtr, const bool bDefer) override
		{
			if constexpr (bSupportScriptHandles)
			{
				checkSlow(IsInGameThread());
				HandleDataStore.DisableScriptHandlesForElement(InElementId);
			}

			if (bDefer)
			{
				InExpectedDataPtr->StoreDestructionRequestCallstack();

				FScopeLock DeferredElementsToRemoveLock(&DeferredElementsToRemoveCS);
				DeferredElementsToRemove.Add(MakeTuple(InElementId, InExpectedDataPtr));
			}
			else
			{
				HandleDataStore.RemoveDataForElement(InElementId, InExpectedDataPtr);
			}
		}

		virtual FScriptTypedElementInternalDataPtr GetDataForScriptElement(const FTypedHandleElementId InElementId) override
		{
			if constexpr (bSupportScriptHandles)
			{
				return HandleDataStore.GetInternalDataForScriptHandle(InElementId);
			}
			else
			{
				return {};
			}
		}

		virtual const FTypedElementInternalData& GetDataForElement(const FTypedHandleElementId InElementId) const override
		{
			return HandleDataStore.GetDataForElement(InElementId);
		}

		virtual void ProcessDeferredElementsToRemove() override
		{
			FScopeLock DeferredElementsToRemoveLock(&DeferredElementsToRemoveCS);
			for (const FDeferredElementToRemove& DeferredElementToRemove : DeferredElementsToRemove)
			{
				HandleDataStore.RemoveDataForElement(DeferredElementToRemove.Get<0>(), DeferredElementToRemove.Get<1>());
			}
			DeferredElementsToRemove.Reset();
		}

		virtual void SetDataTypeId(const FTypedHandleTypeId InTypeId) override
		{
			TTypedElementInternalDataStore<ElementDataType>::SetStaticDataTypeId(InTypeId);
		}

		virtual FTypedHandleTypeId GetDataTypeId() const override
		{
			return TTypedElementInternalDataStore<ElementDataType>::StaticDataTypeId();
		}

		virtual FName GetDataTypeName() const override
		{
			return TTypedElementInternalDataStore<ElementDataType>::StaticDataTypeName();
		}

		TTypedElementInternalDataStore<ElementDataType> HandleDataStore;

		using FDeferredElementToRemove = TTuple<FTypedHandleElementId, const FTypedElementInternalData*>;
		FCriticalSection DeferredElementsToRemoveCS;
		TArray<FDeferredElementToRemove> DeferredElementsToRemove;
	};

	TYPEDELEMENTFRAMEWORK_API void RegisterElementTypeImpl(const FName InElementTypeName, TUniquePtr<FRegisteredElementType>&& InRegisteredElementType);
	TYPEDELEMENTFRAMEWORK_API void RegisterElementInterfaceImpl(const FName InElementTypeName, UObject* InElementInterface, const TSubclassOf<UInterface>& InBaseInterfaceType, const bool InAllowOverride);
	TYPEDELEMENTFRAMEWORK_API UObject* GetElementInterfaceImpl(const FTypedHandleTypeId InElementTypeId, const TSubclassOf<UInterface>& InBaseInterfaceType) const;

	template <typename ElementDataType>
	TTypedElementOwner<ElementDataType> CreateElementImpl(const FName InElementTypeName, const FTypedHandleElementId InElementId)
	{
		FRegisteredElementType* RegisteredElementType = GetRegisteredElementTypeFromName(InElementTypeName);
		checkf(RegisteredElementType, TEXT("Element type '%s' has not been registered!"), *InElementTypeName.ToString());

		checkf(RegisteredElementType->GetDataTypeId() == TTypedElementInternalDataStore<ElementDataType>::StaticDataTypeId(), TEXT("Element type '%s' uses '%s' as its handle data type, but '%s' was requested!"), *InElementTypeName.ToString(), *RegisteredElementType->GetDataTypeName().ToString(), *TTypedElementInternalDataStore<ElementDataType>::StaticDataTypeName().ToString());

		FTypedHandleElementId NewElementId = InElementId;
		FTypedElementInternalData& NewElementData = RegisteredElementType->AddDataForElement(NewElementId);

		TTypedElementOwner<ElementDataType> ElementOwner;
		ElementOwner.Private_InitializeAddRef(static_cast<TTypedElementInternalData<ElementDataType>&>(NewElementData));

		return ElementOwner;
	}

	template <typename ElementDataType>
	void DestroyElementImpl(TTypedElementOwner<ElementDataType>& InOutElementOwner)
	{
		FRegisteredElementType* RegisteredElementType = GetRegisteredElementTypeFromId(InOutElementOwner.GetId().GetTypeId());
		checkf(RegisteredElementType, TEXT("Element type ID '%d' has not been registered!"), InOutElementOwner.GetId().GetTypeId());

		RegisteredElementType->RemoveDataForElement(InOutElementOwner.GetId().GetElementId(), InOutElementOwner.Private_GetInternalData(), /*bDefer*/true);
		InOutElementOwner.Private_DestroyNoRef();
	}

	template <typename BaseInterfaceType>
	void GetElementImpl(const FTypedElementId& InElementId, const TSubclassOf<UInterface>& InBaseInterfaceType, TTypedElement<BaseInterfaceType>& OutElement) const
	{
		OutElement.Private_DestroyReleaseRef();

		if (InElementId)
		{
			FRegisteredElementType* RegisteredElementType = GetRegisteredElementTypeFromId(InElementId.GetTypeId());
			checkf(RegisteredElementType, TEXT("Element type ID '%d' has not been registered!"), InElementId.GetTypeId());

			if (UObject* InterfaceObject = RegisteredElementType->Interfaces.FindRef(InBaseInterfaceType->GetFName()))
			{
				const FTypedElementInternalData& TypedElementInternalData = RegisteredElementType->GetDataForElement(InElementId.GetElementId());
				if constexpr (std::is_void<BaseInterfaceType>::value)
				{ 
					OutElement.Private_InitializeAddRef(TypedElementInternalData, InterfaceObject->GetInterfaceAddress(InBaseInterfaceType));
				}
				else
				{
					OutElement.Private_InitializeAddRef(TypedElementInternalData, Cast<BaseInterfaceType>(InterfaceObject));
				}
			}
		}
	}

	template <typename BaseInterfaceType>
	void GetElementImpl(const FTypedElementHandle& InElementHandle, const TSubclassOf<UInterface>& InBaseInterfaceType, TTypedElement<BaseInterfaceType>& OutElement) const
	{
		OutElement.Private_DestroyReleaseRef();

		if (InElementHandle)
		{
			FRegisteredElementType* RegisteredElementType = GetRegisteredElementTypeFromId(InElementHandle.GetId().GetTypeId());

			ensureMsgf(RegisteredElementType, TEXT("Element type ID '%d' has not been registered!"), InElementHandle.GetId().GetTypeId());
			if (!RegisteredElementType)
			{
				// Stale handle
				return;
			}

			if (UObject* InterfaceObject = RegisteredElementType->Interfaces.FindRef(InBaseInterfaceType->GetFName()))
			{
				const FTypedElementInternalData& TypedElementInternalData = *InElementHandle.Private_GetInternalData();
				if constexpr (std::is_void<BaseInterfaceType>::value)
				{
					OutElement.Private_InitializeAddRef(TypedElementInternalData, InterfaceObject->GetInterfaceAddress(InBaseInterfaceType));
				}
				else
				{
					OutElement.Private_InitializeAddRef(TypedElementInternalData, Cast<BaseInterfaceType>(InterfaceObject));
				}
			}
		}
	}

	void AddRegisteredElementType(TUniquePtr<FRegisteredElementType>&& InRegisteredElementType)
	{
		checkf(InRegisteredElementType->TypeId > 0, TEXT("Element type ID was unassigned!"));
		checkf(!GetRegisteredElementTypeFromId(InRegisteredElementType->TypeId), TEXT("Element type '%d' has already been registered!"), InRegisteredElementType->TypeId);
		checkf(!GetRegisteredElementTypeFromName(InRegisteredElementType->TypeName), TEXT("Element type '%s' has already been registered!"), *InRegisteredElementType->TypeName.ToString());

		{
			UE::TWriteScopeLock RegisteredElementTypesLock(RegisteredElementTypesRW);

			LLM_SCOPE(ELLMTag::EngineMisc);
			RegisteredElementTypesNameToId.Add(InRegisteredElementType->TypeName, InRegisteredElementType->TypeId);
			RegisteredElementTypes[InRegisteredElementType->TypeId - 1] = MoveTemp(InRegisteredElementType);
		}
	}

	FRegisteredElementType* GetRegisteredElementTypeFromId(const FTypedHandleTypeId InTypeId) const
	{
		UE::TReadScopeLock RegisteredElementTypesLock(RegisteredElementTypesRW);

		return InTypeId > 0
			? RegisteredElementTypes[InTypeId - 1].Get()
			: nullptr;
	}

	FRegisteredElementType* GetRegisteredElementTypeFromName(const FName& InTypeName) const
	{
		UE::TReadScopeLock RegisteredElementTypesLock(RegisteredElementTypesRW);

		if (const FTypedHandleTypeId* TypeId = RegisteredElementTypesNameToId.Find(InTypeName))
		{
			return RegisteredElementTypes[(*TypeId) - 1].Get();
		}

		return nullptr;
	}

	TYPEDELEMENTFRAMEWORK_API void NotifyElementListPendingChanges();

	TYPEDELEMENTFRAMEWORK_API void OnBeginFrame();
	TYPEDELEMENTFRAMEWORK_API void OnEndFrame();
	TYPEDELEMENTFRAMEWORK_API void OnPostGarbageCollect();

	inline void IncrementDisableElementDestructionOnGCCount()
	{
		checkf(DisableElementDestructionOnGCCount != MAX_uint8, TEXT("DisableElementDestructionOnGCCount overflow!"));
		++DisableElementDestructionOnGCCount;
	}

	inline void DecrementDisableElementDestructionOnGCCount()
	{
		checkf(DisableElementDestructionOnGCCount != 0, TEXT("DisableElementDestructionOnGCCount underflow!"));
		--DisableElementDestructionOnGCCount;
	}

	mutable FTransactionallySafeRWLock RegisteredElementTypesRW;
	TUniquePtr<FRegisteredElementType> RegisteredElementTypes[TypedHandleMaxTypeId - 1];
	TSortedMap<FName, FTypedHandleTypeId, FDefaultAllocator, FNameFastLess> RegisteredElementTypesNameToId;

	mutable FCriticalSection ActiveElementListsCS;
	TSet<FTypedElementList*> ActiveElementLists;

	TSet<FScriptTypedElementList*> ActiveScriptElementLists; 

	uint8 DisableElementDestructionOnGCCount = 0;
	bool bIsWithinFrame = false;

	FOnElementReplaced OnElementReplacedDelegate;
	FOnElementUpdated OnElementUpdatedDelegate;
	FSimpleMulticastDelegate OnProcessingDeferredElementsToDestroyDelegate;
};
