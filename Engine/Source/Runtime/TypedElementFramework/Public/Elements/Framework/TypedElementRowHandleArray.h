// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Containers/ContainersFwd.h"
#include "Elements/Framework/TypedElementRowHandleArrayView.h"

namespace UE::Editor::DataStorage
{
	/** 
	 * Array dedicated to handling row handles. This provides various unique and optimized functionality that 
	 * makes working with a large number row handles easier and more efficient.
	 */
	class FRowHandleArray
	{
	public:
		using EFlags = FRowHandleArrayView::EFlags;

		FRowHandleArray() = default;
		TYPEDELEMENTFRAMEWORK_API explicit FRowHandleArray(FRowHandleArrayView InRows);

		/** Returns the number of rows stored. */
		TYPEDELEMENTFRAMEWORK_API int32 Num() const;

		/** 
		 * Reserve space for additional row handles. This allocates memory but doesn't add new row handles. If the requested count is less
		 * than the current count then nothing happens.
		 */
		TYPEDELEMENTFRAMEWORK_API void Reserve(int32 Count);

		/** Reduces the amount of memory used by the array to exactly fit the current number or elements. */
		TYPEDELEMENTFRAMEWORK_API void Shrink();

		/** 
		 * Appends the provided row handle at the end of the array. This may result in the array becoming unsorted and/or no longer unique.
		 * If Guarantee is set to IsUnique the user takes responsibility to guaranteed that the new row is not already present. If array
		 * is sorted, the IsUnique hint is ignored and the array will determine if the values are unique. The IsSorted guarantee is always
		 * ignored. If the arrays was already not unique setting the Guarantee flags has no effect.
		 */
		TYPEDELEMENTFRAMEWORK_API void Add(RowHandle Row, EFlags Guarantee = EFlags::None);

		/** 
		 * Append the provided list of rows to the end of the array. If Guarantee is set to IsSorted and/or IsUnique the user guarantees
		 * that the provided row handle array view is sorted by row handle and/or only contains unique values. In this case this function
		 * will try to append the additional rows and maintain the sorted status and if applicable the uniqueness state, otherwise the array 
		 * will be marked as unsorted and no longer unique.
		 */
		TYPEDELEMENTFRAMEWORK_API void Append(TConstArrayView<RowHandle> AdditionalRows, EFlags Guarantee = EFlags::None);

		/**
		 * Append the provided list of rows to the end of the array. If the current array and the additional rows are both sorted then
		 * this function will try to append the additional rows and maintain the sorted status and if applicable the uniqueness state,
		 * otherwise the array will be marked as unsorted and no longer unique.
		 */
		TYPEDELEMENTFRAMEWORK_API void Append(FRowHandleArrayView AdditionalRows);
		
		/**
		 * Removes the provided row. If the array is sorted, a binary search will be used to find the row, but requires moving all
		 * subsequent rows down. If the array is not sorted a linear search is required but the last row will be moved into the removed
		 * location for quicker removal.
		 */
		TYPEDELEMENTFRAMEWORK_API void Remove(RowHandle Row);
		/**
		 * Removes the provided list of rows. If the array is sorted, a binary search will be used to find the row, but requires moving all
		 * subsequent rows down. If the array is not sorted a linear search is required but the last row will be moved into the removed
		 * location for quicker removals.
		 */
		TYPEDELEMENTFRAMEWORK_API void Remove(TConstArrayView<RowHandle> RowsToRemove);
		/**
		 * Removes the provided list of rows. If the array is sorted, a binary search will be used to find the row, but requires moving all
		 * subsequent rows down. If the array is not sorted a linear search is required but the last row will be moved into the removed
		 * location for quicker removals. If both the array and list of rows to remove are sorted than an optimized version can run that
		 * quickly finds the first row to remove and then removes the other rows as it's moving the remainder down.
		 */
		TYPEDELEMENTFRAMEWORK_API void Remove(FRowHandleArrayView RowsToRemove);

		/**
		 * Checks if the provided row is stored in this array. If the array is sorted a faster binary search is used, otherwise a linear
		 * search is required.
		 */
		TYPEDELEMENTFRAMEWORK_API bool Contains(RowHandle Row) const;

		/**
		 * Removes all row handles, but keeps the allocated memory around. If the provided size is larger than the number of rows
		 * that fit in the container then extra memory will be allocated so there's enough space for the new number of rows.
		 */
		TYPEDELEMENTFRAMEWORK_API void Reset(int32 NewSize = 0);

		/**
		 * Removes all row handles and releases all allocated memory. If the provided slack size is not zero then memory is
		 * reserved for at least the provided number of rows.
		 */
		TYPEDELEMENTFRAMEWORK_API void Empty(int32 Slack = 0);

		/** Sorts the row handles in numerical order from lowest to highest. */
		TYPEDELEMENTFRAMEWORK_API void Sort();
		/**
		 * Sorts the row handles in numerical order from lowest to highest. This version is faster as it doesn't need to allocate
		 * a temporary intermediate array. ScratchBuffer has to be at least as large as the number of rows in the array.
		 */
		TYPEDELEMENTFRAMEWORK_API void Sort(TArrayView<RowHandle> ScratchBuffer);

		/** Goes through the array and removes any duplicate entries. If the array isn't sorted it will be sorted first. */
		TYPEDELEMENTFRAMEWORK_API void MakeUnique();
		/** 
		 * Goes through the array and only keeps a single instance of any value that appears twice or more. If the array 
		 * isn't sorted it will be sorted first. After this function completes the array only contains unique values.
		 */
		TYPEDELEMENTFRAMEWORK_API void ReduceToDuplicates();

		/** Merges the provided array into the array while remaining sorted. The provided array is expected to be sorted. */
		TYPEDELEMENTFRAMEWORK_API void SortedMerge(FRowHandleArrayView AddedRows);
		/** Merges the provided array into the array while remaining sorted. The provided array is expected to be sorted. */
		TYPEDELEMENTFRAMEWORK_API void SortedMerge(const FRowHandleArray& AddedRows);
		/** Merges the provided array into the array while remaining sorted. */
		TYPEDELEMENTFRAMEWORK_API void SortedMerge(FRowHandleArray&& AddedRows);

		/**
		 * Reverses the order in which the rows are stored, making the first element the last and vice versa. If the array
		 * was marked as sorted it will no longer be considered sorted. Another call to InvertOrder will not restore the
		 * sorted state.
		 */
		TYPEDELEMENTFRAMEWORK_API void InvertOrder();

		/** Constructs a view of the array to provide access to its rows. */
		TYPEDELEMENTFRAMEWORK_API FRowHandleArrayView GetRows() const;
		/** 
		 * Returns a mutable view of the array of rows. This will invalidate sorted and uniqueness status of the array.
		 * If Guarantee is set to IsSorted and/or IsUnique the user takes responsibility to guaranteed that the rows remain sorted in
		 * numeric order and/or all values remain unique after the new rows are modified. If the arrays was already not sorted and/or not unique
		 * setting the Guarantee flags has no effect.
		 */
		TYPEDELEMENTFRAMEWORK_API TArrayView<RowHandle> GetMutableRows(EFlags Guarantee = EFlags::None);
		/** Whether or not the contained rows are numerically ordered from smallest to largest. */
		TYPEDELEMENTFRAMEWORK_API bool IsSorted() const;
		/** Whether or all rows in the view only appear once. */
		TYPEDELEMENTFRAMEWORK_API bool IsUnique() const;
		/** Whether or not there are any values in the view. */
		TYPEDELEMENTFRAMEWORK_API bool IsEmpty() const;
		/** Whether or not the two arrays contain the same values. */
		TYPEDELEMENTFRAMEWORK_API bool IsSame(const FRowHandleArray& Other) const;

	private:
		void SortedMergeInternal(FRowHandleArrayView AddedRows);
		int32 Find(RowHandle Row) const;

		TArray<RowHandle> Rows;
		bool bIsSorted = true;
		bool bIsUnique = true;
	};
} // namespace UE::Editor::DataStorage
