// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <initializer_list>
#include "DataStorage/Handles.h"
#include "Containers/ArrayView.h"
#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"

namespace UE::Editor::DataStorage
{
	/**
	 * Provides a view of a list of row handles.
	 * Note that this view is more restrictive than typical array views. For instance, an typical array view would remain valid after 
	 * changing values the array, but will invalidate FRowHandleArrayView as the value may not be unique and/or cause the array to no
	 * longer be sorted.
	 */
	class FRowHandleArrayView
	{
	public:
		enum class EFlags
		{
			None = 0,
			IsSorted = 1 << 0, //< The view is sorted by row handle.
			IsUnique = 1 << 1 //< Every value in the view is guaranteed to appear only once.
		};

		FRowHandleArrayView() = default;
		TYPEDELEMENTFRAMEWORK_API FRowHandleArrayView(const RowHandle* Rows UE_LIFETIMEBOUND, int32 RowCount, EFlags Flags);
		TYPEDELEMENTFRAMEWORK_API FRowHandleArrayView(TConstArrayView<RowHandle> Rows, EFlags Flags);
		TYPEDELEMENTFRAMEWORK_API FRowHandleArrayView(std::initializer_list<RowHandle> Rows UE_LIFETIMEBOUND, EFlags Flags);

		/** 
		 * Returns a pointer to the first row in the view.
		 * This function is primarily used for compatibility to C++ features and libraries.
		 */
		TYPEDELEMENTFRAMEWORK_API const RowHandle* begin() const;
		/**
		 * Returns a pointer the address after the last row in the view. 
		 * This function is primarily used for compatibility to C++ features and libraries.
		 */
		TYPEDELEMENTFRAMEWORK_API const RowHandle* end() const;

		/** Retrieves the row handle at the provided index or asserts if the index is out of bounds. */
		TYPEDELEMENTFRAMEWORK_API RowHandle operator[](uint32 Index) const;
		/** Returns the address of row handles in memory.*/
		TYPEDELEMENTFRAMEWORK_API const RowHandle* GetData() const;
		/**
		 * If the view is not empty, this return the last row handle in the view. Calling this function on an empty view will cause an
		 * assert.
		 */
		TYPEDELEMENTFRAMEWORK_API const RowHandle& First() const;
		/**
		 * If the view is not empty, this return the last row handle in the view. Calling this function on an empty view will cause an
		 * assert.
		 */
		TYPEDELEMENTFRAMEWORK_API const RowHandle& Last() const;
		/** The number of rows this view shows. */
		TYPEDELEMENTFRAMEWORK_API int32 Num() const;
		/**
		 * Returns the total number of bytes that are in use by rows. This is may not be the total amount of memory in use by the 
		 * container this view maps to.
		 */
		TYPEDELEMENTFRAMEWORK_API int32 NumBytes() const;

		/** Conversion operator to allow the view to be transparently used in places where a immutable array view of row handles is required. */
		TYPEDELEMENTFRAMEWORK_API operator TConstArrayView<RowHandle>() const;

		/** Whether or not the contained rows are numerically ordered from smallest to largest. */
		TYPEDELEMENTFRAMEWORK_API bool IsSorted() const;
		/** Whether or all rows in the view only appear once. */
		TYPEDELEMENTFRAMEWORK_API bool IsUnique() const;
		/** Whether or not there are any values in the view. */
		TYPEDELEMENTFRAMEWORK_API bool IsEmpty() const;

		/** 
		 * Whether or not the view contains the provided row. If the view is sorted a much binary search can be used for a much faster 
		 * search otherwise a slower linear search is done.
		 */
		TYPEDELEMENTFRAMEWORK_API bool Contains(RowHandle Row) const;

	private:
		const RowHandle* Rows = nullptr;
		int32 RowCount = 0;
		bool bIsSorted = false;
		bool bIsUnique = false;
	};

	ENUM_CLASS_FLAGS(FRowHandleArrayView::EFlags);

} // namespace UE::Editor::DataStorage
