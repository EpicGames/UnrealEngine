// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

namespace UE::Editor::DataStorage
{
	/**
	 * @section Search provider
	 *
	 * @description
	 * Interface and supporting classes to provide searching of rows by column.
	 */
	class ICoreProvider;

	/** Interface to provide searching of rows by column. */
	class FColumnSearcherInterface
	{
	public:
		virtual ~FColumnSearcherInterface() = default;

		virtual void GetSearchableString(FString& SearchableString, const ICoreProvider& Storage, RowHandle Row) const = 0;
	};

	template<typename ColumnType>
	class TColumnSearcherInterface : public FColumnSearcherInterface
	{
	public:
		virtual ~TColumnSearcherInterface() override = default;

		virtual void GetSearchableString(FString& SearchableString, const ICoreProvider& Storage, RowHandle Row) const override final;

	protected:
		virtual void GetSearchableString(FString& SearchableString, const ICoreProvider& Storage, const ColumnType& Column) const = 0;
	};
}// namespace UE::Editor::DataStorage

#include "Elements/Framework/TypedElementSearcher.inl"