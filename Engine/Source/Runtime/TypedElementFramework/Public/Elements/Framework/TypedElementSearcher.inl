// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

namespace UE::Editor::DataStorage
{
	template<typename ColumnType>
	void TColumnSearcherInterface<ColumnType>::GetSearchableString(
		FString& SearchableString, const ICoreProvider& Storage, RowHandle Row) const
	{
		if (const ColumnType* Column = Storage.GetColumn<ColumnType>(Row))
		{
			GetSearchableString(SearchableString, Storage, *Column);
		}
	}
}// namespace UE::Editor::DataStorage