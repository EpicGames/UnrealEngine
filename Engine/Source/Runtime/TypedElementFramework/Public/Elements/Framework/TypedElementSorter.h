// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "DataStorage/Handles.h"
#include "Containers/StringFwd.h"
#include "Internationalization/Text.h"
#include "Math/NumericLimits.h"
#include "Misc/EnumClassFlags.h"

namespace UE::Editor::DataStorage
{
	/**
	 * @section Sort provider
	 *
	 * @description
	 * Interface and supporting classes to provide sorting of rows by column.
	 */
	class ICoreProvider;

	struct FPrefixInfo
	{
		uint64 Prefix = TNumericLimits<uint64>::Max();
		bool bHasRemainingBytes = false;
	};

	/** Interface to provide sorting of rows by column. */
	class FColumnSorterInterface
	{
	public:
		virtual ~FColumnSorterInterface() = default;

		enum class ESortType : uint8
		{
			/**
			 * Supports sort algorithms that sort by only using a single (packed) value of 64 bits.
			 * Algorithms supporting this option will only use `SortPrefix` and will only call this once per row.
			 */
			FixedSize64,
			/**
			 * Supports sort algorithms that sort by only using one or more fixed sized values like integers or floats.
			 * Algorithms supporting this option will only use `SortPrefix` and assume only fixed sized values are used. Including
			 * values with variable size such as strings will lead to suboptimal performance.
			 */
			FixedSizeOnly,
			/** 
			 * Supports sort algorithms that sort using a comparative function.
			 * Algorithms supporting this option will only use the `Compare` function to sort. Comparative sorting requires more reads
			 * from columns than fixed sized sorting or hybrid sorting an is also more susceptible to frame spikes when distributed over
			 * multiple frames and/or threads. It does however provide greater flexibility when requiring complex comparisons such as ones
			 * requiring multiple strings.
			 */
			ComparativeSort,
			/**
			 * Supports sort algorithms that sort using both a prefix and a comparative function.
			 * Algorithms supporting this option will use both the `SortPrefix` and `Compare` functions to sort. Typically rows are grouped
			 * into buckets based on their prefix and when a bucket is small enough a compare may be used to sort the bucket.
			 * A hybrid approach is typically faster for larger numbers of rows as is reduces the amount of times columns have to be read.
			 * It also reduces or removes the need for a compare which can be beneficial if the compare is expensive. Lastly it typically
			 * also allows for more even work distribution over frames/threads, reducing the risk of spikes. However it does come at an
			 * increased peak memory usage and limits the number of variable sized variables to one, which also have to be the last 
			 * variable in the chain.
			 */
			HybridSort,

			/** Indicates the final value, cannot be used as input. */
			Max
		};

		/** Returns the type of sorting required for this column. */
		virtual ESortType GetSortType() const = 0;

		/** If set, this can be used. */
		virtual FText GetShortName() const = 0;

		/**
		 * Compare the content of the left row to the right row and return a negative number if left is smaller than right, zero if left
		 * and right are equal and a positive number if right is larger than left.
		 */
		virtual int32 Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const = 0;

		/**
		 * The numeric prefix for the column. For numeric values this typically the number itself. For strings it's typically the next 8
		 * characters or 4 wide characters starting at the provided byte index. The utility function can `CreateSortPrefix` be used to 
		 * help create a prefix from one or more variables.
		 * Note that using a prefix is limited to only one value that has a variable length (e.g. strings) and has to be the last value
		 * to be sorted.
		 */
		virtual FPrefixInfo CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const = 0;
	};

	template<FColumnSorterInterface::ESortType SortType, typename ColumnType>
	class TColumnSorterInterface : public FColumnSorterInterface
	{
		static_assert(static_cast<uint32>(SortType) < static_cast<uint32>(ESortType::Max), "Unexpected sort type for TColumnSorterInterface.");
	};

	template<typename ColumnType>
	class TColumnSorterInterface<FColumnSorterInterface::ESortType::FixedSize64, ColumnType> : public FColumnSorterInterface
	{
	public:
		virtual ~TColumnSorterInterface() override = default;

		int32 Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const override final;
		FPrefixInfo CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const override final;
		ESortType GetSortType() const override final;

	protected:
		virtual FPrefixInfo CalculatePrefix(const ColumnType& Column, uint32 ByteIndex) const = 0;
	};

	template<typename ColumnType>
	class TColumnSorterInterface<FColumnSorterInterface::ESortType::FixedSizeOnly, ColumnType> : public FColumnSorterInterface
	{
	public:
		virtual ~TColumnSorterInterface() override = default;

		int32 Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const override final;
		FPrefixInfo CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const override final;
		ESortType GetSortType() const override final;

	protected:
		virtual FPrefixInfo CalculatePrefix(const ColumnType& Column, uint32 ByteIndex) const = 0;
	};

	template<typename ColumnType>
	class TColumnSorterInterface<FColumnSorterInterface::ESortType::ComparativeSort, ColumnType> : public FColumnSorterInterface
	{
	public:
		virtual ~TColumnSorterInterface() override = default;

		int32 Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const override final;
		FPrefixInfo CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const override final;
		ESortType GetSortType() const override final;

	protected:
		virtual int32 Compare(const ColumnType& Left, const ColumnType& Right) const = 0;
	};

	template<typename ColumnType>
	class TColumnSorterInterface<FColumnSorterInterface::ESortType::HybridSort, ColumnType> : public FColumnSorterInterface
	{
	public:
		virtual ~TColumnSorterInterface() override = default;

		int32 Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const override final;
		FPrefixInfo CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const override final;
		ESortType GetSortType() const override final;

	protected:
		virtual int32 Compare(const ColumnType& Left, const ColumnType& Right) const = 0;
		virtual FPrefixInfo CalculatePrefix(const ColumnType& Column, uint32 ByteIndex) const = 0;
	};

	namespace Private
	{
		template<typename T>
		concept SortNumericValue = std::is_integral_v<T> || std::is_floating_point_v<T>;

		template <typename T>
		struct TSortStringViewType : std::false_type {};

		template <typename T>
		struct TSortStringViewType<TStringView<T>> : std::true_type {};

		template<typename T>
		concept SortStringViewValue = TSortStringViewType<T>::value;

		template<typename T>
		concept SortStringViewCopyableType =
			SortStringViewValue<T> ||
			std::is_same_v<T, FString> ||
			std::is_same_v<T, FAnsiString> ||
			std::is_same_v<T, FWideString> ||
			std::is_same_v<T, FUtf8String>;

		template<typename T>
		concept SortStringVariantType = SortStringViewCopyableType<T> || std::is_same_v<T, FText>;

		template<typename T>
		concept NameValue = std::is_same_v<T, FName>;
	}

	/**
	 * @section Sort support structures
	 *
	 * @description
	 * Includes a variety of structures that wrap types to add additional information needed for sorting. An example is a view
	 * to wrap strings to indicate whether the sort is case sensitive or not.
	 */

	struct FSortCaseSensitive {};
	struct FSortCaseInsensitive {};
	template<typename T>
	concept SortCase = std::is_same_v<T, FSortCaseSensitive> || std::is_same_v<T, FSortCaseInsensitive>;
	
	template<SortCase Casing, Private::SortStringViewValue StringView>
	struct TSortStringView
	{
		using StringViewType = StringView;
		static constexpr bool bIsCaseSensitive = std::is_same_v<Casing, FSortCaseSensitive>;
		static constexpr ESearchCase::Type SearchCase = bIsCaseSensitive ? ESearchCase::CaseSensitive : ESearchCase::IgnoreCase;
		
		TSortStringView() = default;
		
		template<Private::SortStringViewCopyableType ViewType>
		TSortStringView(const ViewType& InView) : View(InView) {}
		template<Private::SortStringViewCopyableType ViewType>
		TSortStringView& operator=(const ViewType& InView) { this->View = InView; return *this; }

		TSortStringView(const FText& Text) : View(Text.ToString()) {}
		TSortStringView& operator=(const FText& Text) { this->View = Text.ToString(); return *this; }
		
		template<SortCase TargetCasing, Private::SortStringViewCopyableType ViewType>
		TSortStringView(TargetCasing, const ViewType& String) : View(String) { static_assert(std::is_same_v<Casing, TargetCasing>); }
		template<SortCase TargetCasing>
		TSortStringView(TargetCasing, const FText& Text) : View(Text.ToString()) { static_assert(std::is_same_v<Casing, TargetCasing>); }

		bool operator==(const TSortStringView& Rhs) const { return View.Equals(Rhs.View, SearchCase) == true; }
		bool operator!=(const TSortStringView& Rhs) const { return View.Equals(Rhs.View, SearchCase) == false; }
		bool operator< (const TSortStringView& Rhs) const { return View.Compare(Rhs.View, SearchCase) < 0; }
		bool operator<=(const TSortStringView& Rhs) const { return View.Compare(Rhs.View, SearchCase) <= 0; }
		bool operator> (const TSortStringView& Rhs) const { return View.Compare(Rhs.View, SearchCase) > 0; }
		bool operator>=(const TSortStringView& Rhs) const { return View.Compare(Rhs.View, SearchCase) >= 0; }

		StringView View;
	};

	template<SortCase Casing, Private::SortStringViewCopyableType View>
	TSortStringView(Casing, const View&) -> TSortStringView<Casing, TStringView<typename View::ElementType>>;
	template<SortCase Casing>
	TSortStringView(Casing, const FText&) -> TSortStringView<Casing, FStringView>;

	enum class ENameSortBy : bool { Id, String };

	enum class ESortByNameFlags
	{
		Default = 0,
		WithNone = 1 << 0, /** If an empty name is found, use the string "None" to represent it. */
		RemoveLeadingSlash = 1 << 1, /** Ignore any leading slashes, if they exist. */
	};
	ENUM_CLASS_FLAGS(ESortByNameFlags)

	/** Sort FNames by their name. If an empty name is provided it will be treated as an empty string. */
	template<ESortByNameFlags InFlags = ESortByNameFlags::Default>
	struct TSortByName { static const ESortByNameFlags Flags = InFlags; };
	/** Sort FNames by their id. */
	struct FSortById {};
	
	template<typename>
	struct IsSortByName : std::false_type {};
	template<ESortByNameFlags Flags>
	struct IsSortByName<TSortByName<Flags>> : std::true_type {};

	template<typename T>
	concept SortBy = std::is_same_v<T, FSortById> || IsSortByName<T>::value;

	template<SortBy By>
	struct TSortNameView
	{
		constexpr static bool bIsById = std::is_same_v<By, FSortById>;
		using CompareType = std::conditional_t<bIsById, int32, FString>;
		constexpr static bool bIsFixedSize = bIsById;

		TSortNameView() = default;
		explicit TSortNameView(const FName& Name) : View(&Name) {}
		TSortNameView& operator=(const FName& Name);

		template<SortBy TargetBy>
		TSortNameView(TargetBy, const FName& Name) : View(&Name) { static_assert(std::is_same_v<By, TargetBy>); }

		uint32 GetByteSize() const;
		constexpr static uint32 GetElementSize();
		FPrefixInfo CalculatePrefix(int32 CurrentIndex, int32 ByteIndex) const;

		int32 Compare(const FName& Rhs) const;
		int32 Compare(const TSortNameView& Rhs) const;
		bool operator==(const TSortNameView& Rhs) const;
		bool operator!=(const TSortNameView& Rhs) const;
		bool operator< (const TSortNameView& Rhs) const;
		bool operator<=(const TSortNameView& Rhs) const;
		bool operator> (const TSortNameView& Rhs) const;
		bool operator>=(const TSortNameView& Rhs) const;

		const FName* View = nullptr;
		mutable CompareType Cache = {};
		mutable bool bIsCached = false;

	private:
		int32 Compare(const FName& Lhs, const FName& Rhs) const;
		void CacheCompareType() const;
	};

	template<SortBy By>
	TSortNameView(By, const FName&) -> TSortNameView<By>;

	template<typename... ValueTypes>
	FPrefixInfo CreateSortPrefix(uint32 ByteIndex, ValueTypes&&... Values);

	/**
	 * @section Sort type information
	 *
	 * @description
	 * Utility structs and function that describe the way types can be sorted and packed into an index.
	 */

	namespace Private
	{
		template<typename T>
		constexpr uint64 MoveToLocation(int32 ByteIndex, T Value);

		template<typename Numeric>
		constexpr auto Rebase(Numeric Value);
	}

	template<typename ValueType>
	struct TSortTypeInfo
	{
		// Used to short-circuit several calls to minimize noise in the error output of the compiler.
		static constexpr bool bIsSupportedType = false;

		// Check is just a random way to say "false" so the compiler doesn't evaluate the assert until an attempt is made to instance 
		// this version.
		static_assert(sizeof(ValueType) == 0, "Unsupported sort type.");
	};

	template<Private::SortNumericValue NumericType>
	struct TSortTypeInfo<NumericType>
	{
		static constexpr bool bIsSupportedType = true;
		static constexpr bool bIsFixedSize = true;
		constexpr static uint32 GetByteSize(NumericType Value) { return sizeof(NumericType); }
		constexpr static uint32 GetElementSize() { return sizeof(NumericType); }
		constexpr static FPrefixInfo CalculatePrefix(int32 CurrentIndex, int32 ByteIndex, NumericType Value)
		{
			// Due to an issue with MSVC resolving forward declared partially specialized templates that use concepts, this 
			// can't be moved to the .inl file.
			return FPrefixInfo
			{
				.Prefix = Private::MoveToLocation(CurrentIndex - ByteIndex, Private::Rebase(Value)),
				.bHasRemainingBytes = false
			};
		}
	};

	template<SortCase Casing, typename T>
	struct TSortTypeInfo<TSortStringView<Casing, T>>
	{
		static constexpr bool bIsSupportedType = true;
		static constexpr bool bIsFixedSize = false;
		constexpr static uint32 GetByteSize(TSortStringView<Casing, T> Value);
		constexpr static uint32 GetElementSize();
		constexpr static FPrefixInfo CalculatePrefix(
			int32 CurrentIndex, int32 ByteIndex, TSortStringView<Casing, T> Value);
	};

	template<typename StringType> requires Private::SortStringVariantType<StringType>
	struct TSortTypeInfo<StringType>
	{
		static constexpr bool bIsSupportedType = false;
		static_assert(sizeof(StringType) == 0, 
			"Strings and string views are not directly supported. Use `TSortStringView` to indicate if sorting is case sensitive or not.");
	};

	template<SortBy By>
	struct TSortTypeInfo<TSortNameView<By>>
	{
		static constexpr bool bIsSupportedType = true;
		static constexpr bool bIsFixedSize = TSortNameView<By>::bIsFixedSize;
		static uint32 GetByteSize(const TSortNameView<By>& Value) { return Value.GetByteSize(); };
		constexpr static uint32 GetElementSize() { return TSortNameView<By>::GetElementSize(); };
		static FPrefixInfo CalculatePrefix(
			int32 CurrentIndex, int32 ByteIndex, TSortNameView<By> Value) { return Value.CalculatePrefix(CurrentIndex, ByteIndex); };
	};

	// Not a direct specialization, but done through requirements to prevent the compiler from evaluating the static_assert when it's not instanced.
	template<Private::NameValue NameType>
	struct TSortTypeInfo<NameType>
	{
		static constexpr bool bIsSupportedType = false;
		static_assert(sizeof(NameType) == 0,
			"FNames are not directly supported. Use `TSortNameView` to indicate if sorting is based on a string or the unique FName number.");
	};
} // namespace UE::Editor::DataStorage

#include "Elements/Framework/TypedElementSorter.inl"
