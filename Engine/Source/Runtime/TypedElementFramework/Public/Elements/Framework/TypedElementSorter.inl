// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/Char.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

namespace UE::Editor::DataStorage
{
	//
	// FixedSizeOnly Sort
	//

	template<typename ColumnType>
	int32 TColumnSorterInterface<FColumnSorterInterface::ESortType::FixedSize64, ColumnType > ::Compare(
		const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const
	{
		return 0;
	}

	template<typename ColumnType>
	FPrefixInfo TColumnSorterInterface<FColumnSorterInterface::ESortType::FixedSize64, ColumnType>::CalculatePrefix(
		const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const
	{
		const ColumnType* Column = Storage.GetColumn<ColumnType>(Row);
		return Column ? CalculatePrefix(*Column, ByteIndex) : FPrefixInfo{};
	}

	template<typename ColumnType>
	FColumnSorterInterface::ESortType TColumnSorterInterface<FColumnSorterInterface::ESortType::FixedSize64, ColumnType>::GetSortType() const
	{
		return FColumnSorterInterface::ESortType::FixedSize64;
	}



	//
	// FixedSizeOnly Sort
	//

	template<typename ColumnType>
	int32 TColumnSorterInterface<FColumnSorterInterface::ESortType::FixedSizeOnly, ColumnType > ::Compare(
		const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const
	{
		return 0;
	}

	template<typename ColumnType>
	FPrefixInfo TColumnSorterInterface<FColumnSorterInterface::ESortType::FixedSizeOnly, ColumnType>::CalculatePrefix(
		const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const
	{
		const ColumnType* Column = Storage.GetColumn<ColumnType>(Row);
		return Column ? CalculatePrefix(*Column, ByteIndex) : FPrefixInfo{};
	}

	template<typename ColumnType>
	FColumnSorterInterface::ESortType TColumnSorterInterface<FColumnSorterInterface::ESortType::FixedSizeOnly, ColumnType>::GetSortType() const
	{
		return FColumnSorterInterface::ESortType::FixedSizeOnly;
	}

	
	
	//
	// Comparative Sort
	//

	template<typename ColumnType>
	int32 TColumnSorterInterface<FColumnSorterInterface::ESortType::ComparativeSort, ColumnType > ::Compare(
		const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const
	{
		const ColumnType* LeftColumn = Storage.GetColumn<ColumnType>(Left);
		const ColumnType* RigthColumn = Storage.GetColumn<ColumnType>(Right);

		return (LeftColumn && RigthColumn)
			? Compare(*LeftColumn, *RigthColumn)
			: ((LeftColumn == nullptr) - (RigthColumn == nullptr));
	}

	template<typename ColumnType>
	FPrefixInfo TColumnSorterInterface<FColumnSorterInterface::ESortType::ComparativeSort, ColumnType>::CalculatePrefix(
		const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const
	{
		return FPrefixInfo{};
	}

	template<typename ColumnType>
	FColumnSorterInterface::ESortType TColumnSorterInterface<FColumnSorterInterface::ESortType::ComparativeSort, ColumnType>::GetSortType() const
	{
		return FColumnSorterInterface::ESortType::ComparativeSort;
	}


	//
	// Hybrid Sort
	//
	
	template<typename ColumnType>
	int32 TColumnSorterInterface<FColumnSorterInterface::ESortType::HybridSort, ColumnType>::Compare(
		const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const
	{
		const ColumnType* LeftColumn = Storage.GetColumn<ColumnType>(Left);
		const ColumnType* RigthColumn = Storage.GetColumn<ColumnType>(Right);

		return (LeftColumn && RigthColumn)
			? Compare(*LeftColumn, *RigthColumn)
			: ((LeftColumn == nullptr) - (RigthColumn == nullptr));
	}

	template<typename ColumnType>
	FPrefixInfo TColumnSorterInterface<FColumnSorterInterface::ESortType::HybridSort, ColumnType>::CalculatePrefix(
		const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const
	{
		const ColumnType* Column = Storage.GetColumn<ColumnType>(Row);
		return Column ? CalculatePrefix(*Column, ByteIndex) : FPrefixInfo{};
	}

	template<typename ColumnType>
	FColumnSorterInterface::ESortType TColumnSorterInterface<FColumnSorterInterface::ESortType::HybridSort, ColumnType>::GetSortType() const
	{
		return FColumnSorterInterface::ESortType::HybridSort;
	}

	namespace Private
	{
		template<typename T>
		constexpr uint64 MoveToLocation(int32 ByteIndex, T Value)
		{
			constexpr int32 ByteSize = sizeof(T);
			const uint64 Result = static_cast<uint64>(Value);
			const int32 ByteShift = 8 - ByteIndex - ByteSize;
			const int32 BitShift = ByteShift * 8;
			return (BitShift >= 0) ? (Result << BitShift) : (Result >> -BitShift);
		}

		template<typename Numeric>
		constexpr auto Rebase(Numeric Value)
		{
			constexpr uint64 BitShift = (sizeof(Numeric) * 8) - 1;

			if constexpr (std::is_floating_point_v<Numeric>)
			{
				using SignedInt = std::conditional_t<sizeof(Numeric) == sizeof(double), int64, int32>;
				using UnsignedInt = std::make_unsigned_t<SignedInt>;

				union { Numeric F; UnsignedInt U; } Converter;
				Converter.F = Value;

				SignedInt mask = -static_cast<SignedInt>(Converter.U >> BitShift) | (UnsignedInt(1) << BitShift);
				return Converter.U ^ mask;
			}
			else if constexpr (std::is_signed_v<Numeric>)
			{
				using Unsigned = std::make_unsigned_t<Numeric>;
				
				Numeric mask = static_cast<Numeric>(-(Value >> BitShift) | (Unsigned(1) << BitShift));
				return static_cast<Unsigned>(Value ^ mask);
			}
			else
			{
				static_assert(std::is_unsigned_v<Numeric>, "Input value to rebase for sort indexing must be a numeric value.");
				return Value;
			}
		}

		template<bool bCaseSensitive, typename CharType>
		CharType ToUpper(CharType Input)
		{
			if constexpr (bCaseSensitive)
			{
				return Input;
			}
			else
			{
				return TChar<CharType>::ToUpper(Input);
			}
		}

		template<typename ValueType>
		void CalculatePrefix(FPrefixInfo& Result, int32 CurrentIndex, int32 ByteIndex, const ValueType& Value)
		{
			if constexpr (TSortTypeInfo<ValueType>::bIsSupportedType)
			{
				constexpr int32 ResultSize = sizeof(FPrefixInfo::Prefix);
				int32 Size = static_cast<int32>(TSortTypeInfo<ValueType>::GetByteSize(Value));

				if constexpr (TSortTypeInfo<ValueType>::bIsFixedSize)
				{
					int32 Position = CurrentIndex - ByteIndex;
					if (Position < ResultSize)
					{
						Result.Prefix |= TSortTypeInfo<ValueType>::CalculatePrefix(CurrentIndex, ByteIndex, Value).Prefix;
						Result.bHasRemainingBytes = (Position + Size) > sizeof(FPrefixInfo::Prefix);
					}
				}
				else
				{
					constexpr uint32 ElementSize = TSortTypeInfo<ValueType>::GetElementSize();

					int32 Index = CurrentIndex - ByteIndex;
					if (ResultSize - (Index & (ResultSize - 1)) >= static_cast<int32>(ElementSize))
					{
						FPrefixInfo Intermediate =
							TSortTypeInfo<ValueType>::CalculatePrefix(CurrentIndex, ByteIndex, Value);
						Result.Prefix |= Intermediate.Prefix;
						Result.bHasRemainingBytes = Intermediate.bHasRemainingBytes;
					}
				}
			}
		}

		template<typename ValueType, typename... ValueTypes>
		void CalculatePrefix(FPrefixInfo& Result, int32 CurrentIndex, int32 ByteIndex,
			const ValueType& Value, ValueTypes&&... Values)
		{
			if constexpr (TSortTypeInfo<ValueType>::bIsSupportedType)
			{
				constexpr int32 ResultSize = sizeof(FPrefixInfo::Prefix);
				int32 Size = static_cast<int32>(TSortTypeInfo<ValueType>::GetByteSize(Value));
				// Skip over bytes that fall out of the requested range.
				if (CurrentIndex + Size <= ByteIndex)
				{
					CalculatePrefix(Result, CurrentIndex + Size, ByteIndex, Values...);
				}
				else
				{
					static_assert(TSortTypeInfo<ValueType>::bIsFixedSize, "Only the last value type can be of variable size.");

					// Try to pack as many bytes into the final result as are available. This might require pulling portions of values.
					int32 Position = CurrentIndex - ByteIndex;
					if (Position < ResultSize)
					{
						// Continue while there are still bytes available.
						Result.Prefix |= TSortTypeInfo<ValueType>::CalculatePrefix(CurrentIndex, ByteIndex, Value).Prefix;
						CalculatePrefix(Result, CurrentIndex + Size, ByteIndex, Values...);
					}
				}
			}
		}
	}

	
	
	//
	// TSortNameView
	//

	template<SortBy By>
	TSortNameView<By>& TSortNameView<By>::operator=(const FName& Name)
	{ 
		View = &Name; 
		bIsCached = false; 
		return *this;
	}

	template<SortBy By>
	uint32 TSortNameView<By>::GetByteSize() const
	{
		if constexpr (bIsById)
		{
			return sizeof(Cache);
		}
		else
		{
			CacheCompareType();
			return static_cast<uint32>(Cache.NumBytesWithoutNull());
		}
	}

	template<SortBy By>
	constexpr uint32 TSortNameView<By>::GetElementSize()
	{
		if constexpr (bIsById)
		{
			return sizeof(Cache);
		}
		else
		{
			return sizeof(typename CompareType::ElementType);
		}
	}

	template<SortBy By>
	FPrefixInfo TSortNameView<By>::CalculatePrefix(int32 CurrentIndex, int32 ByteIndex) const
	{
		CacheCompareType();
		if constexpr (bIsById)
		{
			return TSortTypeInfo<CompareType>::CalculatePrefix(CurrentIndex, ByteIndex, Cache);
		}
		else
		{
			using StringView = TStringView<typename CompareType::ElementType>;
			using SortStringView = TSortStringView<FSortCaseInsensitive, StringView>;
			return TSortTypeInfo<SortStringView>::CalculatePrefix(CurrentIndex, ByteIndex, StringView(Cache));
		}
	}
	
	template<SortBy By>
	int32 TSortNameView<By>::Compare(const FName& Lhs, const FName& Rhs) const
	{
		if constexpr (bIsById)
		{
			return View->CompareIndexes(Rhs);
		}
		else
		{
			auto Compare = [this](const FName& To)
				{
					if constexpr (EnumHasAllFlags(By::Flags, ESortByNameFlags::RemoveLeadingSlash))
					{
						CacheCompareType();
						FString ToString = To.ToString();
						ToString.RemoveFromStart(TEXT("/"));
						return Cache.Compare(ToString, ESearchCase::IgnoreCase);
					}
					else
					{
						return View->Compare(To);
					}
				};

			if constexpr (EnumHasAllFlags(By::Flags, ESortByNameFlags::WithNone))
			{
				return Compare(Rhs);
			}
			else
			{
				bool LeftIsAssigned = !View->IsNone();
				bool RightIsAssigned = !Rhs.IsNone();
				return (LeftIsAssigned && RightIsAssigned)
					? Compare(Rhs)
					// Note that this is in reversed order then elsewhere because in this case the empty value should be at the top, not the bottom.
					: static_cast<int32>(LeftIsAssigned) - static_cast<int32>(RightIsAssigned);
			}
		}
	}

	template<SortBy By>
	int32 TSortNameView<By>::Compare(const FName& Rhs) const
	{
		return View ? Compare(*View, Rhs) : 1;
	}

	template<SortBy By>
	int32 TSortNameView<By>::Compare(const TSortNameView& Rhs) const
	{
		return (View && Rhs.View)
			? Compare(*View, *Rhs.View)
			: static_cast<int32>(View == nullptr) - static_cast<int32>(Rhs.View == nullptr);
	}

	template<SortBy By>
	bool TSortNameView<By>::operator==(const TSortNameView& Rhs) const
	{ 
		return Compare(Rhs) == 0;
	}

	template<SortBy By>
	bool TSortNameView<By>::operator!=(const TSortNameView& Rhs) const
	{ 
		return Compare(Rhs) != 0;
	}

	template<SortBy By>
	bool TSortNameView<By>::operator< (const TSortNameView& Rhs) const
	{ 
		return Compare(Rhs) < 0;
	}
	
	template<SortBy By>
	bool TSortNameView<By>::operator<=(const TSortNameView& Rhs) const
	{ 
		return Compare(Rhs) <= 0;
	}

	template<SortBy By>
	bool TSortNameView<By>::operator> (const TSortNameView& Rhs) const
	{ 
		return Compare(Rhs) > 0;
	}

	template<SortBy By>
	bool TSortNameView<By>::operator>=(const TSortNameView& Rhs) const
	{ 
		return Compare(Rhs) >= 0;
	}

	template<SortBy By>
	void TSortNameView<By>::CacheCompareType() const
	{
		if (View && !bIsCached)
		{
			if constexpr (bIsById)
			{
				Cache = View->GetComparisonIndex().ToUnstableInt();
			}
			else
			{
				Cache.Reset();
				if constexpr (EnumHasAllFlags(By::Flags, ESortByNameFlags::WithNone))
				{
					View->ToString(Cache);
				}
				else
				{
					if (!View->IsNone())
					{
						View->ToString(Cache);
					}
				}

				if constexpr (EnumHasAllFlags(By::Flags, ESortByNameFlags::RemoveLeadingSlash))
				{
					Cache.RemoveFromStart(TEXT("/"));
				}
			}
			bIsCached = true;
		}
	}



	// TSortTypeInfo<TSortStringView<...>>

	template<SortCase Casing, typename T>
	constexpr uint32 TSortTypeInfo<TSortStringView<Casing, T>>::GetByteSize(TSortStringView<Casing, T> Value)
	{
		return static_cast<uint32>(Value.View.NumBytes());
	}

	template<SortCase Casing, typename T>
	constexpr uint32 TSortTypeInfo<TSortStringView<Casing, T>>::GetElementSize()
	{
		return sizeof(typename T::ElementType);
	}

	template<SortCase Casing, typename T>
	constexpr FPrefixInfo TSortTypeInfo<TSortStringView<Casing, T>>::CalculatePrefix(
		int32 CurrentIndex, int32 ByteIndex, TSortStringView<Casing, T> Value)
	{
		constexpr bool bIsCaseSensitive = TSortStringView<Casing, T>::bIsCaseSensitive;

		uint64 Result = 0;
		bool bHasRemainingCharacters = false;

		uint32 BytesIntoResult = ByteIndex < CurrentIndex ? (CurrentIndex & 7) : 0;
		uint32 RemainingBytes = 8 - BytesIntoResult;
		int32 StringIndex = FMath::Clamp<int32>(ByteIndex - CurrentIndex, 0, static_cast<int32>(Value.View.NumBytes()));
		
		if constexpr (sizeof(typename T::ElementType) == 1)
		{
			int32 Min = FMath::Min<int32>(Value.View.Len() - StringIndex, RemainingBytes);

			switch (Min)
			{
			case 8:
				Result |= (static_cast<uint64>(Private::ToUpper<bIsCaseSensitive>(Value.View[StringIndex + Min - 8])) << 56);
			case 7:
				Result |= (static_cast<uint64>(Private::ToUpper<bIsCaseSensitive>(Value.View[StringIndex + Min - 7])) << 48);
			case 6:
				Result |= (static_cast<uint64>(Private::ToUpper<bIsCaseSensitive>(Value.View[StringIndex + Min - 6])) << 40);
			case 5:
				Result |= (static_cast<uint64>(Private::ToUpper<bIsCaseSensitive>(Value.View[StringIndex + Min - 5])) << 32);
			case 4:
				Result |= (static_cast<uint64>(Private::ToUpper<bIsCaseSensitive>(Value.View[StringIndex + Min - 4])) << 24);
			case 3:
				Result |= (static_cast<uint64>(Private::ToUpper<bIsCaseSensitive>(Value.View[StringIndex + Min - 3])) << 16);
			case 2:
				Result |= (static_cast<uint64>(Private::ToUpper<bIsCaseSensitive>(Value.View[StringIndex + Min - 2])) << 8);
			case 1:
				Result |= (static_cast<uint64>(Private::ToUpper<bIsCaseSensitive>(Value.View[StringIndex + Min - 1])));
			}
			Result <<= 64 - (BytesIntoResult + Min) * 8;
			bHasRemainingCharacters = StringIndex + Min < Value.View.Len();
		}
		else
		{
			StringIndex >>= 1;
			int32 Min = FMath::Min<int32>(Value.View.Len() - StringIndex, RemainingBytes >> 1);
			switch (Min)
			{
			case 4:
				Result |= (static_cast<uint64>(Private::ToUpper<bIsCaseSensitive>(Value.View[StringIndex + Min - 4])) << 48);
			case 3:
				Result |= (static_cast<uint64>(Private::ToUpper<bIsCaseSensitive>(Value.View[StringIndex + Min - 3])) << 32);
			case 2:
				Result |= (static_cast<uint64>(Private::ToUpper<bIsCaseSensitive>(Value.View[StringIndex + Min - 2])) << 16);
			case 1:
				Result |= (static_cast<uint64>(Private::ToUpper<bIsCaseSensitive>(Value.View[StringIndex + Min - 1])));
			}
			Result <<= 64 - (BytesIntoResult + (Min *2)) * 8;
			bHasRemainingCharacters = StringIndex + Min < Value.View.Len();
		}

		return FPrefixInfo
		{
			.Prefix = Result,
			.bHasRemainingBytes = bHasRemainingCharacters
		};
	}

	template<typename... ValueTypes>
	FPrefixInfo CreateSortPrefix(uint32 ByteIndex, ValueTypes&&... Values)
	{
		FPrefixInfo Result
		{
			.Prefix = 0,
			.bHasRemainingBytes = true
		};
		Private::CalculatePrefix(Result, 0, static_cast<int32>(ByteIndex), Values...);
		return Result;
	}
} // namespace UE::Editor::DataStorage
