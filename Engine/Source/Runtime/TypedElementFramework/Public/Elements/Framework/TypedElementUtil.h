// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "Elements/Framework/TypedElementLimits.h"
#include "Elements/Framework/TypedElementListFwd.h"

class UTypedElementRegistry;
struct FScriptTypedElementHandle;
struct FTypedElementHandle;

namespace TypedElementUtil
{

/**
 * Batch the given elements by their type.
 */
TYPEDELEMENTFRAMEWORK_API void BatchElementsByType(FTypedElementListConstRef InElementsToBatch, TMap<FTypedHandleTypeId, TArray<FTypedElementHandle>>& OutElementsByType);
TYPEDELEMENTFRAMEWORK_API void BatchElementsByType(TArrayView<const FTypedElementHandle> InElementsToBatch, TMap<FTypedHandleTypeId, TArray<FTypedElementHandle>>& OutElementsByType);

TYPEDELEMENTFRAMEWORK_API TArray<FScriptTypedElementHandle> ConvertToScriptElementArray(const TArray<FTypedElementHandle>& InNativeHandles, UTypedElementRegistry* Registry);
TYPEDELEMENTFRAMEWORK_API TArray<FTypedElementHandle> ConvertToNativeElementArray(const TArray<FScriptTypedElementHandle>& InScriptHandles);

} // namespace TypedElementUtil
