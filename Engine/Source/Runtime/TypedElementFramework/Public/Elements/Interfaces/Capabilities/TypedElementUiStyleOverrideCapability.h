// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementUiCapabilities.h"
#include "Misc/Attribute.h"

struct FSlateColor;

/**
 * Interface to provide access to widgets that support working with style overrides.
 */
class UE_DEPRECATED(5.8, "Please use FSceneOutlinerColumn to find the foreground color using Helpers::GetTreeItemFromRowHandle and FSceneOutlinerCommonLabelData::GetForegroundColor")
	ITypedElementUiStyleOverrideCapability
PRAGMA_DISABLE_DEPRECATION_WARNINGS
: public ITypedElementUiCapability
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SLATE_METADATA_TYPE(ITypedElementUiStyleOverrideCapability, ITypedElementUiCapability)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	~ITypedElementUiStyleOverrideCapability() override = default;

	virtual void SetForegroundColor(const TAttribute<FSlateColor>& InColorAndOpacity) = 0;
};

template<typename WidgetType>
class UE_DEPRECATED(5.8, "Please use FSceneOutlinerColumn to find the foreground color using Helpers::GetTreeItemFromRowHandle and FSceneOutlinerCommonLabelData::GetForegroundColor")
	TTypedElementUiStyleOverrideCapability
PRAGMA_DISABLE_DEPRECATION_WARNINGS
: public ITypedElementUiStyleOverrideCapability
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
public:
	explicit TTypedElementUiStyleOverrideCapability(WidgetType& InWidget) : Widget(InWidget){}
	
	virtual void SetForegroundColor(const TAttribute<FSlateColor>& InColorAndOpacity) override
	{
		Widget.SetForegroundColor(InColorAndOpacity);
	}

private:
	WidgetType& Widget;
};
