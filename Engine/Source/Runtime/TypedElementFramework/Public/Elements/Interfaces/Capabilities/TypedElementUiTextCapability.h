// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementUiCapabilities.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"

/**
 * Interface to provide access to widgets that support working with text.
 */
class UE_DEPRECATED(5.8, "Please use FSceneOutlinerColumn to get the Outlier ptr and use its GetFilterHighlightText() function")
	ITypedElementUiTextCapability
PRAGMA_DISABLE_DEPRECATION_WARNINGS
: public ITypedElementUiCapability
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SLATE_METADATA_TYPE(ITypedElementUiTextCapability, ITypedElementUiCapability)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	~ITypedElementUiTextCapability() override = default;

	virtual void SetText(const TAttribute<FText>& Text) = 0;

	virtual void SetHighlightText(const TAttribute<FText>& Text) = 0;
};

template<typename WidgetType>
class UE_DEPRECATED(5.8, "Please use FSceneOutlinerColumn to get the Outlier ptr and use its GetFilterHighlightText() function")
	TTypedElementUiTextCapability
PRAGMA_DISABLE_DEPRECATION_WARNINGS
: public ITypedElementUiTextCapability
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
public:
	explicit TTypedElementUiTextCapability(WidgetType& InWidget) : Widget(InWidget){}
	
	void SetText(const TAttribute<FText>& Text) override
	{
		Widget.SetText(Text);
	}

	void SetHighlightText(const TAttribute<FText>& Text) override
	{
		Widget.SetHighlightText(Text);
	}

private:
	WidgetType& Widget;
};
