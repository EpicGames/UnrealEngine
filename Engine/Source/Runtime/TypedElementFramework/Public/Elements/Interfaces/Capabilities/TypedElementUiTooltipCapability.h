// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementUiCapabilities.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Widgets/IToolTip.h"

/**
 * Interface to provide access to tool tips on widgets.
 */
class UE_DEPRECATED(5.8, "Please use FTypedElementLabelColumn and FAttributeBinder to bind the tooltip attribute to the widget instead")
	ITypedElementUiTooltipCapability
PRAGMA_DISABLE_DEPRECATION_WARNINGS
: public ITypedElementUiCapability
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SLATE_METADATA_TYPE(ITypedElementUiTooltipCapability, ITypedElementUiCapability)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	~ITypedElementUiTooltipCapability() override = default;

	virtual void SetToolTipText(const TAttribute<FText>& ToolTipText) = 0;
	virtual void SetToolTipText(const FText& ToolTipText) = 0;
	virtual void SetToolTip(const TAttribute<TSharedPtr<IToolTip>>& ToolTip) = 0;
	virtual TSharedPtr<IToolTip> GetToolTip() = 0;

	virtual void EnableToolTipForceField(const bool bEnableForceField) = 0;
	virtual bool HasToolTipForceField() const = 0;
};

template<typename WidgetType>
class UE_DEPRECATED(5.8, "Please use FTypedElementLabelColumn and FAttributeBinder to bind the tooltip attribute to the widget instead")
	TTypedElementUiTooltipCapability
PRAGMA_DISABLE_DEPRECATION_WARNINGS
: public ITypedElementUiTooltipCapability
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
public:
	explicit TTypedElementUiTooltipCapability(WidgetType& InWidget) : Widget(InWidget){}

	void SetToolTipText(const TAttribute<FText>& ToolTipText) override
	{
		Widget.SetToolTipText(ToolTipText);	
	}

	void SetToolTipText(const FText& ToolTipText) override
	{
		Widget.SetToolTipText(ToolTipText);
	}

	void SetToolTip(const TAttribute<TSharedPtr<IToolTip>>& ToolTip) override
	{
		Widget.SetToolTip(ToolTip);
	}

	TSharedPtr<IToolTip> GetToolTip() override
	{
		return Widget.GetToolTip();
	}

	void EnableToolTipForceField(const bool bEnableForceField) override
	{
		Widget.EnableToolTipForceField(bEnableForceField);
	}

	bool HasToolTipForceField() const override
	{
		return Widget.HasToolTipForceField();
	}

private:
	WidgetType& Widget;
};
