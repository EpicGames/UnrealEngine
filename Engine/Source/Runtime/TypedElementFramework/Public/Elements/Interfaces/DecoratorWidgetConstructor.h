// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "TypedElementDataStorageInterface.h"
#include "TypedElementDataStorageUiInterface.h"
#include "Widgets/SWidget.h"

#include "DecoratorWidgetConstructor.generated.h"

/**
 * Base class used to for decorator widgets in TEDS UI.
 * A decorator widget is appended to regular TEDS UI widgets to provide additional functionality such as UI effects, context menus etc
 * A decorator widget constructor can be registered against a specific purpose and column using IUiProvider.
 * They will then be automatically added to any widgets where the widget row matches the purpose and column
 * (both on creation and dynamically post creation)
 */
USTRUCT()
struct FTedsDecoratorWidgetConstructor : public FTedsWidgetConstructorBase
{
	GENERATED_BODY()

public:
	TYPEDELEMENTFRAMEWORK_API explicit FTedsDecoratorWidgetConstructor(const UScriptStruct* InTypeInfo);
	explicit FTedsDecoratorWidgetConstructor(EForceInit) : FTedsWidgetConstructorBase(ForceInit) {} //< For compatibility and shouldn't be directly used.

	virtual ~FTedsDecoratorWidgetConstructor() = default;

	/**
	 * Construct a decorator widget that wraps InChildWidget
	 * @param InChildWidget The actual internal widget, it is expected that the decorator widget you create contains this widget as a child
	 * @param DataStorage Pointer to the TEDS interface
	 * @param DataStorageUi Pointer to the TEDS UI interface
	 * @param DataRow The row containing the actual data InChildWidget was created to observe (e.g Actor row) 
	 * @param WidgetRow The row containing UI specific data about InChildWidget (e.g widget color)
	 * @param Arguments Any metadata arguments specified when requesting this decorator
	 */
	virtual TSharedPtr<SWidget> CreateDecoratorWidget(
		TSharedPtr<SWidget> InChildWidget,
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle DataRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments);

protected:
	// The constructor's typeinfo
	TWeakObjectPtr<const UScriptStruct> TypeInfo = nullptr;
};

template<>
struct TStructOpsTypeTraits<FTedsDecoratorWidgetConstructor> : public TStructOpsTypeTraitsBase2<FTedsDecoratorWidgetConstructor>
{
	enum
	{
		WithNoInitConstructor = true,
		WithPureVirtual = true,
	};
};