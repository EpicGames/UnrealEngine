// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Features/IModularFeature.h"
#include "Templates/Function.h"
#include "UObject/Interface.h"
#include "UObject/ObjectKey.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

namespace UE::Editor::DataStorage
{
	class FObjectCompatibilityImmediateScope;

	/**
	 * Interface to provide compatibility with existing systems that don't directly
	 * support the data storage.
	 */
	class ICompatibilityProvider : public IModularFeature
	{
		friend class FObjectCompatibilityImmediateScope;
	public:
		using ObjectRegistrationFilter = TFunction<bool(const ICompatibilityProvider&, const UObject*)>;
		using ObjectToRowDealiaser = TFunction<RowHandle(const ICompatibilityProvider&, const UObject*)>;
	
		inline static const FName ObjectMappingDomain = "Object";

		/**
		 * @section Type-agnostic functions
		 * These allow compatibility with any type. These do eventually fall back to the explicit versions.
		 * Any references given are non-owning so it's up to the caller to deregister the object after it's no longer
		 * available.
		 */

		/** 
		 * Adds a reference to an existing object to the data storage. The data storage does NOT take ownership of the object and
		 * the caller is responsible for managing the life cycle of the object. The address is only used for associating the object
		 * with a row and to setup the initial row data.
		 */
		template<typename ObjectType>
		RowHandle AddCompatibleObject(ObjectType&& Object);
	
		/** Removes a previously registered object from the data storage. */
		template<typename ObjectType>
		void RemoveCompatibleObject(ObjectType&& Object);

		template<typename ObjectType>
		RowHandle FindRowWithCompatibleObject(ObjectType&& Object) const;

		/**
		 * @section Callback registration
		 * Functions to register callbacks with the compatibility layer to help refine its operations.
		 */
	 
		/**
		 * Objects like actors are registered through the compatibility layer in bulk. This can lead to objects being added that cause
		 * conflicts with other data in the data storage. This callback offers the opportunity to inspect the objects that are being
		 * added and if they include an object that shouldn't be store it can filter them out.
		 */
		virtual void RegisterRegistrationFilter(ObjectRegistrationFilter Filter) = 0;
		/**
		 * Notifications and request can be made to the compatibility layer for objects that are stored but don't directly map to a row.
		 * An example is a UObject represented by a column. If the UObject gets updated there's no direct mapping to the row the column is
		 * stored in but the row still needs to be updated. For cases like this it's possible to store information to find the row that's
		 * being aliased.
		 */
		virtual void RegisterDealiaserCallback(ObjectToRowDealiaser Dealiaser) = 0;
		/**
		 * Allows a specific type to be associated with a table. Whenever a compatible object is added, the type information of that object
		 * will be used to find the closest match in the registered types and use the associated table. E.g. actors derive from uobjects so
		 * if the type information of an actor is registered the actor table will be used instead of the uobject table.
		 */
		virtual void RegisterTypeTableAssociation(TWeakObjectPtr<UStruct> TypeInfo, TableHandle Table) = 0;

		/**
		 * @section Explicit functions
		 * These are functions that work on specific types.
		 */

		/** Adds a UObject to the data storage. */
		virtual RowHandle AddCompatibleObjectExplicit(UObject* Object) = 0;
		/** Adds an FStruct to the data storage. */
		virtual RowHandle AddCompatibleObjectExplicit(void* Object, TWeakObjectPtr<const UScriptStruct> TypeInfo) = 0;
	
		/** Removes a UObject from the data storage. */
		virtual void RemoveCompatibleObjectExplicit(UObject* Object) = 0;
		/** Removes an FStruct from the data storage. */
		virtual void RemoveCompatibleObjectExplicit(void* Object) = 0;

		/** Finds a previously stored UObject. If not found an invalid row handle will be returned. */
		virtual RowHandle FindRowWithCompatibleObjectExplicit(const UObject* Object) const = 0;
		/** Finds a previously stored FStruct. If not found an invalid row handle will be returned. */
		virtual RowHandle FindRowWithCompatibleObjectExplicit(const void* Object) const = 0;

		/**
		 * @section Miscellaneous functions
		 */

		/** Check if a custom extension is supported. This can be used to check for in-development features, custom extensions, etc. */
		virtual bool SupportsExtension(FName Extension) const = 0;
		/** Provides a list of all extensions that are enabled. */
		virtual void ListExtensions(TFunctionRef<void(FName)> Callback) const = 0;

	protected:
		/**
		 * @section Immediate mode scope functions
		 */

		struct UE_INTERNAL FObjectCompatibilityImmediateScopeWriteAccess
		{
			static void RegisterObject(FObjectCompatibilityImmediateScope& Scope, 
				void* Object, const UScriptStruct* TypeInfo, RowHandle Row);
			static void SetPrevious(FObjectCompatibilityImmediateScope& Scope, FObjectCompatibilityImmediateScope* InPrevious);
			static FObjectCompatibilityImmediateScope* GetPrevious(const FObjectCompatibilityImmediateScope& Scope);

			static void ListUObjects(const FObjectCompatibilityImmediateScope& Scope, 
				TFunctionRef<void(RowHandle, UObject*, const UClass*)> Callback);
			static void ListExternalObjects(const FObjectCompatibilityImmediateScope& Scope, 
				TFunctionRef<void(RowHandle, void*, const UScriptStruct*)> Callback);
		};
		UE_INTERNAL
		virtual void EnterImmediateMode(FObjectCompatibilityImmediateScope& Scope) = 0;
		UE_INTERNAL
		virtual void LeaveImmediateMode(FObjectCompatibilityImmediateScope& Scope) = 0;
	};

	/**
	 * Scope object to tell TEDS Compatibility that any objects being registered on that thread with this scope should be
	 * immediately registered instead of queued up once this goes out of scope.
	 *
	 * Scopes can be nested and are able to provide the row handles to the objects that have been newly or previously
	 * registered. The scope is local to the calling thread so other threads will continue to queue registrations.
	 * 
	 * The immediate mode limits how much batch and distributed registration TEDS Compatibility can do. Avoid using
	 * this scope when possible and prefer to limit the use to a small number of objects. This scope was introduced to
	 * aid the transition to the multi-threaded environment TEDS provides and as such this scope may be removed in the future.
	 * 
	 * Warning: This scope is to aid the transition of existing single-threaded systems over to the editor data storage.
	 */
	class UE_INTERNAL FObjectCompatibilityImmediateScope
	{
		PRAGMA_DISABLE_INTERNAL_WARNINGS
		friend struct ICompatibilityProvider::FObjectCompatibilityImmediateScopeWriteAccess;
		PRAGMA_ENABLE_INTERNAL_WARNINGS
	public:
		explicit FObjectCompatibilityImmediateScope(ICompatibilityProvider& InCompatibility);
		~FObjectCompatibilityImmediateScope();

		RowHandle FindObject(const void* Object) const;
		int32 GetObjectCount() const;

	private:
		struct ObjectInfoType
		{
			void* Address;
			const UScriptStruct* TypeInfo; // If this is null, Address points to a UObject.
			RowHandle Row;
		};
		TArray<ObjectInfoType> Objects;
		FObjectCompatibilityImmediateScope* Previous = nullptr;
		ICompatibilityProvider& Compatibility;
	};

	// 
	// Implementations
	// 

	PRAGMA_DISABLE_INTERNAL_WARNINGS // For FObjectCompatibilityImmediateScope and FObjectCompatibilityImmediateScopeWriteAccess

	inline FObjectCompatibilityImmediateScope::FObjectCompatibilityImmediateScope(ICompatibilityProvider& InCompatibility)
		: Compatibility(InCompatibility)
	{
		InCompatibility.EnterImmediateMode(*this);
	}

	inline FObjectCompatibilityImmediateScope::~FObjectCompatibilityImmediateScope()
	{
		Compatibility.LeaveImmediateMode(*this);
	}

	inline RowHandle FObjectCompatibilityImmediateScope::FindObject(const void* Object) const
	{
		const ObjectInfoType* Result = Objects.FindByPredicate([Object](const ObjectInfoType& Info) { return Info.Address == Object; });
		return Result ? Result->Row : InvalidRowHandle;
	}
	inline int32 FObjectCompatibilityImmediateScope::GetObjectCount() const { return Objects.Num(); }
	
	inline void ICompatibilityProvider::FObjectCompatibilityImmediateScopeWriteAccess::RegisterObject(
		FObjectCompatibilityImmediateScope& Scope, void* Object, const UScriptStruct* TypeInfo, RowHandle Row)
	{
		Scope.Objects.Add(
			FObjectCompatibilityImmediateScope::ObjectInfoType
			{
				.Address = Object,
				.TypeInfo = TypeInfo,
				.Row = Row
			});
	}
	

	inline void ICompatibilityProvider::FObjectCompatibilityImmediateScopeWriteAccess::SetPrevious(
		FObjectCompatibilityImmediateScope& Scope, FObjectCompatibilityImmediateScope* InPrevious)
	{
		checkf(Scope.Previous == nullptr, TEXT("A previous scope has already been set for FObjectCompatibilityImmediateScope."));
		Scope.Previous = InPrevious;
	}

	inline FObjectCompatibilityImmediateScope* ICompatibilityProvider::FObjectCompatibilityImmediateScopeWriteAccess::GetPrevious(
		const FObjectCompatibilityImmediateScope& Scope)
	{ 
		return Scope.Previous;
	}

	inline void ICompatibilityProvider::FObjectCompatibilityImmediateScopeWriteAccess::ListUObjects(
		const FObjectCompatibilityImmediateScope& Scope, TFunctionRef<void(RowHandle, UObject*, const UClass*)> Callback)
	{
		for (const FObjectCompatibilityImmediateScope::ObjectInfoType& Entry : Scope.Objects)
		{
			if (!Entry.TypeInfo)
			{
				Callback(Entry.Row, static_cast<UObject*>(Entry.Address), static_cast<UObject*>(Entry.Address)->GetClass());
			}
		}
	}

	inline void ICompatibilityProvider::FObjectCompatibilityImmediateScopeWriteAccess::ListExternalObjects(
		const FObjectCompatibilityImmediateScope& Scope, TFunctionRef<void(RowHandle, void*, const UScriptStruct*)> Callback)
	{
		for (const FObjectCompatibilityImmediateScope::ObjectInfoType& Entry : Scope.Objects)
		{
			if (Entry.TypeInfo)
			{ 
				Callback(Entry.Row, Entry.Address, Entry.TypeInfo);
			}
		}
	}

	PRAGMA_ENABLE_INTERNAL_WARNINGS // For FObjectCompatibilityImmediateScope and FObjectCompatibilityImmediateScopeWriteAccess
	
	template<typename Type> Type* GetRawPointer(const TWeakObjectPtr<Type> Object)	{ return Object.Get(); }
	template<typename Type> Type* GetRawPointer(const TObjectPtr<Type> Object)		{ return Object.Get(); }
	template<typename Type> Type* GetRawPointer(const TStrongObjectPtr<Type> Object){ return Object.Get(); }
	template<typename Type> Type* GetRawPointer(const TObjectKey<Type> Object)		{ return Object.ResolveObjectPtr(); }
	template<typename Type> Type* GetRawPointer(const TUniquePtr<Type> Object)		{ return Object.Get(); }
	template<typename Type> Type* GetRawPointer(const TSharedPtr<Type> Object)		{ return Object.Get(); }
	#if UE_ENABLE_NOTNULL_WRAPPER
	template<typename Type> Type* GetRawPointer(TNotNull<Type*> Object)				{ return Object; }
	#endif
	template<typename Type> Type* GetRawPointer(Type* Object)						{ return Object; }
	template<typename Type> Type* GetRawPointer(Type& Object)						{ return &Object; }

	template<typename ObjectType>
	RowHandle ICompatibilityProvider::AddCompatibleObject(ObjectType&& Object)
	{
		auto RawPointer = GetRawPointer(Forward<ObjectType>(Object));
		using BaseType = std::remove_cv_t<std::remove_pointer_t<decltype(RawPointer)>>;

		if constexpr (std::is_base_of_v<UObject, BaseType>)
		{
			return AddCompatibleObjectExplicit(RawPointer);
		}
		else
		{
			return AddCompatibleObjectExplicit(RawPointer, BaseType::StaticStruct());
		}
	}

	template<typename ObjectType>
	void ICompatibilityProvider::RemoveCompatibleObject(ObjectType&& Object)
	{
		RemoveCompatibleObjectExplicit(GetRawPointer(Forward<ObjectType>(Object)));
	}

	template<typename ObjectType>
	RowHandle ICompatibilityProvider::FindRowWithCompatibleObject(ObjectType&& Object) const
	{
		return FindRowWithCompatibleObjectExplicit(GetRawPointer(Forward<ObjectType>(Object)));
	}

	template<typename Subsystem>
	struct TTypedElementSubsystemTraits final
	{
		template<typename T, typename = void>
		struct HasRequiresGameThreadVariable 
		{ 
			static constexpr bool bAvailable = false; 
		};
		template<typename T>
		struct HasRequiresGameThreadVariable <T, decltype((void)T::bRequiresGameThread)>
		{ 
			static constexpr bool bAvailable = true; 
		};

		template<typename T, typename = void>
		struct HasIsHotReloadableVariable
		{ 
			static constexpr bool bAvailable = false;
		};
		template<typename T>
		struct HasIsHotReloadableVariable <T, decltype((void)T::bIsHotReloadable)>
		{ 
			static constexpr bool bAvailable = true;
		};

		static constexpr bool RequiresGameThread()
		{
			if constexpr (HasRequiresGameThreadVariable<Subsystem>::bAvailable)
			{
				return Subsystem::bRequiresGameThread;
			}
			else
			{
				static_assert(HasRequiresGameThreadVariable<Subsystem>::bAvailable, "Subsystem provided to the Typed Elements did not "
					"have a 'static constexpr bool bRequiresGameThread = true|false` declared or have a specialization for "
					"TTypedElementSubsystemTraits.");
				return true;
			}
		}

		static constexpr bool IsHotReloadable()
		{
			if constexpr (HasIsHotReloadableVariable<Subsystem>::bAvailable)
			{
				return Subsystem::bIsHotReloadable;
			}
			else
			{
				static_assert(HasIsHotReloadableVariable<Subsystem>::bAvailable, "Subsystem provided to the Typed Elements did not "
					"have a 'static constexpr bool bIsHotReloadable = true|false` declared or have a specialization for "
					"TTypedElementSubsystemTraits.");
				return false;
			}
		}
	};
} // namespace UE::Editor::DataStorage
