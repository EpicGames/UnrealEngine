// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TypedElementDataStorageFactory.generated.h"

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
	class IUiProvider;
	class ICompatibilityProvider;
}

/**
 * Base class that can be used to register various elements, such as queries and widgets, with
 * the Editor Data Storage.
 */
UCLASS(MinimalAPI)
class UEditorDataStorageFactory : public UObject
{
	GENERATED_BODY()

public:
	~UEditorDataStorageFactory() override = default;

	/** 
	 * Returns the order registration will be executed. Factories with a lower number will be executed
	 * before factories with a higher number.
	 */
	virtual uint8 GetOrder() const { return 127; }

	/**
	 * All factories will have this called before any Register functions on any factories are called
	 */
	virtual void PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage) {}

	/**
	 * All factories will have this called after every Register functions on all factories are called
	 */
	virtual void PostRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage) {}

	virtual void RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage) {}
	virtual void RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::ICompatibilityProvider& DataStorageCompatibility) {}
	virtual void RegisterTickGroups(UE::Editor::DataStorage::ICoreProvider& DataStorage) const {}
	virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) {}
	
	virtual void RegisterHierarchies(UE::Editor::DataStorage::ICoreProvider& DataStorage) {}

	virtual void RegisterRegistrationFilters(UE::Editor::DataStorage::ICompatibilityProvider& DataStorageCompatibility) const {}
	virtual void RegisterDealiaser(UE::Editor::DataStorage::ICompatibilityProvider& DataStorageCompatibility) const {}
	
	virtual void RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const {}
	virtual void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const {}
		
	/** 
     * Used to register searchers for a specific property type to be used in the default generated property searcher. The widget constructor
     * can be used to provide a custom searcher if more complex searching logic is needed. Implementations of this function should mostly be
     * calling `RegisterSearcherGeneratorForProperty` on the UI provider.
     */
    virtual void RegisterPropertySearchers(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const {}

	/** 
	 * Used to register sorters for a specific property type to be used in the default generated property sorter. The widget constructor
	 * can be used to provide a custom sorter if more complex sorting logic is needed. Implementations of this function should mostly be
	 * calling `RegisterSorterGeneratorForProperty` on the UI provider.
	 */
	virtual void RegisterPropertySorters(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const {}

	/**
	 * Called in reverse order before the DataStorage object is shut down
	 */
	virtual void PreShutdown(UE::Editor::DataStorage::ICoreProvider& DataStorage) {}
};
