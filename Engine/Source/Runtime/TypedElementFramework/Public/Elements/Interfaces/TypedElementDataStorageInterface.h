// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "DataStorage/CommonTypes.h"
#include "DataStorage/Scope/EditorDataScopeTypes.h"
#include "DataStorage/Handles.h"
#include "DataStorage/MapKey.h"
#include "DataStorage/Queries/Description.h"
#include "DataStorage/Queries/Types.h"
#include "DataStorage/Queries/Conditions.h"
#include "Delegates/Delegate.h"
#include "Elements/Framework/TypedElementColumnUtils.h"
#include "Elements/Framework/TypedElementQueryCapabilities.h"
#include "Elements/Framework/TypedElementQueryFunctions.h"
#include "Elements/Interfaces/TypedElementQueryStorageInterfaces.h"
#include "Elements/Framework/TypedElementRowHandleArray.h"
#include "Elements/Framework/TypedElementRowHandleArrayView.h"
#include "Features/IModularFeature.h"
#include "Math/NumericLimits.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Timespan.h"
#include "Templates/Function.h"
#include "UObject/Interface.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"

class UClass;
class USubsystem;
class UScriptStruct;
class UEditorDataStorageFactory;

namespace UE::Editor::DataStorage
{
using FTypedElementOnDataStorageCreation = FSimpleMulticastDelegate;
using FTypedElementOnDataStorageDestruction = FSimpleMulticastDelegate;
using FTypedElementOnDataStorageUpdate = FSimpleMulticastDelegate;

/**
 * Convenience structure that can be used to pass a list of columns to functions that don't
 * have an dedicate templated version that takes a column list directly, for instance when
 * multiple column lists are used. Note that the returned array view is only available while
 * this object is constructed, so care must be taken with functions that return a const array view.
 */
template<TColumnType... Columns>
struct TTypedElementColumnTypeList
{
	const UScriptStruct* ColumnTypes[sizeof...(Columns)] = { Columns::StaticStruct()... };
	
	operator TConstArrayView<const UScriptStruct*>() const { return ColumnTypes; }
};

struct FHierarchyRegistrationParams
{
	FName Name;

	// Optionally add a column to any row participating in the hierarchy if its parent changes (useful for change detection).
	// This behaves similar to the sync tags and is removed at the end of the frame.
	// For the Relations backend this maps to bEnableSubjectChangedColumn on the underlying relation type.
	bool bEnableParentChangedColumn = false;

	UE_DEPRECATED(5.8, "Providing an explicit column has been deprecated, use the bool argument + ICoreProvider::GetParentChangedColumn()")
	const UScriptStruct* ParentChangedColumn = nullptr;

	/** Selects which storage backend implements this hierarchy. */
	enum class EBackend : uint8
	{
		/** Column-on-row storage (FTedsHierarchyRegistrar). Existing behavior. */
		Legacy,
		/** Relation-row storage with interval encoding for O(1) descendant queries. */
		Relations
	};
	EBackend Backend = EBackend::Legacy;
};

/**
 * Configuration for a role (Subject or Object) in a relation type.
 */
struct FTedsRelationRoleTraits
{
	/**
	 * Whether only one entity can hold this role per relation.
	 * For hierarchies: Object (parent) is typically exclusive (one parent per child).
	 */
	bool bExclusive = false;

	/**
	 * Policy for what happens when the entity in this role is destroyed.
	 */
	enum class EDestructionPolicy : uint8
	{
		/** Just remove the relation row, leave other participant unchanged */
		CleanUp,
		/** Destroy the entity in the other role (e.g., destroy child when parent is deleted) */
		Cascade,
		/** Leave the other participant without relation (orphan it) */
		Orphan
	};
	EDestructionPolicy DestructionPolicy = EDestructionPolicy::CleanUp;
};

/** Controls how hierarchical queries (IsDescendantOf, etc.) are implemented for a relation type. */
enum class EHierarchyMode : uint8
{
	/** Not a hierarchy. Hierarchical APIs are not available. */
	Disabled,
	/** IsDescendantOf walks the parent chain. Always correct, O(depth). */
	WalkOnly,
	/** IsDescendantOf uses interval encoding for O(1) checks. Falls back to walk when intervals
	 *  are stale; intervals are automatically refreshed at frame end. Recommended for dynamic hierarchies. */
	IntervalEncoded,
};

/**
 * Configuration for registering a new relation type.
 * Relation types define how two rows can be connected.
 */
struct FTedsRelationTraits
{
	/** Configuration for the Subject role (the "from" / "child" side) */
	FTedsRelationRoleTraits Subject;

	/** Configuration for the Object role (the "to" / "parent" side) */
	FTedsRelationRoleTraits Object;

	/** Selects how hierarchical descendant queries are implemented for this relation type. */
	EHierarchyMode HierarchyMode = EHierarchyMode::Disabled;

	/**
	 * Initial gap between interval values for new nodes.
	 * Larger gaps allow more insertions before rebalancing is needed.
	 * Only used when HierarchyMode is IntervalEncoded.
	 */
	int64 IntervalGap = 1000;
};

/**
 * Parameters for registering a relation type.
 */
struct FRelationRegistrationParams
{
	/** Unique name for this relation type */
	FName Name;

	/** Relation type configuration */
	FTedsRelationTraits Traits;

	/**
	 * When true, a tag column is stamped on the Subject row whenever a relation of this type
	 * is created or destroyed with that row as Subject. The column is removed at FrameEnd.
	 * Retrieve the column type via ICoreProvider::GetRelationSubjectChangedColumn.
	 * Use with FObserver::OnAdd or a FProcessor Where().All(col) for change detection.
	 */
	bool bEnableSubjectChangedColumn = false;

	/**
	 * When true, a tag column is stamped on the Object row whenever a relation of this type
	 * is created or destroyed with that row as Object. The column is removed at FrameEnd.
	 * Retrieve the column type via ICoreProvider::GetRelationObjectChangedColumn.
	 */
	bool bEnableObjectChangedColumn = false;
};

class ICoreProvider : public 
	Queries::ITableInfo<
	Queries::ITableManagement<
	Queries::IDynamicColumnInfo<
	IModularFeature>>>
{
public:
	/**
	 * @section Factories
	 *
	 * @description
	 * Factories are an automated way to register tables, queries and other information with TEDS.
	 */

	/** Finds a factory instance registered with TEDS */
	virtual const UEditorDataStorageFactory* FindFactory(const UClass* FactoryType) const = 0;

	virtual UEditorDataStorageFactory* FindFactory(const UClass* FactoryType) = 0;

	/** Convenience function for FindFactory */
	template<typename FactoryT>
	const FactoryT* FindFactory() const;
	
	template<typename FactoryT>
	FactoryT* FindFactory();

	/** 
	 * Reserves a row to be assigned to a table at a later point. If the row is no longer needed before it's been assigned
	 * to a table, it should still be released with RemoveRow.
	 */
	virtual RowHandle ReserveRow() = 0;
	/**
	 * Reserve multiple rows at once to be assigned to a table at a later point. If multiple rows are needed, the batch version will
	 * generally have better performance. If a row is no longer needed before it's been assigned to a table, it should still be released 
	 * with RemoveRow.
	 * The reservation callback will be called once per reserved row.
	 */
	virtual void BatchReserveRows(int32 Count, TFunctionRef<void(RowHandle)> ReservationCallback) = 0;
	/**
	 * Reserve multiple rows at once to be assigned to a table at a later point. If multiple rows are needed, the batch version will
	 * generally have better performance. If a row is no longer needed before it's been assigned to a table, it should still be released
	 * with RemoveRow.
	 * The provided range will be have its values set to the reserved row handles.
	 */
	virtual void BatchReserveRows(TArrayView<RowHandle> ReservedRows) = 0;

	/** Adds a new row to the provided table. */
	virtual RowHandle AddRow(TableHandle Table) = 0;
	/**
	 * Adds a new row to the provided table. Callers are expected to use the callback to
	 * initialize the row if needed.
	 */
	virtual RowHandle AddRow(TableHandle Table, RowCreationCallbackRef OnCreated) = 0;
	/** Adds a new row to the provided table using a previously reserved row. */
	virtual bool AddRow(RowHandle ReservedRow, TableHandle Table) = 0;
	/**
	 * Adds a new row to the provided table using a previously reserved row. Callers are expected to use the callback to
	 * initialize the row if needed.
	 */
	virtual bool AddRow(RowHandle ReservedRow, TableHandle Table, RowCreationCallbackRef OnCreated) = 0;

	/**
	 * Add multiple rows at once. For each new row the OnCreated callback is called. Callers are expected to use the callback to
	 * initialize the row if needed.
	 */
	virtual bool BatchAddRow(TableHandle Table, int32 Count, RowCreationCallbackRef OnCreated) = 0;
	/**
	 * Add multiple rows at once. For each new row the OnCreated callback is called. Callers are expected to use the callback to
	 * initialize the row if needed. This version uses a set of previously reserved rows. Any row that can't be used will be 
	 * released.
	 */
	virtual bool BatchAddRow(TableHandle Table, TConstArrayView<RowHandle> ReservedHandles, RowCreationCallbackRef OnCreated) = 0;

	/** Removes a previously reserved or added row. If the row handle is invalid or already removed, nothing happens */
	virtual void RemoveRow(RowHandle Row) = 0;

	/** Removes multiple rows at one. If any of the rows are invalid or already removed, nothing happens. */
	virtual void BatchRemoveRows(TConstArrayView<RowHandle> Rows) = 0;

	/** 
	 * Removes all rows that have at least the provided columns.
	 * This can be used to for instance remove all rows with a specific type tag, such as removing all rows with
	 * an entity tag to remove all entities from the data storage.
	 */
	virtual void RemoveAllRowsWithColumns(TConstArrayView<const UScriptStruct*> Columns) = 0;

	/**
	 * Removes all rows that have at least the provided columns as template arguments.
	 * This can be used to for instance remove all rows with a specific type tag, such as removing all rows with
	 * an entity tag to remove all entities from the data storage.
	 */
	template<TColumnType... Columns>
	void RemoveAllRowsWith();

	/** Checks whether or not a row is in use. This is true even if the row has only been reserved. */
	virtual bool IsRowAvailable(RowHandle Row) const = 0;
	/** Checks whether or not a row has been reserved but not yet assigned to a table. */
	virtual bool IsRowAssigned(RowHandle Row) const = 0;

	enum class EFilterOptions : uint8
	{
		/** Only include rows in the result that have the required columns and pass the filter function */
		Default = 0,
		/** Invert the filter to return rows that did not pass the filter function */
		InvertFilter = 1 << 0,
	};

	/** Filters the provided rows using the query filter and stores the results in the provided list. */
	virtual void FilterRowsBy(
		FRowHandleArray& Result, FRowHandleArrayView Input, EFilterOptions Options, Queries::TConstQueryFunction<bool>& Filter) const = 0;
	/** Filters the provided rows using the query filter and stores the results in the provided list. */
	template<typename FilterFunction>
	void FilterRowsBy(FRowHandleArray& Result, FRowHandleArrayView Input, EFilterOptions Options, FilterFunction&& Filter) const;

	/**
	 * @section Column management
	 */

	/** Adds a column to a row or does nothing if already added. */
	virtual void AddColumn(RowHandle Row, const UScriptStruct* ColumnType) = 0;
	template<TColumnType ColumnType>
	void AddColumn(RowHandle Row);
	/**
	 * Adds a new data column and initializes it. The relocator will be used to copy or move the column out of
	 * its temporary location into the final table if the addition needs to be deferred.
	 */
	virtual void AddColumnData(RowHandle Row, const UScriptStruct* ColumnType,
		const ColumnCreationCallbackRef Initializer,
		ColumnCopyOrMoveCallback Relocator) = 0;
	template<TDataColumnType ColumnType>
	void AddColumn(RowHandle Row, ColumnType&& Column);

	/**
	 * Adds a ValueTag with the given value to a row
	 * A row can have multiple ValueTags, but only one of each tag type.
	 * Example:
	 *   AddColumn(Row, ValueTag(TEXT("Color"), TEXT("Red));     // Valid
	 *   AddColumn(Row, ValueTag(TEXT("Direction"), TEXT("Up")); // Valid
	 *   AddColumn(Row, ValueTag(TEXT("Color"), TEXT("Blue"));   // Will do nothing since there already exists a Color value tag
	 * Note: Current support for changing a value tag from one value to another requires that the tag is removed before a new one
	 *       is added.  This will likely change in the future to transparently replace the tag to have consistent behaviour with other usages
	 *       of AddColumn
	 */
	virtual void AddColumn(RowHandle Row, const FValueTag& Tag, const FName& Value) = 0;

	template<typename T>
	void AddColumn(RowHandle Row, const FName& Tag) = delete;
	
	template<typename T>
	void AddColumn(RowHandle Row, const FName& Tag, const FName& Value) = delete;
	
	template<>
	void AddColumn<FValueTag>(RowHandle Row, const FName& Tag, const FName& Value);

	template<TEnumType EnumT>
	void AddColumn(RowHandle Row, EnumT Value);
	
	template<auto Value, TEnumType EnumT = decltype(Value)>
	void AddColumn(RowHandle Row);

	template<TDynamicColumnTemplate DynamicColumnTemplate>
	void AddColumn(RowHandle Row, const FName& Identifier);
	
	template<TDynamicColumnTemplate DynamicColumnTemplate>
	void AddColumn(RowHandle Row, const FName& Identifier, DynamicColumnTemplate&& TemplateInstance);

	/**
	 * Adds multiple columns from a row. This is typically more efficient than adding columns one 
	 * at a time.
	 */
	virtual void AddColumns(RowHandle Row, TConstArrayView<const UScriptStruct*> Columns) = 0;
	template<TColumnType... Columns>
	void AddColumns(RowHandle Row);

	/** Removes a column from a row or does nothing if already removed. */
	virtual void RemoveColumn(RowHandle Row, const UScriptStruct* ColumnType) = 0;
	template<TColumnType Column>
	void RemoveColumn(RowHandle Row);

	template<TEnumType EnumT>
	void RemoveColumn(RowHandle Row);

	/**
	 * Removes a value tag from the given row
	 * If tag does not exist on row, operation will do nothing.
	 */
	virtual void RemoveColumn(RowHandle Row, const FValueTag& Tag) = 0;

	template<typename T>
	void RemoveColumn(RowHandle Row, const FName& Tag) = delete;

	template<TDynamicColumnTemplate DynamicColumnTemplateType>
	void RemoveColumn(RowHandle Row, const FName& Identifier);
	
	template<>
	void RemoveColumn<FValueTag>(RowHandle Row, const FName& Tag);

	/**
	 * Removes multiple columns from a row. This is typically more efficient than adding columns one
	 * at a time.
	 */
	virtual void RemoveColumns(RowHandle Row, TConstArrayView<const UScriptStruct*> Columns) = 0;
	template<TColumnType... Columns>
	void RemoveColumns(RowHandle Row);

	/** 
	 * Adds and removes the provided column types from the provided row. This is typically more efficient 
	 * than individually adding and removing columns as well as being faster than adding and removing
	 * columns separately.
	 */
	virtual void AddRemoveColumns(RowHandle Row, TConstArrayView<const UScriptStruct*> ColumnsToAdd,
		TConstArrayView<const UScriptStruct*> ColumnsToRemove) = 0;
	
	/** Adds and removes the provided column types from the provided list of rows. */
	virtual void BatchAddRemoveColumns(
		TConstArrayView<RowHandle> Rows,
		TConstArrayView<const UScriptStruct*> ColumnsToAdd,
		TConstArrayView<const UScriptStruct*> ColumnsToRemove) = 0;
	
	/** Retrieves a pointer to the column of the given row or a nullptr if not found or if the column type is a tag. */
	virtual void* GetColumnData(RowHandle Row, const UScriptStruct* ColumnType) = 0;
	virtual const void* GetColumnData(RowHandle Row, const UScriptStruct* ColumnType) const = 0;
	/** Returns a pointer to the column of the given row or a nullptr if the type couldn't be found or the row doesn't exist. */
	template<TDataColumnType ColumnType>
	ColumnType* GetColumn(RowHandle Row);
	template<TDataColumnType ColumnType>
	const ColumnType* GetColumn(RowHandle Row) const;
	// Gets a dynamic column identified by the ColumnTypeTemplate and Identifier
	template<TDynamicColumnTemplate ColumnTypeTemplate>
	ColumnTypeTemplate* GetColumn(RowHandle Row, const FName& Identifer);
	template<TDynamicColumnTemplate ColumnTypeTemplate>
	const ColumnTypeTemplate* GetColumn(RowHandle Row, const FName& Identifer) const;
	
	/** 
	 * Determines if the provided row contains the collection of columns and tags.
	 * Note that for rows that haven't been assigned yet it's not possible to check if a column exists as the table to check for hasn't
	 * been assigned yet. In these cases a list of known additions will be checked as these will be added once the table is assigned.
	 */
	virtual bool HasColumns(RowHandle Row, TConstArrayView<const UScriptStruct*> ColumnTypes) const = 0;
	virtual bool HasColumns(RowHandle Row, TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ColumnTypes) const = 0;
	template<TColumnType... ColumnTypes>
	bool HasColumns(RowHandle Row) const;
	template<Queries::TColumnDescType... ColumnTypes>
	bool HasColumns(RowHandle Row, const ColumnTypes&... Columns) const;

	/** 
	 * Lists the columns on a row. This includes data and tag columns.
	 * Note that for rows that haven't been assigned yet it's not possible to return the full list as no table has been assigned yet.
	 * In these cases a list of known additions will be returned as these will be added once the table is assigned.
	 */
	virtual void ListColumns(RowHandle Row, ColumnListCallbackRef Callback) const = 0;

	/** 
	 * Lists the column type and data on a row. This includes data and tag columns. Not all columns may have data so the data pointer in 
	 * the callback can be null.
	 * Note that for rows that haven't been assigned yet it's not possible to return the full list as no table has been assigned yet.
	 * In these cases a list of known additions will be returned.
	 */
	virtual void ListColumns(RowHandle Row, ColumnListWithDataCallbackRef Callback) = 0;

	/** Determines if the columns in the row match the query conditions. */
	virtual bool MatchesColumns(RowHandle Row, const Queries::FConditions& Conditions) const = 0;

	/**
	 * Generates a new dynamic column from a Template.  A dynamic column is uniquely identified using the given template and an Identifier
	 * This function is idempotent - multiple calls with the same parameters will result in subsequent calls returning the same type
	 * The TemplateType may be a typed derived from either FColumn or FTag
	 */
	virtual const UScriptStruct* GenerateDynamicColumn(const FDynamicColumnDescription& Description) = 0;

	/**
	 * Executes the given callback for each known dynamic column that derives from the base template provided
	 */
	virtual void ForEachDynamicColumn(const UScriptStruct& Template, TFunctionRef<void(const UScriptStruct& Type)> Callback) const = 0;

	/**
	 * Executes the given callback for each known dynamic column that derives from the base template provided
	 */
	template<TDynamicColumnTemplate DynamicColumnTemplateType>
	void ForEachDynamicColumn(TFunctionRef<void(const UScriptStruct& Type)> Callback) const;

	// Hierarchy Interface
	//-----------------------
	
	virtual UE::Editor::DataStorage::FHierarchyHandle RegisterHierarchy(const UE::Editor::DataStorage::FHierarchyRegistrationParams& Params) = 0;
	virtual UE::Editor::DataStorage::FHierarchyHandle FindHierarchyByName(const FName& Name) const = 0;
	virtual bool IsValidHierarchyHandle(FHierarchyHandle) const = 0;

	// Gets the tag type indicating the row is a child in this hierarchy and thus has a parent
	virtual const UScriptStruct* GetChildTagType(FHierarchyHandle InHierarchyHandle) const = 0;
	// Gets the tag type indicating the row is a parent in this hierarchy and thus has a child
	virtual const UScriptStruct* GetParentTagType(FHierarchyHandle InHierarchyHandle) const = 0;
	// Gets the column type storing the hierarchy data. Note that the format of this type is opaque
	// and is used internally.  It is the caller's responsibility to provide at least Read access
	// to this column type in query requirements in order to use the HierarchyAccessInterface on the row
	virtual const UScriptStruct* GetHierarchyDataColumnType(FHierarchyHandle InHierarchyHandle) const = 0;

	// Gets the column type that is added to a row for a frame when the parent changes (if specified in RegisterHierarchy)
	virtual const UScriptStruct* GetParentChangedColumnType(FHierarchyHandle InHierarchyHandle) const = 0;
	
	// Try to resolve every row that has an Unresolved Parent for the given hierarchy (@see SetUnresolvedParent)
	// Unresolved hierarchies are evaluated by a processor every frame to see if the parent is known, but this function can be manually called
	// to immediately try to resolve all rows for a particular hierarchy
	virtual void ResolveHierarchy(FHierarchyHandle InHierarchyHandle) = 0;

	// List all the currently known hierarchies by name
	virtual void ListHierarchyNames(TFunctionRef<void(const FName& HierarchyName)> Callback) const = 0;

	// CoreProvider Context API
	// =========================

	// Establishes a parent relationship between the Target and the Parent
	virtual void SetParentRow(
		FHierarchyHandle InHierarchyHandle,
		RowHandle Target,
		RowHandle Parent) = 0;

	/**
	 * Establishes a parent relationship between the Target row and a Parent that is not registered in TEDS yet.
	 * Every frame, TEDS will attempt to resolve the missing relation by looking up the parent using TEDS Mapping.
	 * An optional MaxFrameCount can be specified, which will be decremented every frame until the potential relationship is discarded
	 */
	virtual void SetUnresolvedParent(
		FHierarchyHandle InHierarchyHandle,
		RowHandle Target,
		FMapKey ParentId,
		FName MappingDomain) = 0;

	// Gets the parent of a target, if there is one
	virtual RowHandle GetParentRow(FHierarchyHandle InHierarchyHandle, RowHandle Target) const = 0;
	
	// Returns a callable which will extract the parent row from the hierarchy's HierarchyDataColumn
	// Note that the second parameter must be the same as what is returned by GetHierarchyDataColumn()
	// and the first parameter should point to a struct of that type
	virtual TFunction<RowHandle(const void*, const UScriptStruct*)> CreateParentExtractionFunction(FHierarchyHandle InHierarchyHandle) const = 0; 
	
	virtual bool HasChildren(FHierarchyHandle InHierarchyHandle, RowHandle Row) const = 0;

	using FHierarchyIterationCallback = TFunction<void(const ICoreProvider& Context, RowHandle Owner, RowHandle Target)>;
	
	enum class ETraversalOrder : uint8
	{
		/** Specifies that a pre-order traversal should be performed. In a pre-order traversal, rows are visited before their children. */
		PreOrder,
		/** Specifies that a post-order traversal should be performed. In a post-order traversal, rows are visited after their children. */
		PostOrder,
	};

	// Iterates depth first from the passed in Row, calling the VisitFn on each row
	// The Row passed in will have VisitFn called on it
	virtual void WalkDepthFirst(
		FHierarchyHandle InHierarchyHandle,
		RowHandle Row,
		FHierarchyIterationCallback VisitFn,
		ETraversalOrder TraversalOrder = ETraversalOrder::PreOrder) const = 0;
	
	/**
	 * Iterates the immediate children of the given row without descending further
	 *  down the hierarchy.
	 * 
	 * @param InHierarchyHandle Hierarchy to use
	 * @param Row Parent row (not included in iteration)
	 * @param VisitFn Iteration function that is expected to return true if iteration should continue
	 *  or false if it should be stopped early.
	 * @return true if iteration completed (including if there were no children) without any VisitFn call returning
	 *  false. Returns false if VisitFn stopped iteration early or there was an error.
	 */
	virtual bool IterateChildren(
		FHierarchyHandle InHierarchyHandle,
		RowHandle Row,
		TFunctionRef<bool(const ICoreProvider& Context, RowHandle Child)> VisitFn) const
	{
		return false;
	}

	/**
	 * Outputs the registered query callbacks to the given output device for debugging purposes.
	 */
	virtual void DebugPrintQueryCallbacks(FOutputDevice& Output) = 0;

	// Relations Interface
	// ===================
	// Relations provide a way to connect two rows together. Unlike the hierarchy system which stores
	// parent-child data on participating rows, relations are stored as separate rows containing
	// Subject (from) and Object (to) handles. This allows:
	// - Zero archetype changes on participating entity rows
	// - Unlimited participation in different relation types
	// - Per-relation metadata (timestamps, weights, etc.)
	// - O(1) hierarchical queries via interval encoding

	/**
	 * @section Relation Type Management
	 */

	/**
	 * Registers a new relation type with the given configuration.
	 * Returns InvalidRelationTypeHandle if registration fails (e.g., name already exists).
	 */
	virtual RelationTypeHandle RegisterRelationType(const FRelationRegistrationParams& Params) = 0;

	/** Finds a previously registered relation type by name. Returns InvalidRelationTypeHandle if not found. */
	virtual RelationTypeHandle FindRelationType(const FName& Name) const = 0;

	/** Checks if a relation type handle is valid. */
	virtual bool IsValidRelationType(RelationTypeHandle Type) const = 0;

	/** Gets the traits for a registered relation type. Returns nullptr if handle is invalid. */
	virtual const FTedsRelationTraits* GetRelationTypeTraits(RelationTypeHandle Type) const = 0;

	/** Lists all registered relation types by calling the callback for each. */
	virtual void ListRelationTypes(TFunctionRef<void(RelationTypeHandle, const FName&)> Callback) const = 0;

	/**
	 * Returns the tag column type stamped on Subject rows when a relation of this type changes.
	 * Only non-null when the relation was registered with bEnableSubjectChangedColumn=true.
	 * Returns nullptr for an invalid handle or when the column was not enabled.
	 */
	virtual const UScriptStruct* GetRelationSubjectChangedColumn(RelationTypeHandle Type) const = 0;

	/**
	 * Returns the tag column type stamped on Object rows when a relation of this type changes.
	 * Only non-null when the relation was registered with bEnableObjectChangedColumn=true.
	 * Returns nullptr for an invalid handle or when the column was not enabled.
	 */
	virtual const UScriptStruct* GetRelationObjectChangedColumn(RelationTypeHandle Type) const = 0;

	/**
	 * @section Relation Instance Management
	 */

	/**
	 * Creates a relation between two rows.
	 * Returns the handle of the newly created relation row, or InvalidRowHandle if creation fails.
	 *
	 * For hierarchical relations, this also:
	 * - Computes and stores depth, root, and interval values
	 * - Updates indexes for efficient queries
	 */
	virtual RowHandle CreateRelation(RelationTypeHandle Type, RowHandle Subject, RowHandle Object) = 0;

	/**
	 * Creates multiple relations at once (more efficient than individual calls).
	 * Subjects and Objects arrays must have the same length, or Objects can have length 1
	 * (in which case all subjects relate to the same object).
	 *
	 * @param OutRelationRows Optional output array to receive the created relation row handles.
	 */
	virtual void BatchCreateRelations(
		RelationTypeHandle Type,
		TConstArrayView<RowHandle> Subjects,
		TConstArrayView<RowHandle> Objects,
		TArray<RowHandle>* OutRelationRows = nullptr) = 0;

	/**
	 * Destroys a specific relation between two rows.
	 * Returns true if a relation was found and destroyed.
	 */
	virtual bool DestroyRelation(RelationTypeHandle Type, RowHandle Subject, RowHandle Object) = 0;

	/**
	 * Checks if a specific relation exists between two rows.
	 */
	virtual bool HasRelation(RelationTypeHandle Type, RowHandle Subject, RowHandle Object) const = 0;

	/** Returns true if Subject has at least one Object in this relation type. */
	virtual bool HasRelationObject(RelationTypeHandle Type, RowHandle Subject) const = 0;

	/** Returns true if Object has at least one Subject in this relation type. */
	virtual bool HasRelationSubject(RelationTypeHandle Type, RowHandle Object) const = 0;

	/**
	 * @section Relation Queries
	 */

	/**
	 * Gets all Objects that the given Subject has relations to.
	 * For a "ChildOf" relation, this returns the parent(s) of the given row.
	 */
	virtual void GetRelationObjects(
		RelationTypeHandle Type,
		RowHandle Subject,
		TArray<RowHandle>& OutObjects) const = 0;

	/**
	 * Gets the single Object for a given Subject. Use this for exclusive-object relations
	 * (bExclusiveObject=true), where a subject has at most one object.
	 * Returns InvalidRowHandle if the subject has no object in this relation.
	 */
	virtual RowHandle GetRelationObject(RelationTypeHandle Type, RowHandle Subject) const = 0;

	/**
	 * Gets all Subjects that have relations to the given Object.
	 * For a "ChildOf" relation, this returns the children of the given row.
	 */
	virtual void GetRelationSubjects(
		RelationTypeHandle Type,
		RowHandle Object,
		TArray<RowHandle>& OutSubjects) const = 0;

	/**
	 * Gets the single Subject for a given Object. Use this for exclusive-subject relations
	 * (bExclusiveSubject=true), where an object has at most one subject.
	 * Returns InvalidRowHandle if the object has no subject in this relation.
	 */
	virtual RowHandle GetRelationSubject(RelationTypeHandle Type, RowHandle Object) const = 0;

	/**
	 * @section Hierarchical Relation Queries (requires HierarchyMode != EHierarchyMode::Disabled)
	 */

	/**
	 * Checks if Descendant is anywhere below Ancestor in the hierarchy.
	 * @param bIncludeSelf If true, returns true when Descendant == Ancestor.
	 */
	virtual bool IsDescendantOf(
		RelationTypeHandle Type,
		RowHandle Descendant,
		RowHandle Ancestor,
		bool bIncludeSelf = false) const = 0;

	/**
	 * Checks if Ancestor is anywhere above Descendant in the hierarchy.
	 * Equivalent to IsDescendantOf with arguments swapped.
	 * @param bIncludeSelf If true, returns true when Ancestor == Descendant.
	 */
	inline bool IsAncestorOf(
		RelationTypeHandle Type,
		RowHandle Ancestor,
		RowHandle Descendant,
		bool bIncludeSelf = false) const
	{
		return IsDescendantOf(Type, Descendant, Ancestor, bIncludeSelf);
	}

	/**
	 * Gets the root of the hierarchy containing this row.
	 * Returns the row itself if it has no parent (is a root).
	 */
	virtual RowHandle GetHierarchyRoot(RelationTypeHandle Type, RowHandle Row) const = 0;

	/**
	 * Gets the depth of a row in the hierarchy.
	 * Roots return 0, their direct children return 1, etc.
	 */
	virtual int32 GetHierarchyDepth(RelationTypeHandle Type, RowHandle Row) const = 0;

	/** Gets all descendants of a row in the hierarchy. */
	virtual void GetDescendants(
		RelationTypeHandle Type,
		RowHandle Ancestor,
		TArray<RowHandle>& OutDescendants) const = 0;

	/**
	 * Gets all ancestors of a row in the hierarchy (parent, grandparent, etc.).
	 */
	virtual void GetAncestors(
		RelationTypeHandle Type,
		RowHandle Descendant,
		TArray<RowHandle>& OutAncestors) const = 0;

	/** Callback for TraverseDescendants. Receives the current row, its object, and depth relative to StartRow. */
	using FRelationTraversalCallback = TFunctionRef<void(RowHandle Current, RowHandle Object, int32 Depth)>;

	/** Traverses descendants of StartRow depth-first, calling Callback for each. */
	virtual void TraverseDescendants(
		RelationTypeHandle Type,
		RowHandle StartRow,
		FRelationTraversalCallback Callback,
		ETraversalOrder Order = ETraversalOrder::PreOrder,
		int32 MaxDepth = TNumericLimits<int32>::Max()) const = 0;

	/**
	 * @section Callback-Based Relation Enumeration
	 * Prefer these over the TArray variants when you only need iteration.
	 */

	/** Calls Callback for each Object related to Subject. */
	virtual void ForEachRelationObject(RelationTypeHandle Type, RowHandle Subject,
		TFunctionRef<void(RowHandle Object)> Callback) const = 0;

	/** Calls Callback for each Subject related to Object. */
	virtual void ForEachRelationSubject(RelationTypeHandle Type, RowHandle Object,
		TFunctionRef<void(RowHandle Subject)> Callback) const = 0;

	/**
	 * Walks ancestors (parent, grandparent, ...) calling Callback for each.
	 * Return false from Callback to stop early.
	 */
	virtual void TraverseAncestors(RelationTypeHandle Type, RowHandle Descendant,
		TFunctionRef<bool(RowHandle Ancestor, int32 Depth)> Callback) const = 0;

	/** Counts descendants by walking the subtree. Not cached. */
	virtual int32 ComputeDescendantCount(RelationTypeHandle Type, RowHandle Ancestor) const = 0;

	/** Returns true if Object has at least one Subject in this relation type. */
	virtual bool HasRelationSubjects(RelationTypeHandle Type, RowHandle Object) const = 0;

	/**
	 * @section Query
	 * @description
	 * Queries can be constructed using the Query Builder. Note that the Query Builder allows for the creation of queries that
	 * are more complex than the back-end may support. The back-end is allowed to simplify the query, in which case the query
	 * can be used directly in the processor to do additional filtering. This will however impact performance and it's 
	 * therefore recommended to try to simplify the query first before relying on extended query filtering in a processor.
	 */

	/** 
	 * Registers a query with the data storage. The description is processed into an internal format and may be changed. If no valid
	 * could be created an invalid query handle will be returned. It's recommended to use the Query Builder for a more convenient
	 * and safer construction of a query.
	 */
	virtual QueryHandle RegisterQuery(FQueryDescription&& Query) = 0;
	/** Removes a previous registered. If the query handle is invalid or the query has already been deleted nothing will happen. */
	virtual void UnregisterQuery(QueryHandle Query) = 0;
	/** Returns the description of a previously registered query. If the query no longer exists an empty description will be returned. */
	virtual const FQueryDescription& GetQueryDescription(QueryHandle Query) const = 0;
	/**
	 * Calculates a hash based on the topology revision id of the tables associated with the provided query. If this value changes
	 * it indicates that one or more tables has new rows added, had rows removed and/or rows have been reordered.
	 */
	virtual uint64 CalculateQueryTablesTopologyHash(QueryHandle Query) const = 0;
	/**
	 * Tick groups for queries can be given any name and the Data Storage will figure out the order of execution based on found
	 * dependencies. However keeping processors within the same query group can help promote better performance through parallelization.
	 * Therefore a collection of common tick group names is provided to help create consistent tick group names.
	 */
	virtual FName GetQueryTickGroupName(EQueryTickGroups Group) const = 0;
	/** Directly runs a query. If the query handle is invalid or has been deleted nothing will happen. */
	virtual FQueryResult RunQuery(QueryHandle Query) = 0;
	/**
	 * Directly runs a query. The callback will be called for batches of matching rows. During a single call to RunQuery the callback
	 * may be called multiple times. If the query handle is invalid or has been deleted nothing happens and the callback won't be called.
	 */
	virtual FQueryResult RunQuery(QueryHandle Query, DirectQueryCallbackRef Callback) = 0;
	/**
	 * Directly runs a query. The callback will be called for batches of matching rows. During a single call to RunQuery the callback
	 * may be called multiple times. If the query handle is invalid or has been deleted nothing happens and the callback won't be called.
	 */
	virtual FQueryResult RunQuery(QueryHandle Query, EDirectQueryExecutionFlags Flags, DirectQueryCallbackRef Callback) = 0;
	/**
	 * Directly runs a query. The callback will be called for batches of matching rows. During a single call to RunQuery the callback
	 * may be called multiple times. If the query handle is invalid or has been deleted nothing happens and the callback won't be called.
	 */
	virtual FQueryResult RunQuery(QueryHandle Query, ERunQueryFlags Flags, const Queries::TQueryFunction<void>& Callback) = 0;
	/**
	 * Directly runs a query. The callback will be called for batches of matching rows. During a single call to RunQuery the callback
	 * may be called multiple times. If the query handle is invalid or has been deleted nothing happens and the callback won't be called.
	 */
	virtual FQueryResult RunQuery(QueryHandle Query, ERunQueryFlags Flags, const Queries::TConstQueryFunction<void>& Callback) const = 0;
	/**
	 * Directly runs a query. The callback will be called for batches of matching rows. During a single call to RunQuery the callback
	 * may be called multiple times. If the query handle is invalid or has been deleted nothing happens and the callback won't be called.
	 */
	template<FunctionType Function>
	FQueryResult RunQuery(QueryHandle Query, ERunQueryFlags Flags, Function&& Callback);
	/**
	 * Directly runs a query. The callback will be called for batches of matching rows. During a single call to RunQuery the callback
	 * may be called multiple times. If the query handle is invalid or has been deleted nothing happens and the callback won't be called.
	 */
	template<typename ResultType, FunctionType Function>
	FQueryResult RunQuery(QueryHandle Query, ERunQueryFlags Flags, Queries::TResult<ResultType>& Result, Function&& Callback);
	/**
	 * Directly runs a query. The callback will be called for batches of matching rows. During a single call to RunQuery the callback
	 * may be called multiple times. If the query handle is invalid or has been deleted nothing happens and the callback won't be called.
	 */
	template<FunctionType Function>
	FQueryResult RunQuery(QueryHandle Query, ERunQueryFlags Flags, Function&& Callback) const;
	/**
	 * Directly runs a query. The callback will be called for batches of matching rows. During a single call to RunQuery the callback
	 * may be called multiple times. If the query handle is invalid or has been deleted nothing happens and the callback won't be called.
	 */
	template<typename ResultType, FunctionType Function>
	FQueryResult RunQuery(QueryHandle Query, ERunQueryFlags Flags, Queries::TResult<ResultType>& Result, Function&& Callback) const;
	/**
	 * Triggers all queries registered under the activation name to run for one update cycle. The activatable queries will be activated at
	 * start of the cycle and disabled at the end of the cycle and act like regular queries for that cycle. This includes not running
	 * if there are no columns to match against.
	 */
	virtual void ActivateQueries(FName ActivationName) = 0;

	/**
	 * @section Mapping
	 * @description
	 * In order for rows to reference each other it's often needed to find a row based on the content of one of its columns. This can be
	 * done by linearly searching through columns, though this comes at a performance cost. As an alternative the data storage allows
	 * one or more key to be created for a row for fast retrieval.
	 */

	/**
	 * Retrieves the row for a mapped object. Returns an invalid row handle if the no row with the provided key was found in the provided 
	 * domain. 
	 */
	virtual RowHandle LookupMappedRow(const FName& Domain, const FMapKeyView& Key) const = 0;
	/**
	 * Registers a row under a key in the provided domain. The same row can be registered multiple, but an key can only be associated with
	 * a single row.
	 */
	virtual void MapRow(const FName& Domain, FMapKey Key, RowHandle Row) = 0;
	/**
	 * Register multiple rows under their key in the provided domain. The same row can be registered multiple times, but the key can only
	 * be associated with a single row in the domain. During processing the keys will be moved out into the final location.
	 */
	virtual void BatchMapRows(const FName& Domain, TArrayView<TPair<FMapKey, RowHandle>> MapRowPairs) = 0;
	/** Updates the key of a row in the provided domain to a new value. Effectively this is the same as removing a key and adding a new one. */
	virtual void RemapRow(const FName& Domain, const FMapKeyView& OriginalKey, FMapKey NewKey) = 0;
	/** Removes a previously registered key in the provided domain from the mapping table or does nothing if the key no longer exists. */
	virtual void RemoveRowMapping(const FName& Domain, const FMapKeyView& Key) = 0;
	
	/**
	 * @section Tick
	 * Includes callbacks to respond to various steps during TEDS' tick.
	 */
	
	/**
	 * Called periodically when the storage is available. This provides an opportunity to do any repeated processing
	 * for the data storage.
	 */
	virtual FTypedElementOnDataStorageUpdate& OnUpdate() = 0;
	/**
	 * Called periodically when the storage is available. This provides an opportunity clean up after processing and
	 * to get ready for the next batch up updates.
	 */
	virtual FTypedElementOnDataStorageUpdate& OnUpdateCompleted() = 0;

	using FOnCooperativeUpdate = TFunction<void(FTimespan TimeAllowance)>;
	/** The higher the priority, the more frequently tasks get called. See RegisterCooperativeUpdate. */
	enum class ECooperativeTaskPriority : uint8
	{
		High,
		Medium,
		Low
	};
	/**
	 * Each tick TEDS will use the remaining time in a frame to execute task in the cooperative queue. A time slicer uses a 
	 * cooperative threading model so once a task has been started it will not be interrupted and it's up to the registered 
	 * function to try to stay within the provided time window.
	 * Tasks with a higher priority will on average be run more frequently, but other priority tasks will be given a chance to
	 * do work so guarantees are given that lower priority aren't starved out by higher priority tasks.
	 */
	virtual void RegisterCooperativeUpdate(const FName& TaskName, ECooperativeTaskPriority Priority, FOnCooperativeUpdate Callback) = 0;
	/** Removes a previously registered time sliced callback. */
	virtual void UnregisterCooperativeUpdate(const FName& TaskName) = 0;

	/**
	 * @section Miscellaneous
	 */
	
	 /**
	 * Whether or not the data storage is available. The data storage is available most of the time, but can be
	 * unavailable for a brief time between being destroyed and a new one created.
	 */
	virtual bool IsAvailable() const = 0;

	/** Returns a pointer to the registered external system if found, otherwise null. */
	virtual void* GetExternalSystemAddress(UClass* Target) = 0;
	/** Returns a pointer to the registered external system if found, otherwise null. */
	template<typename SystemType>
	SystemType* GetExternalSystem();

	/** Check if a custom extension is supported. This can be used to check for in-development features, custom extensions, etc. */
	virtual bool SupportsExtension(FName Extension) const = 0;
	/** Provides a list of all extensions that are enabled. */
	virtual void ListExtensions(TFunctionRef<void(FName)> Callback) const = 0;


	/**
	 * @section Scope
	 *
	 * @description
	 * Context rows form a hierarchy used to propagate contextual data to the tree of components that make up the editor.
	 * Downstream consumers read scope data by walking up the hierarchy until a matching column is found.
	 * It is expected that users interface with these rows via the functions provided in this section.
	 */

	/** Creates a new scope row with a default label. */
	virtual RowHandle AddScopeRow() = 0;
	/** Creates a new scope row with the given label. */
	virtual RowHandle AddScopeRow(FStringView Label) = 0;
	/** Creates a new scope row with the given label, parented to Parent. */
	virtual RowHandle AddScopeRow(FStringView Label, RowHandle Parent) = 0;
	/** Removes a scope row. */
	virtual void RemoveScopeRow(RowHandle Row) = 0;
	/** Sets the parent of Child to Parent in the scope hierarchy. */
	virtual void SetParentScope(RowHandle Child, RowHandle Parent) = 0;
	/** Returns the parent scope row for the given row in the scope hierarchy. */
	virtual RowHandle GetParentScope(RowHandle Row) = 0;
	/**
	 * Walks up the scope hierarchy from Row, returning the first non-null column data
	 * pointer for ColumnType. Returns nullptr if not found.
	 */
	virtual const void* GetScopeDataRaw(RowHandle Row, const UScriptStruct* ColumnType) = 0;
	/**
	 * Walks up the scope hierarchy from Row, returning the version of the first row that
	 * holds ColumnType data. The version is a per-row counter incremented by any write
	 * to any column on that row — not a per-column version. Returns an invalid version
	 * if no row in the hierarchy holds ColumnType.
	 */
	virtual Scope::FScopeDataVersion GetScopeDataVersion(RowHandle Row, const UScriptStruct* ColumnType) = 0;
	/**
	 * Sets (creates if absent) scope data on a specific row.
	 * The row's version counter is bumped (shared across all columns on this row).
	 * Returns the new version.
	 */
	virtual Scope::FScopeDataVersion SetScopeData(RowHandle Row, const UScriptStruct* ColumnType,
		ColumnCreationCallbackRef Initializer, ColumnCopyOrMoveCallback Relocator) = 0;
	/**
	 * Removes scope data from a specific row. The row's version counter is bumped.
	 * Returns true if the column was present and removed.
	 */
	virtual bool RemoveScopeData(RowHandle Row, const UScriptStruct* ColumnType) = 0;
	/**
	 * Returns all columns visible from a scope row by walking up the hierarchy.
	 * Closer rows shadow parents.
	 */
	virtual TArray<const UScriptStruct*> GetAllVisibleScopeColumns(RowHandle Row) = 0;
	/** Returns the root scope row. InvalidRowHandle if not yet initialized. */
	virtual RowHandle GetRootScope() const = 0;
	/** Returns the hierarchy handle for the editor scope hierarchy. */
	virtual FHierarchyHandle GetScopeHierarchy() = 0;

	/** Typed hierarchy-walking lookup from a specific row. */
	template<TColumnType T>
	const T* GetScopeData(RowHandle Row);
	/** Typed hierarchy-walking version query from a specific row. */
	template<TColumnType T>
	Scope::FScopeDataVersion GetScopeDataVersion(RowHandle Row);
	/** Template convenience: set scope data on a specific row. */
	template<TColumnType T>
	Scope::FScopeDataVersion SetScopeData(RowHandle Row, T&& Data);
	/** Template convenience: remove scope data from a specific row. */
	template<TColumnType T>
	bool RemoveScopeData(RowHandle Row);

	/**
	 * @section Deprecated
	 */
	UE_DEPRECATED(5.6, "Use 'LookUpMappedRow' instead and use the new FMapKey(View) instead of the explicit 'GenerateIndexHash'-functions.")
	inline RowHandle FindIndexedRow(IndexHash Index) const;
	UE_DEPRECATED(5.6, "Use 'MapRow' instead and use the new FMapKey(View) instead of the explicit 'GenerateIndexHash'-functions.")
	inline void IndexRow(IndexHash Index, RowHandle Row);
	UE_DEPRECATED(5.6, "Use 'BatchMapRows' instead and use the new FMapKey(View) instead of the explicit 'GenerateIndexHash'-functions.")
	inline void BatchIndexRows(TConstArrayView<TPair<IndexHash, RowHandle>> IndexRowPairs);
	UE_DEPRECATED(5.6, "Use 'RemapRow' instead and use the new FMapKey(View) instead of the explicit 'GenerateIndexHash'-functions.")
	inline void Reindex(IndexHash OriginalIndex, IndexHash NewIndex);
	UE_DEPRECATED(5.6, "Use 'RemovedRowMapping' instead and use the new FMapKey(View) instead of the explicit 'GenerateIndexHash'-functions.")
	inline void RemoveIndex(IndexHash Index);
	UE_DEPRECATED(5.6, "Use 'RemapRow' instead and use the new FMapKey(View) instead of the explicit 'GenerateIndexHash'-functions. The Row argument is no longer used.")
	inline void ReindexRow(IndexHash OriginalIndex, IndexHash NewIndex, RowHandle Row);

	UE_DEPRECATED(5.7, "Use the version of 'LookupMappedRow' that uses a domain.")
	inline RowHandle LookupMappedRow(const FMapKeyView& Key) const;
	UE_DEPRECATED(5.7, "Use the version of 'MapRow' that uses a domain.")
	inline void MapRow(FMapKey Key, RowHandle Row);
	UE_DEPRECATED(5.7, "Use the version of 'BatchMapRows' that uses a domain.")
	inline void BatchMapRows(TArrayView<TPair<FMapKey, RowHandle>> MapRowPairs);
	UE_DEPRECATED(5.7, "Use the version of 'RemapRow' that uses a domain.")
	inline void RemapRow(const FMapKeyView& OriginalKey, FMapKey NewKey);
	UE_DEPRECATED(5.7, "Use the version of 'RemoveRowMapping' that uses a domain.")
	inline void RemoveRowMapping(const FMapKeyView& Key);
};

ENUM_CLASS_FLAGS(ICoreProvider::EFilterOptions)

// Implementations

template <typename FactoryT>
const FactoryT* ICoreProvider::FindFactory() const
{
	return static_cast<const FactoryT*>(FindFactory(FactoryT::StaticClass()));
}

template <typename FactoryT>
FactoryT* ICoreProvider::FindFactory()
{
	return static_cast<FactoryT*>(FindFactory(FactoryT::StaticClass()));
}

template<typename FilterFunction>
void ICoreProvider::FilterRowsBy(FRowHandleArray& Result, FRowHandleArrayView Input, EFilterOptions Options, FilterFunction&& Filter) const
{
	FilterRowsBy(Result, Input, Options, Queries::BuildConstQueryFunction<bool>(Forward<FilterFunction>(Filter)));
}

template<TColumnType Column>
void ICoreProvider::AddColumn(RowHandle Row)
{
	AddColumn(Row, Column::StaticStruct());
}

template<TColumnType Column>
void ICoreProvider::RemoveColumn(RowHandle Row)
{
	RemoveColumn(Row, Column::StaticStruct());
}

template<TColumnType... Columns>
void ICoreProvider::AddColumns(RowHandle Row)
{
	AddColumns(Row, { Columns::StaticStruct()...});
}

template <>
inline void ICoreProvider::AddColumn<FValueTag>(RowHandle Row, const FName& Tag, const FName& Value)
{
	AddColumn(Row, FValueTag(Tag), Value);
}

template <>
inline void ICoreProvider::RemoveColumn<FValueTag>(RowHandle Row, const FName& Tag)
{
	using namespace UE::Editor::DataStorage;
	RemoveColumn(Row, FValueTag(Tag));
}

template<TDynamicColumnTemplate DynamicColumnTemplate>
void ICoreProvider::RemoveColumn(RowHandle Row, const FName& Identifier)
{
	RemoveColumn(Row, FindDynamicColumnType<DynamicColumnTemplate>(Identifier));
}

template<TEnumType EnumT>
void ICoreProvider::AddColumn(RowHandle Row, EnumT Value)
{
	const UEnum* Enum = StaticEnum<EnumT>();
	const FName ValueAsFName = *Enum->GetNameStringByValue(static_cast<int64>(Value));
	if (ValueAsFName != NAME_None)
	{
		AddColumn(Row, FValueTag(Enum->GetFName()), ValueAsFName);
	}
}

template<TColumnType... Columns>
void ICoreProvider::RemoveAllRowsWith()
{
	RemoveAllRowsWithColumns({ Columns::StaticStruct()... });
}

template<auto Value, TEnumType EnumT>
void ICoreProvider::AddColumn(RowHandle Row)
{
	AddColumn<EnumT>(Row, Value);
}

template <TDynamicColumnTemplate DynamicColumnTemplate>
void ICoreProvider::AddColumn(RowHandle Row, const FName& Identifier)
{
	const FDynamicColumnDescription Description
	{
		.TemplateType = DynamicColumnTemplate::StaticStruct(),
		.Identifier = Identifier
	};
	const UScriptStruct* StructInfo = GenerateDynamicColumn(Description);
	AddColumn(Row, StructInfo);
}

template <TDynamicColumnTemplate DynamicColumnTemplate>
void ICoreProvider::AddColumn(RowHandle Row, const FName& Identifier, DynamicColumnTemplate&& TemplateInstance)
{
	const FDynamicColumnDescription Description
	{
		.TemplateType = DynamicColumnTemplate::StaticStruct(),
		.Identifier = Identifier
	};
	const UScriptStruct* StructInfo = GenerateDynamicColumn(Description);
	AddColumnData(Row, StructInfo,
		[&TemplateInstance](void* ColumnData, const UScriptStruct&)
		{
			if constexpr (std::is_move_constructible_v<DynamicColumnTemplate>)
			{
				new(ColumnData) DynamicColumnTemplate(MoveTemp(TemplateInstance));
			}
			else
			{
				new(ColumnData) DynamicColumnTemplate(TemplateInstance);
			}
		},
		[](const UScriptStruct&, void* Destination, void* Source)
		{
			if constexpr (std::is_move_assignable_v<DynamicColumnTemplate>)
			{
				*static_cast<DynamicColumnTemplate*>(Destination) = MoveTemp(*static_cast<DynamicColumnTemplate*>(Source));
			}
			else
			{
				*static_cast<DynamicColumnTemplate*>(Destination) = *static_cast<DynamicColumnTemplate*>(Source);
			}
		});
	
}

template<TEnumType EnumT>
void ICoreProvider::RemoveColumn(RowHandle Row)
{
	const UEnum* Enum = StaticEnum<EnumT>();
	RemoveColumn(Row, FValueTag(Enum->GetFName()));
}

template<TColumnType... Columns>
void ICoreProvider::RemoveColumns(RowHandle Row)
{
	RemoveColumns(Row, { Columns::StaticStruct()...});
}

template<TDataColumnType ColumnType>
void ICoreProvider::AddColumn(RowHandle Row, ColumnType&& Column)
{
	AddColumnData(Row, ColumnType::StaticStruct(),
		[&Column](void* ColumnData, const UScriptStruct&)
		{
			if constexpr (std::is_move_constructible_v<ColumnType>)
			{
				new(ColumnData) ColumnType(MoveTemp(Column));
			}
			else
			{
				new(ColumnData) ColumnType(Column);
			}
		},
		[](const UScriptStruct&, void* Destination, void* Source)
		{
			if constexpr (std::is_move_assignable_v<ColumnType>)
			{
				*reinterpret_cast<ColumnType*>(Destination) = MoveTemp(*reinterpret_cast<ColumnType*>(Source));
			}
			else
			{
				*reinterpret_cast<ColumnType*>(Destination) = *reinterpret_cast<ColumnType*>(Source);
			}
		});
}

template<TDataColumnType ColumnType>
ColumnType* ICoreProvider::GetColumn(RowHandle Row)
{
	return reinterpret_cast<ColumnType*>(GetColumnData(Row, ColumnType::StaticStruct()));
}

template<TDataColumnType ColumnType>
const ColumnType* ICoreProvider::GetColumn(RowHandle Row) const
{
	return reinterpret_cast<const ColumnType*>(GetColumnData(Row, ColumnType::StaticStruct()));
}

template <TDynamicColumnTemplate DynamicColumnTemplate>
DynamicColumnTemplate* ICoreProvider::GetColumn(RowHandle Row, const FName& Identifier)
{
	const FDynamicColumnDescription Description
	{
		.TemplateType = DynamicColumnTemplate::StaticStruct(),
		.Identifier = Identifier
	};
	const UScriptStruct* StructInfo = GenerateDynamicColumn(Description);
	if (StructInfo)
	{
		return static_cast<DynamicColumnTemplate*>(GetColumnData(Row, StructInfo));
	}
	return nullptr;
}

template <TDynamicColumnTemplate DynamicColumnTemplate>
const DynamicColumnTemplate* ICoreProvider::GetColumn(RowHandle Row, const FName& Identifier) const
{
	if (const UScriptStruct* StructInfo = FindDynamicColumnType<DynamicColumnTemplate>(Identifier))
	{
		return static_cast<const DynamicColumnTemplate*>(GetColumnData(Row, StructInfo));
	}
	return nullptr;
}

template<TColumnType... ColumnType>
bool ICoreProvider::HasColumns(RowHandle Row) const
{
	return HasColumns(Row, TConstArrayView<const UScriptStruct*>({ ColumnType::StaticStruct()... }));
}

template<Queries::TColumnDescType... ColumnTypes>
bool ICoreProvider::HasColumns(RowHandle Row, const ColumnTypes&... Columns) const
{
	TArray<const UScriptStruct*, TInlineAllocator<sizeof...(ColumnTypes)>> TypeInfo;
	TypeInfo.Reserve(sizeof...(ColumnTypes));

	auto GetType = [this, &TypeInfo]<Queries::TColumnDescType ColumnType>(const ColumnType& Column) -> bool
		{
			const UScriptStruct* BaseColumnType = Column.TypeInfo.Get();
			const UScriptStruct* Result = Column.Identifier.IsNone()
				? BaseColumnType
				: (BaseColumnType ? FindDynamicColumnType(*BaseColumnType, Column.Identifier) : nullptr);
			if (Result)
			{
				TypeInfo.Add(Result);
				return true;
			}
			else
			{
				return false;
			}
		};

	if (!(GetType(Columns) && ...))
	{
		return false;
	}
	return HasColumns(Row, TConstArrayView<const UScriptStruct*>(TypeInfo));
}

template<typename SystemType>
SystemType* ICoreProvider::GetExternalSystem()
{
	return reinterpret_cast<SystemType*>(GetExternalSystemAddress(SystemType::StaticClass()));
}

void ICoreProvider::ReindexRow(
	IndexHash OriginalIndex, IndexHash NewIndex, RowHandle Row)
{
	RemapRow(NAME_None, FMapKeyView(OriginalIndex), FMapKey(NewIndex));
}

RowHandle ICoreProvider::FindIndexedRow(IndexHash Index) const
{
	return LookupMappedRow(NAME_None, FMapKeyView(Index));
}

void ICoreProvider::IndexRow(IndexHash Index, RowHandle Row)
{
	MapRow(NAME_None, FMapKey(Index), Row);
}

void ICoreProvider::BatchIndexRows(TConstArrayView<TPair<IndexHash, RowHandle>> IndexRowPairs)
{
	for (const TPair<IndexHash, RowHandle>& Pair : IndexRowPairs)
	{
		MapRow(NAME_None, FMapKey(Pair.Key), Pair.Value);
	}
}

void ICoreProvider::Reindex(IndexHash OriginalIndex, IndexHash NewIndex)
{
	RemapRow(NAME_None, FMapKeyView(OriginalIndex), FMapKey(NewIndex));
}

void ICoreProvider::RemoveIndex(IndexHash Index)
{
	RemoveRowMapping(NAME_None, FMapKeyView(Index));
}

RowHandle ICoreProvider::LookupMappedRow(const FMapKeyView& Key) const
{
	return LookupMappedRow(NAME_None, Key);
}

void ICoreProvider::MapRow(FMapKey Key, RowHandle Row)
{
	MapRow(NAME_None, Key, Row);
}

void ICoreProvider::BatchMapRows(TArrayView<TPair<FMapKey, RowHandle>> MapRowPairs)
{
	BatchMapRows(NAME_None, MapRowPairs);
}

void ICoreProvider::RemapRow(const FMapKeyView& OriginalKey, FMapKey NewKey)
{
	RemapRow(NAME_None, OriginalKey, NewKey);
}

void ICoreProvider::RemoveRowMapping(const FMapKeyView& Key)
{
	RemoveRowMapping(NAME_None, Key);
}

template<TDynamicColumnTemplate DynamicColumnTemplateType>
void ICoreProvider::ForEachDynamicColumn(TFunctionRef<void(const UScriptStruct& Type)> Callback) const
{
	ForEachDynamicColumn(DynamicColumnTemplateType::StaticStruct(), Callback);
}

template<FunctionType Function>
FQueryResult ICoreProvider::RunQuery(QueryHandle Query, ERunQueryFlags Flags, Function&& Callback)
{
	return RunQuery(Query, Flags, Queries::BuildQueryFunction<void>(Forward<Function>(Callback)));
}

template<typename ResultType, FunctionType Function>
FQueryResult ICoreProvider::RunQuery(QueryHandle Query, ERunQueryFlags Flags, Queries::TResult<ResultType>& Result, Function&& Callback)
{
	return RunQuery(Query, Flags, Queries::BuildQueryFunction(Result, Forward<Function>(Callback)));
}

template<FunctionType Function>
FQueryResult ICoreProvider::RunQuery(QueryHandle Query, ERunQueryFlags Flags, Function&& Callback) const
{
	return RunQuery(Query, Flags, Queries::BuildConstQueryFunction<void>(Forward<Function>(Callback)));
}

template<typename ResultType, FunctionType Function>
FQueryResult ICoreProvider::RunQuery(QueryHandle Query, ERunQueryFlags Flags, Queries::TResult<ResultType>& Result, Function&& Callback) const
{
	return RunQuery(Query, Flags, Queries::BuildConstQueryFunction(Result, Forward<Function>(Callback)));
}

// Scope template implementations

template<TColumnType T>
const T* ICoreProvider::GetScopeData(RowHandle Row)
{
	return static_cast<const T*>(GetScopeDataRaw(Row, T::StaticStruct()));
}

template<TColumnType T>
Scope::FScopeDataVersion ICoreProvider::GetScopeDataVersion(RowHandle Row)
{
	return GetScopeDataVersion(Row, T::StaticStruct());
}

template<TColumnType T>
Scope::FScopeDataVersion ICoreProvider::SetScopeData(RowHandle Row, T&& Data)
{
	return SetScopeData(Row, T::StaticStruct(),
		[&Data](void* ColumnData, const UScriptStruct&)
		{
			if constexpr (std::is_move_constructible_v<T>)
			{
				new(ColumnData) T(MoveTemp(Data));
			}
			else
			{
				new(ColumnData) T(Data);
			}
		},
		[](const UScriptStruct&, void* Destination, void* Source)
		{
			if constexpr (std::is_move_assignable_v<T>)
			{
				*static_cast<T*>(Destination) = MoveTemp(*static_cast<T*>(Source));
			}
			else
			{
				*static_cast<T*>(Destination) = *static_cast<T*>(Source);
			}
		});
}

template<TColumnType T>
bool ICoreProvider::RemoveScopeData(RowHandle Row)
{
	return RemoveScopeData(Row, T::StaticStruct());
}

} // namespace UE::Editor::DataStorage
