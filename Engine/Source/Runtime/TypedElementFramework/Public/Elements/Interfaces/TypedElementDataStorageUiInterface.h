// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "DataStorage/Queries/Conditions.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Framework/TypedElementMetaData.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Framework/TypedElementSearcher.h"
#include "Elements/Framework/TypedElementSorter.h"
#include "Features/IModularFeature.h"
#include "Templates/UnrealTypeTraits.h"
#include "Templates/SharedPointer.h"
#include "UObject/Interface.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "TypedElementDataStorageUiInterface.generated.h"

namespace UE::Editor::DataStorage
{
	class ITedsWidget;
	class IUiProvider;
}

class SWidget;
struct FWidgetPurposeNameColumn;

/**
 * Base class for Widget Constructors in TEDS.
 * Contains common functionality needed by the different types of widget constructors that TEDS UI supports.
 */
USTRUCT()
struct FTedsWidgetConstructorBase
{
public:

	GENERATED_BODY()

	TYPEDELEMENTFRAMEWORK_API explicit FTedsWidgetConstructorBase(const UScriptStruct* InTypeInfo);
	explicit FTedsWidgetConstructorBase(EForceInit) {} //< For compatibility and shouldn't be directly used.
	virtual ~FTedsWidgetConstructorBase() = default;
	
	/** Initializes a new constructor based on the provided arguments. */
	TYPEDELEMENTFRAMEWORK_API virtual bool Initialize(const UE::Editor::DataStorage::FMetaDataView& InArguments,
		TArray<TWeakObjectPtr<const UScriptStruct>> InMatchedColumnTypes, UE::Editor::DataStorage::RowHandle FactoryRowHandle);

	/** Retrieves the type information for the constructor type. */
	TYPEDELEMENTFRAMEWORK_API virtual const UScriptStruct* GetTypeInfo() const;
	/** Retrieves the columns, if any, that were matched to this constructor when it was created. */
	TYPEDELEMENTFRAMEWORK_API virtual const TArray<TWeakObjectPtr<const UScriptStruct>>& GetMatchedColumns() const;
	
	/** Retrieves the query conditions that need to match for this widget constructor to produce a widget. */
	TYPEDELEMENTFRAMEWORK_API virtual const UE::Editor::DataStorage::Queries::FConditions* GetQueryConditions(const UE::Editor::DataStorage::ICoreProvider* Storage) const;
protected:
	
	bool Initialize_Internal(const UE::Editor::DataStorage::FMetaDataView& InArguments,
		TArray<TWeakObjectPtr<const UScriptStruct>> InMatchedColumnTypes);

protected:
	// The columns this widget constructor matched against
	TArray<TWeakObjectPtr<const UScriptStruct>> MatchedColumnTypes;

	// If the constructor matched against any dynamic columns, map from the base template for the column -> FName identifier
	TMap<TWeakObjectPtr<const UScriptStruct>, FName> MatchedDynamicTemplates;

	// The row containing the widget factory this constructor was factories from
	UE::Editor::DataStorage::RowHandle WidgetFactoryRow = UE::Editor::DataStorage::InvalidRowHandle;

	// The query conditions required by this constructor
	const UE::Editor::DataStorage::Queries::FConditions* QueryConditions = nullptr;

	// The constructor's typeinfo
	const UScriptStruct* TypeInfo = nullptr;
};

template<>
struct TStructOpsTypeTraits<FTedsWidgetConstructorBase> : public TStructOpsTypeTraitsBase2<FTedsWidgetConstructorBase>
{
	enum
	{
		WithNoInitConstructor = true,
		WithPureVirtual = true,
	};
};

/**
 * Base class used to construct TEDS UI widgets with.
 * See below for the options to register a constructor with the Data Storage.
 * In most cases you want to inherit from FSimpleWidgetConstructor instead which has a simpler pipeline to create widgets
 */
USTRUCT()
struct FTypedElementWidgetConstructor : public FTedsWidgetConstructorBase
{
	GENERATED_BODY()

public:
	TYPEDELEMENTFRAMEWORK_API explicit FTypedElementWidgetConstructor(const UScriptStruct* InTypeInfo);
	explicit FTypedElementWidgetConstructor(EForceInit) : FTedsWidgetConstructorBase(ForceInit) {} //< For compatibility and shouldn't be directly used.

	virtual ~FTypedElementWidgetConstructor() override = default;
	
	/** Returns a list of additional columns the widget requires to be added to its rows. */
	TYPEDELEMENTFRAMEWORK_API virtual TConstArrayView<const UScriptStruct*> GetAdditionalColumnsList() const;
	
	/** 
	 * Returns an FText of the friendly name for the data the created widget represents.
	 * By default the associated column is used. If there are multiple columns associated with the constructor
	 * the default implementation will attempt to find the longest common starting string for all the columns.
	 * Individual widget constructors can override this function with a name specific to them.
	 * 
	 * If no row is specified, it will default to an InvalidRowHandle, assuming that we are asking for a display 
	 * name for the whole widget and not a specific instance.
	 */
	TYPEDELEMENTFRAMEWORK_API virtual FText CreateWidgetDisplayNameText(
		UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle Row = UE::Editor::DataStorage::InvalidRowHandle) const;
	
	/**
	 *	Calls Construct() to create the internal widget, and then stores it in a container before returning.
	 *	In most cases you want to call this to first create the initial TEDS widget, to ensure the internal widget is
	 *	automatically created/destroyed if the row matches/unmatches the required columns.
	 *	
	 *	Construct() can be called later to (re)create the internal widget if ever required.
	 *	@see Construct
	 */
	TYPEDELEMENTFRAMEWORK_API virtual TSharedPtr<SWidget> ConstructFinalWidget(
		UE::Editor::DataStorage::RowHandle Row, /** The row the widget will be stored in. */
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		const UE::Editor::DataStorage::FMetaDataView& Arguments);

	/**
	 * Constructs the widget according to the provided information. Information is collected by calling
	 * the below functions CreateWidget and AddColumns. It's recommended to overload those
	 * functions to build widgets according to a standard recipe and to reduce the amount of code needed.
	 * If a complexer situation is called for this function can also be directly overwritten.
	 * In most cases, you want to call ConstructFinalWidget to create the actual widget.
	 */
	TYPEDELEMENTFRAMEWORK_API virtual TSharedPtr<SWidget> Construct(
		UE::Editor::DataStorage::RowHandle Row, /** The row the widget will be stored in. */
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		const UE::Editor::DataStorage::FMetaDataView& Arguments);
	
	/**
	 * Constructs a list of classes that are used to sort the rows with the column(s) associated with this widget constructor. 
	 * If this widget constructor is used to create multiple widgets, for instance when used in a column of a grid view, these sorters can
	 * be used to order the rows in a logical order described by the sorter. A widget constructor can provide multiple constructors which will
	 * allow the same list of rows to be sorted in different orders.
	 */
	TYPEDELEMENTFRAMEWORK_API virtual TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> ConstructColumnSorters(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		const UE::Editor::DataStorage::FMetaDataView& Arguments);
		
	/**
     * Constructs a list of classes that are used to convert the column data into a searchable string on the rows with the column(s)
     * associated with this widget constructor. A widget constructor can provide multiple searching constructors which will
     * allow the same list of rows to be searched in different ways.
     */
    TYPEDELEMENTFRAMEWORK_API virtual TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSearcherInterface>> ConstructColumnSearchers(
        UE::Editor::DataStorage::ICoreProvider* DataStorage,
        UE::Editor::DataStorage::IUiProvider* DataStorageUi,
        const UE::Editor::DataStorage::FMetaDataView& Arguments);

	/**
	 * Helper function to get the row containing the widget factory this constructor was factoried from
	 */
	TYPEDELEMENTFRAMEWORK_API UE::Editor::DataStorage::RowHandle GetWidgetFactoryRow() const;

	/*
	 * Deprecated public functions
	 */

	/**
	 * Returns a friendly name for the data the created widget represents.
	 * By default the associated column is used. If there are multiple columns associated with the constructor
	 * the default implementation will attempt to find the longest common starting string for all the columns.
	 * Individual widget constructors can override this function with a name specific to them.
	 */
	UE_DEPRECATED(5.6, "CreateWidgetDisplayName is deprecated, please use CreateWidgetDisplayNameText instead, which will return FText rather than an FString.")
	TYPEDELEMENTFRAMEWORK_API virtual FString CreateWidgetDisplayName(
		UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle Row) const final;

	// To get around name hiding since the deprecated functions overload methods from the base class
	using FTedsWidgetConstructorBase::Initialize;
	using FTedsWidgetConstructorBase::GetQueryConditions;

	/** Initializes a new constructor based on the provided arguments.. */
	UE_DEPRECATED(5.6, "Use Initialize that takes in the factory row handle instead")
	TYPEDELEMENTFRAMEWORK_API virtual bool Initialize(const UE::Editor::DataStorage::FMetaDataView& InArguments,
		TArray<TWeakObjectPtr<const UScriptStruct>> InMatchedColumnTypes, const UE::Editor::DataStorage::Queries::FConditions& InQueryConditions);

	/** Retrieves the query conditions that need to match for this widget constructor to produce a widget. */
	UE_DEPRECATED(5.6, "Use GetQueryConditions that takes in ICoreProvider* instead")
	TYPEDELEMENTFRAMEWORK_API virtual const UE::Editor::DataStorage::Queries::FConditions* GetQueryConditions() const final;

protected:
	/** Create a new instance of the target widget. This is a required function. */
	TYPEDELEMENTFRAMEWORK_API virtual TSharedPtr<SWidget> CreateWidget(const UE::Editor::DataStorage::FMetaDataView& Arguments);
	/** Create a new instance of the target widget. This is a required function. */
	TYPEDELEMENTFRAMEWORK_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow, 
		const UE::Editor::DataStorage::FMetaDataView& Arguments);
	/** Set any values in columns if needed. The columns provided through GetAdditionalColumnsList() will have already been created. */
	TYPEDELEMENTFRAMEWORK_API virtual bool SetColumns(UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle Row);
	
	/** Creates a (friendly) name for the provided column type. */
	TYPEDELEMENTFRAMEWORK_API virtual FText DescribeColumnType(const UScriptStruct* ColumnType) const;

	/** 
	 * Last opportunity to configure anything in the widget or the row. This step can be needed to initialize widgets with data stored
	 * in columns.
	 */
	TYPEDELEMENTFRAMEWORK_API virtual bool FinalizeWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle Row,
		const TSharedPtr<SWidget>& Widget);

	/** Add the default misc columns we want a widget row to have. */
	TYPEDELEMENTFRAMEWORK_API void AddDefaultWidgetColumns(UE::Editor::DataStorage::RowHandle Row, UE::Editor::DataStorage::ICoreProvider* DataStorage) const;

	/**
	 * Helper function to get the actual target row with the data the widget is operating on (if applicable). Returns InvalidRowHandle if there is no
	 * target row
	 */
	TYPEDELEMENTFRAMEWORK_API UE::Editor::DataStorage::RowHandle GetTargetRow(
		UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle WidgetRow) const;

	/**
	 * Helper function to perform a function on all column types for this widget (includes matched dynamic templates) 
	 */
	TYPEDELEMENTFRAMEWORK_API void ForAllColumnTypes(TFunctionRef<void(TNotNull<const UScriptStruct*>)> Function) const;
};

/**
 * A simple widget constructor that cuts down on most of the boilerplate, in most cases you want to inherit from this to create your widget constructor
 * Only requires you to override CreateWidget() to create the actual SWidget
 */
USTRUCT()
struct FSimpleWidgetConstructor : public FTypedElementWidgetConstructor
{
	GENERATED_BODY()

	/** Call this constructor with StaticStruct() on your derived class to pass in the type information */
	TYPEDELEMENTFRAMEWORK_API explicit FSimpleWidgetConstructor(const UScriptStruct* InTypeInfo);

	FSimpleWidgetConstructor() : Super(StaticStruct()) {} //< For compatibility and shouldn't be directly used.
	
	virtual ~FSimpleWidgetConstructor() override = default;

	/*
	 * Required function to create the actual widget instance.
	 * 
	 * @param DataStorage A pointer to the TEDS data storage interface
	 * @param DataStorageUi A pointer to the TEDS data storage UI interface
	 * @param TargetRow The row for the actual data this widget is being created for (can be InvalidRowHandle if there is no target row attached to this widget)
	 * @param WidgetRow The row that contains information about the widget itself
	 * @param Arguments Any metadata arguments that were specified
	 * @return The actual widget instance
	 */
	TYPEDELEMENTFRAMEWORK_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

	/*
	 * Override this function to add any columns to the WidgetRow before CreateWidget is called
	 * 
	 * @param DataStorage A pointer to the TEDS data storage interface
	 * @param WidgetRow The row that contains information about the widget itself
	 * @return Whether any columns were added
	 */
	TYPEDELEMENTFRAMEWORK_API virtual bool SetColumns(UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle WidgetRow) override;

protected:

	/** Old CreateWidget overload that exists for backwards compatibility, you should use the overload that provides the row instead */
	TYPEDELEMENTFRAMEWORK_API virtual TSharedPtr<SWidget> CreateWidget(const UE::Editor::DataStorage::FMetaDataView& Arguments) override final;
	
	/** 
	 * Old function in the widget creation pipeline that isn't used anymore. All your logic should go in CreateWidget() itself
	 */
	TYPEDELEMENTFRAMEWORK_API virtual bool FinalizeWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle Row,
		const TSharedPtr<SWidget>& Widget) override final;
	
	/**
	 * Helper function to call SetColumns() and CreateWidget(), you should not need to override this for simple widget constructors
	 */
	TYPEDELEMENTFRAMEWORK_API virtual TSharedPtr<SWidget> Construct(
		UE::Editor::DataStorage::RowHandle WidgetRow,
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override final;
};

template<>
struct TStructOpsTypeTraits<FTypedElementWidgetConstructor> : public TStructOpsTypeTraitsBase2<FTypedElementWidgetConstructor>
{
	enum
	{
		WithNoInitConstructor = true,
		WithPureVirtual = true,
	};
};

namespace UE::Editor::DataStorage
{
	template<typename PropertyClass> 
	concept PropertyClassType = std::is_base_of_v<FProperty, PropertyClass>;
	
	class IUiProvider : public IModularFeature
	{
	public:
		inline static const FName PurposeMappingDomain = "WidgetPurpose";

		enum class EPurposeType : uint8
		{
			/** General purpose name which allows multiple factory registrations. */
			Generic,
			/**
			 * Only one factory can be registered with this purpose. If multiple factories are registered only the last factory
			 * will be stored.
			 */
			UniqueByName,
			/**
			 * Only one factory can be registered with this purpose for a specific combination of columns. If multiple factories
			 * are registered only the last factory will be stored.
			 */
			UniqueByNameAndColumn
		};

		enum class EMatchApproach : uint8
		{
			/** 
			 * Looks for the longest chain of columns matching widget factories. The matching columns are removed and the process
			 * is repeated until there are no more columns or no matches are found.
			 */
			LongestMatch,
			/** A single widget factory is reduced which matches the requested columns exactly. */
			ExactMatch,
			/** Each column is matched to widget factory. Only widget factories that use a single column are used. */
			SingleMatch
		};

		using FPurposeID = FMapKey;

		/*
		 * Struct describing the default init params for a widget purpose. These can also be added to the purpose row post init using the columns
		 * in WidgetPurposeColumns.h
		 */
		struct FPurposeInfo
		{
			/** The namespace the purpose belongs to (e.g "General", "SceneOutliner") */
			FName Namespace;

			/** The name of the purpose (e.g "RowLabel", "Cell") */
			FName Name;

			/** An optional suffix for the purpose (e.g "Large", "Small", "Default") */
			FName Frame;

			/** The purpose type */
			EPurposeType Type;

			/** A user facing description for the purpose */
			FText Description;

			/** The parent purpose. If valid, widget construction methods will go up the hierarchy if no widgets were found for a purpose */
			FPurposeID ParentPurposeID;

			/** Default constructor for FPurposeInfo, the members with a default value are optional when you only need an FPurposeInfo to look up FPurposeID */
			TYPEDELEMENTFRAMEWORK_API FPurposeInfo(const FName& InNamespace, const FName& InName, const FName& InFrame,
				EPurposeType InPurposeType = EPurposeType::UniqueByNameAndColumn, const FText& InDescription = FText::GetEmpty(),
				const FPurposeID& InParent = FPurposeID());
		
			/** Create an FPurposeInfo struct using a legacy purpose name to parse it as Namespace.Name.Frame (e.g "SceneOutliner.Cell.Large") */
			TYPEDELEMENTFRAMEWORK_API FPurposeInfo(const FName& InLegacyPurposeName, EPurposeType InPurposeType, const FText& InDescription,
				const FPurposeID& InParent = FPurposeID());

			/** Create an FPurposeInfo struct using FWidgetPurposeNameColumn */
			TYPEDELEMENTFRAMEWORK_API explicit FPurposeInfo(const FWidgetPurposeNameColumn& WidgetPurposeNameColumn);

			/** Convert this PurposeInfo to a PurposeID that can be used to look up the purpose row */
			TYPEDELEMENTFRAMEWORK_API FPurposeID GeneratePurposeID() const;

			/** Convert this PurposeInfo to a human-readable name in the legacy format ("Namespace.Name.Frame") */
			TYPEDELEMENTFRAMEWORK_API FName ToString() const;
		};

		using FWidgetConstructorPtr = TSharedPtr<FTypedElementWidgetConstructor>;
		using WidgetCreatedCallback = TFunctionRef<void(const TSharedRef<SWidget>& NewWidget, UE::Editor::DataStorage::RowHandle Row)>;
		using WidgetConstructionCallback = TFunctionRef<bool(FWidgetConstructorPtr, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)>;
		using WidgetPurposeCallback = TFunctionRef<void(FName, EPurposeType, const FText&)>;
		using PropertySorterConstructorCallback = 
			TFunction<TSharedPtr<const FColumnSorterInterface>(TWeakObjectPtr<const UScriptStruct> ColumnType, const FProperty& Property)>;
		using PropertySearcherConstructorCallback = 
			TFunction<TSharedPtr<const FColumnSearcherInterface>(TWeakObjectPtr<const UScriptStruct> ColumnType, const FProperty& Property)>;

		/**
		 * @section Widget purposes
		 * Widget purposes indicates how widgets can be used and categorizes/organizes the available widget factories. Purposes as used to 
		 * match columns to widget constructors and provide contextual information on how the widgets get setup. In practice this allows 
		 * tools to provide what purposes they support and TEDS UI will find the closest matching widget constructors that are compatible
		 * with the purpose.
		 */

		/**
		 * Register a widget purpose. Widget purposes indicates how widgets can be used and categorizes/organizes the available 
		 * widget factories. If the same purpose is registered multiple times, only the first will be recorded and later registrations
		 * will be ignored and the return value will be InvalidRowHandle.
		 * The purposes will be mapped to the TEDS Mapping Table using InPurposeInfo.GeneratePurposeID() which can be used to lookup the purpose row.
		 *
		 * @param InPurposeInfo Struct with init params for the purpose
		 */
		virtual RowHandle RegisterWidgetPurpose(const FPurposeInfo& InPurposeInfo) = 0;
	
		/**
		 * Overload for RegisterWidgetPurpose that allows you to use a custom ID to map to the TEDS Mapping table for lookup
		 * 
		 * @param PurposeID A unique ID for the purpose that can be used to look up the row handle later
		 * @param InPurposeInfo Struct with init params for the purpose
		 */
		virtual RowHandle RegisterWidgetPurpose(const FPurposeID& PurposeID, const FPurposeInfo& InPurposeInfo) = 0;

		/** Find the row handle for a purpose by looking it up using the purpose ID */
		virtual RowHandle FindPurpose(const FPurposeID& PurposeID) const = 0;

		/** Calls the provided callback for all known registered widget purposes. */
		virtual void ListWidgetPurposes(const WidgetPurposeCallback& Callback) const = 0;


		/** Get the name of the default TEDS UI widget purpose used to register default widgets for different types of data (e.g FText -> STextBlock) */
		virtual FPurposeID GetDefaultWidgetPurposeID() const = 0;

		/** Get the name of the general TEDS UI purpose used to register general purpose widgets for columns */
		virtual FPurposeID GetGeneralWidgetPurposeID() const = 0;
		
		/** Get the hierarchy handle that stores widget purposes relationships */
		virtual FHierarchyHandle GetWidgetPurposeHierarchyHandle() const = 0;
	
		/**
		 * @section Widget constructors.
		 * Widget constructors are the glue between TEDS data and Slate. They are associated with specific columns and provide ways
		 * to generate Slate widgets for those columns. They also facilitate communication back-and-forth between TEDS and Slate.
		 * Widget purposes can be used to specify what widget constructor to use when, allowing multiple widget constructors to
		 * work on the same columns. Through the meta data object individual widgets can be further specialized on an individual baseis.
		 */
	
		/** 
		 * Registers a widget factory that will be called when the purpose it's registered under is requested.
		 * This version registers a generic type. Construction using these are typically cheaper as they can avoid
		 * copying the Constructor and take up less memory. The downside is that they can't store additional configuration
		 * options. If the purpose has not been registered the factory will not be recorded and a warning will be printed.
		 * If registration is successful true will be returned otherwise false.
		 */
		virtual bool RegisterWidgetFactory(RowHandle PurposeRow, const UScriptStruct* Constructor) = 0;

		/**
		 * Registers a widget factory that will be called when the purpose it's registered under is requested.
		 * This version registers a generic type. Construction using these are typically cheaper as they can avoid
		 * copying the Constructor and take up less memory. The downside is that they can't store additional configuration
		 * options. If the purpose has not been registered the factory will not be recorded and a warning will be printed.
		 * If registration is successful true will be returned otherwise false.
		 */
		template<typename ConstructorType>
		bool RegisterWidgetFactory(RowHandle PurposeRow);

		/**
		 * Registers a widget factory that will be called when the purpose it's registered under is requested.
		 * This version registers a generic type. Construction using these are typically cheaper as they can avoid
		 * copying the Constructor and take up less memory. The downside is that they can't store additional configuration
		 * options. If the purpose has not been registered the factory will not be recorded and a warning will be printed.
		 * The provided columns will be used when matching the factory during widget construction.
		 * If registration is successful true will be returned otherwise false.
		 */
		virtual bool RegisterWidgetFactory(RowHandle PurposeRow, const UScriptStruct* Constructor,
			Queries::FConditions Columns) = 0;
	
		/**
		 * Registers a widget factory that will be called when the purpose it's registered under is requested.
		 * This version registers a generic type. Construction using these are typically cheaper as they can avoid
		 * copying the Constructor and take up less memory. The downside is that they can't store additional configuration
		 * options. If the purpose has not been registered the factory will not be recorded and a warning will be printed.
		 * The provided columns will be used when matching the factory during widget construction.
		 * If registration is successful true will be returned otherwise false.
		 */
		template<typename ConstructorType>
		bool RegisterWidgetFactory(RowHandle PurposeRow, Queries::FConditions Columns);

		/**
		 * Registers a decorator widget factory against a specific column
		 * Decorator widgets are added onto TEDS UI widgets for the specified purpose when the widget row
		 * has the column/tag they are registered against.
		 * @see FTedsDecoratorWidgetConstructor
		 */
		template<TColumnType ColumnType>
		bool RegisterDecoratorWidgetFactory(RowHandle PurposeRow, const UScriptStruct* Constructor);

		/**
		 * Registers a decorator widget factory against a specific column.
		 * Decorator widgets are added onto TEDS UI widgets for the specified purpose when the widget row
		 * has the column/tag they are registered against.
		 * @see FTedsDecoratorWidgetConstructor
		 */
		virtual bool RegisterDecoratorWidgetFactory(RowHandle PurposeRow, const UScriptStruct* Constructor,
			const UScriptStruct* Column) = 0;
	
		/** 
		 * Creates widget constructors for the requested purpose.
		 * The provided arguments will be used to configure the constructor. Settings made this way will be applied to all
		 * widgets created from the constructor, if applicable.
		 * 
		 * If no factories were found for the requested purpose, the purpose's parent chain will be traversed to try and find an ancestor purpose with factories
		 */
		virtual void CreateWidgetConstructors(RowHandle PurposeRow, 
			const FMetaDataView& Arguments, const WidgetConstructionCallback& Callback) = 0;
	
		/** 
		 * Finds matching widget constructors for provided columns, preferring longer matches over shorter matches.
		 * The provided list of columns will be updated to contain all columns that couldn't be matched.
		 * The provided arguments will be used to configure the constructor. Settings made this way will be applied to all
		 * widgets created from the constructor, if applicable.
		 *
		 * If there are still columns remaining after matching against the requested purpose, the purpose's parent chain will be traversed to try and match
		 * the remaining columns
		 */
		virtual void CreateWidgetConstructors(RowHandle PurposeRow, EMatchApproach MatchApproach, TArray<TWeakObjectPtr<const UScriptStruct>>& Columns,
			const FMetaDataView& Arguments, const WidgetConstructionCallback& Callback) = 0;
	
		/**
		 * Creates all the widgets registered under the provided purpose. This may be a large number of widgets for a wide name
		 * or exactly one when the exact name of the widget is registered. Arguments can be provided, but widgets are free
		 * to ignore them.
		 * 
		 * If no factories were found for the requested purpose, the purpose's parent chain will be traversed to try and find an ancestor purpose with factories
		 */
		virtual void ConstructWidgets(RowHandle PurposeRow, const FMetaDataView& Arguments,
			const WidgetCreatedCallback& ConstructionCallback) = 0;
	
		/** 
		 * Creates a single widget using the provided constructor. 
		 * The provided row will be used to store the widget information on. If columns have already been added to the row, the 
		 * constructor is free to use that to configure the widget. Arguments are used by the constructor to configure the widget.
		 */
		virtual TSharedPtr<SWidget> ConstructWidget(RowHandle Row, FTypedElementWidgetConstructor& Constructor,
			const FMetaDataView& Arguments) = 0;

		/** 
		 * Creates a single widget using the provided constructor. Alternative to ConstructWidget that doesn't create the ContainerTedsWidget that
		 * wraps the actual widget. Only recommended for advanced use cases, in most situations you want to use ConstructWidget
		 * NOTE: This will include any decorator widgets that match with the given row
		 */
		virtual TSharedPtr<SWidget> ConstructInternalWidget(RowHandle Row, FTypedElementWidgetConstructor& Constructor,
			const FMetaDataView& Arguments) = 0;

		/** Create the container widget that every TEDS UI widget is stored in */
		virtual TSharedPtr<ITedsWidget> CreateContainerTedsWidget(RowHandle UiRowHandle) const = 0;

		/** Get the table where TEDS UI widgets are stored */
		virtual TableHandle GetWidgetTable() const = 0;

		/**
		 * @section Searcher generation
		 * Widget constructors can provide a custom searcher object that will allow Slate columns to convert Column data to an FString
		 * that the searcher supports. If none is provided however, TEDS UI will generate one based on the properties in a column that
		 * are marked with the meta data "Searchable". Out of the box TEDS UI supports string property types (FName, FString, FText), but
		 * additional properties can be registered with the functions below as well. Note that this is mechanic is exclusively for properties
		 * that are used in the dynamic generation of searchers. For complex searching that involves data specific knowledge or combines
		 * multiple values it's highly recommended to create a custom searcher and attach it to the appropriate widget constructor.
		 */

		/** Build searchers for the provided list of columns */
		virtual void GeneratePropertySearchers(TArray<TSharedPtr<const FColumnSearcherInterface>>& Results,
			TArrayView<TWeakObjectPtr<const UScriptStruct>> Columns) const = 0;

		/** Register a function to handle converting a property into a searcher. */
		virtual void RegisterSearcherGeneratorForProperty(const FFieldClass* PropertyType, 
			PropertySearcherConstructorCallback PropertySearcherConstructor) = 0;

		/** Register a function to handle converting a property into a searcher. */
		template<PropertyClassType PropertyClass>
		void RegisterSearcherGeneratorForProperty(PropertySearcherConstructorCallback PropertySearcherConstructor);

		/** Unregister a function to handle converting a property into a searcher. */
		virtual void UnregisterSearcherGeneratorForProperty(const FFieldClass* PropertyType) = 0;

		/** Unregister a function to handle converting a property into a searcher. */
		template<PropertyClassType PropertyClass>
		void UnregisterSearcherGeneratorForProperty();
		
		/**
		 * @section Sorter generation
		 * Widget constructors can provide a custom sorter object that will allows Slate columns to sort by the data that the sorter
		 * supports. If none is provided however, TEDS UI will generate one based on the properties in a column that are marked with
		 * the meta data "Sortable". Out of the box TEDS UI supports a variety of property types, but additional properties can be 
		 * registered with the functions below as well. Note that this is mechanic is exclusively for properties that are used in
		 * the dynamic generation of sorters. For complex sorting that involves data specific knowledge or combines multiple values
		 * it's highly recommended to create a custom sorter and attach it to the appropriate widget constructor.
		 */

		/** Build sorters for the provided list of columns */
		virtual void GeneratePropertySorters(TArray<TSharedPtr<const FColumnSorterInterface>>& Results,
			TArrayView<TWeakObjectPtr<const UScriptStruct>> Columns) const = 0;

		/** Register a function to handle converting a property into a sorter. */
		virtual void RegisterSorterGeneratorForProperty(const FFieldClass* PropertyType, 
			PropertySorterConstructorCallback PropertySorterConstructor) = 0;

		/** Register a function to handle converting a property into a sorter. */
		template<PropertyClassType PropertyClass>
		void RegisterSorterGeneratorForProperty(PropertySorterConstructorCallback PropertySorterConstructor);

		/** Unregister a function to handle converting a property into a sorter. */
		virtual void UnregisterSorterGeneratorForProperty(const FFieldClass* PropertyType) = 0;

		/** Unregister a function to handle converting a property into a sorter. */
		template<PropertyClassType PropertyClass>
		void UnregisterSorterGeneratorForProperty();

		/**
		 * @section Miscellaneous
		 */

		/** Check if a custom extension is supported. This can be used to check for in-development features, custom extensions, etc. */
		virtual bool SupportsExtension(FName Extension) const = 0;
		/** Provides a list of all extensions that are enabled. */
		virtual void ListExtensions(TFunctionRef<void(FName)> Callback) const = 0;

		/**
		 * @section Deprecated
		 * Please replace use of the functions below with the recommended replacements.
		 */

		// Old widget constructor callback that takes in a UniquePtr, all new code should use the new WidgetConstructionCallback instead
		using WidgetConstructorCallback UE_DEPRECATED(5.8, "Use the new WidgetConstructionCallback alias instead") =
			TFunctionRef<bool(TUniquePtr<FTypedElementWidgetConstructor>, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)>;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		
		UE_DEPRECATED(5.8, "Use the overload that uses the new WidgetConstructionCallback instead")
		virtual void CreateWidgetConstructors(RowHandle PurposeRow, 
			const FMetaDataView& Arguments, const WidgetConstructorCallback& Callback) = 0;

		UE_DEPRECATED(5.8, "Use the overload that uses the new WidgetConstructionCallback instead")
		virtual void CreateWidgetConstructors(RowHandle PurposeRow, EMatchApproach MatchApproach, TArray<TWeakObjectPtr<const UScriptStruct>>& Columns,
			const FMetaDataView& Arguments, const WidgetConstructorCallback& Callback) = 0;
		
		UE_DEPRECATED(5.7, "Registering Widget Factories by instance is not supported anymore, register them by type instead")
		virtual bool RegisterWidgetFactory(RowHandle PurposeRow, TUniquePtr<FTypedElementWidgetConstructor>&& Constructor) = 0;
	
		UE_DEPRECATED(5.7, "Registering Widget Factories by instance is not supported anymore, register them by type instead")
		virtual bool RegisterWidgetFactory(RowHandle PurposeRow, TUniquePtr<FTypedElementWidgetConstructor>&& Constructor, 
			Queries::FConditions Columns) = 0;
		
		UE_DEPRECATED(5.6, "Use the version that takes in an FPurposeInfo struct instead")
		virtual void RegisterWidgetPurpose(FName Purpose, EPurposeType Type, FText Description) = 0;
	
		UE_DEPRECATED(5.6, "Use the version that takes in PurposeRowHandle instead. FindPurpose() to lookup row handle by PurposeID")
		virtual bool RegisterWidgetFactory(FName Purpose, const UScriptStruct* Constructor) = 0;
	
		template<typename ConstructorType>
		UE_DEPRECATED(5.6, "Use the version that takes in PurposeRowHandle instead. FindPurpose() to lookup row handle by PurposeID")
		bool RegisterWidgetFactory(FName Purpose);

		UE_DEPRECATED(5.6, "Use the version that takes in PurposeRowHandle instead. FindPurpose() to lookup row handle by PurposeID")
		virtual bool RegisterWidgetFactory(FName Purpose, const UScriptStruct* Constructor, Queries::FConditions Columns) = 0;

		template<typename ConstructorType>
		UE_DEPRECATED(5.6, "Use the version that takes in PurposeRowHandle instead. FindPurpose() to lookup row handle by PurposeID")
		bool RegisterWidgetFactory(FName Purpose, Queries::FConditions Columns);

		UE_DEPRECATED(5.6, "Use the version that takes in PurposeRowHandle instead. FindPurpose() to lookup row handle by PurposeID")
		virtual bool RegisterWidgetFactory(FName Purpose, TUniquePtr<FTypedElementWidgetConstructor>&& Constructor) = 0;

		UE_DEPRECATED(5.6, "Use the version that takes in PurposeRowHandle instead. FindPurpose() to lookup row handle by PurposeID")
		virtual bool RegisterWidgetFactory(FName Purpose, TUniquePtr<FTypedElementWidgetConstructor>&& Constructor, 
			Queries::FConditions Columns) = 0;

		UE_DEPRECATED(5.6, "Use the version that takes in PurposeRowHandle instead. FindPurpose() to lookup row handle by PurposeID")
		virtual void CreateWidgetConstructors(FName Purpose, const FMetaDataView& Arguments, const WidgetConstructorCallback& Callback) = 0;

		UE_DEPRECATED(5.6, "Use the version that takes in PurposeRowHandle instead. FindPurpose() to lookup row handle by PurposeID")
		virtual void CreateWidgetConstructors(FName Purpose, EMatchApproach MatchApproach, TArray<TWeakObjectPtr<const UScriptStruct>>& Columns,
			const FMetaDataView& Arguments, const WidgetConstructorCallback& Callback) = 0;

		UE_DEPRECATED(5.6, "Use the version that takes in PurposeRowHandle instead. FindPurpose() to lookup row handle by PurposeID")
		virtual void ConstructWidgets(FName Purpose, const FMetaDataView& Arguments, const WidgetCreatedCallback& ConstructionCallback) = 0;
		
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};

	using FWidgetConstructor = FTypedElementWidgetConstructor;
	using FSimpleWidgetConstructor = FSimpleWidgetConstructor;

	//
	// Implementations
	//

	template<typename ConstructorType>
	bool IUiProvider::RegisterWidgetFactory(FName Purpose)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return this->RegisterWidgetFactory(Purpose, ConstructorType::StaticStruct());
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	template<typename ConstructorType>
	bool IUiProvider::RegisterWidgetFactory(RowHandle PurposeRow)
	{
		return this->RegisterWidgetFactory(PurposeRow, ConstructorType::StaticStruct());
	}

	template<typename ConstructorType>
	bool IUiProvider::RegisterWidgetFactory(FName Purpose, Queries::FConditions Columns)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return this->RegisterWidgetFactory(Purpose, ConstructorType::StaticStruct(), Columns);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	template<typename ConstructorType>
	bool IUiProvider::RegisterWidgetFactory(RowHandle PurposeRow, Queries::FConditions Columns)
	{
		return this->RegisterWidgetFactory(PurposeRow, ConstructorType::StaticStruct(), Columns);
	}

	template<TColumnType ColumnType>
	bool IUiProvider::RegisterDecoratorWidgetFactory(RowHandle PurposeRow, const UScriptStruct* Constructor)
	{
		return this->RegisterDecoratorWidgetFactory(PurposeRow, Constructor, ColumnType::StaticStruct());
	}

	template<PropertyClassType PropertyClass>
	void IUiProvider::RegisterSearcherGeneratorForProperty(PropertySearcherConstructorCallback PropertySearcherConstructor)
	{
		RegisterSearcherGeneratorForProperty(PropertyClass::StaticClass(), PropertySearcherConstructor);
	}

	template<PropertyClassType PropertyClass>
	void IUiProvider::UnregisterSearcherGeneratorForProperty()
	{
		UnregisterSearcherGeneratorForProperty(PropertyClass::StaticClass());
	}

	template<PropertyClassType PropertyClass>
	void IUiProvider::RegisterSorterGeneratorForProperty(PropertySorterConstructorCallback PropertySorterConstructor)
	{
		RegisterSorterGeneratorForProperty(PropertyClass::StaticClass(), PropertySorterConstructor);
	}

	template<PropertyClassType PropertyClass>
	void IUiProvider::UnregisterSorterGeneratorForProperty()
	{
		UnregisterSorterGeneratorForProperty(PropertyClass::StaticClass());
	}
}