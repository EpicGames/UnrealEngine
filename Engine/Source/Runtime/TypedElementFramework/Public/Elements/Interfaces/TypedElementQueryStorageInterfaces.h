// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "DataStorage/CommonTypes.h"
#include "DataStorage/Handles.h"
#include "DataStorage/MapKey.h"
#include "DataStorage/Queries/Types.h"
#include "Elements/Framework/TypedElementQueryCapabilities.h"
#include "UObject/Class.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Templates/Function.h"

class UClass;
class UObject;
class UScriptStruct;

namespace UE::Editor::DataStorage
{
	struct FQueryDescription;
	struct ISubqueryContext;

	using SubqueryCallback = TFunction<void(const FQueryDescription&, ISubqueryContext&)>;
	using SubqueryCallbackRef = TFunctionRef<void(const FQueryDescription&, ISubqueryContext&)>;
	
	/**
	 * Base interface for any contexts provided to query callbacks.
	 */
	struct ICommonQueryContext : public 
		Queries::ITableInfo<
		Queries::IDynamicColumnInfo<
		Queries::IContextCapability>>
	{
		virtual ~ICommonQueryContext() = default;

		/** Returns the number rows in the batch. */
		virtual uint32 GetRowCount() const = 0;
		/**
		 * Returns an immutable view that contains the row handles for all returned results. The returned size will be the same  as the
		 * value returned by GetRowCount().
		 */
		virtual TConstArrayView<RowHandle> GetRowHandles() const = 0;

		/** Return the address of a immutable column matching the requested type or a nullptr if not found. */
		virtual const void* GetColumn(const UScriptStruct* ColumnType) const = 0;
		/** Return the address of a immutable column matching the requested type or a nullptr if not found. */
		template<TDataColumnType Column>
		const Column* GetColumn() const;
		template<TDynamicColumnTemplate TemplateType>
		const TemplateType* GetColumn(const FName& Identifier) const;
		/** Return the address of a mutable column matching the requested type or a nullptr if not found. */
		virtual void* GetMutableColumn(const UScriptStruct* ColumnType) = 0;
		/** Return the address of a mutable column matching the requested type or a nullptr if not found. */
		template<typename Column>
		Column* GetMutableColumn();
		template<TDynamicColumnTemplate TemplateType>
		TemplateType* GetMutableColumn(const FName& Identifier);

		/**
		 * Get a list of columns or nullptrs if the column type wasn't found. Mutable addresses are returned and it's up to
		 * the caller to not change immutable addresses.
		 */
		virtual void GetColumns(TArrayView<char*> RetrievedAddresses, TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ColumnTypes,
			TConstArrayView<EQueryAccessType> AccessTypes) = 0;
		/**
		 * Get a list of columns or nullptrs if the column type wasn't found. Mutable addresses are returned and it's up to
		 * the caller to not change immutable addresses. This version doesn't verify that the enough space is provided and
		 * it's up to the caller to guarantee the target addresses have enough space.
		 */
		virtual void GetColumnsUnguarded(int32 TypeCount, char** RetrievedAddresses, const TWeakObjectPtr<const UScriptStruct>* ColumnTypes,
			const EQueryAccessType* AccessTypes) = 0;
		/* 
		 * Returns whether a column matches the requested type or not. This version only applies to the table that's currently set in
		 * the context. This version is faster of checking in the current row or table, but the version using a row is needed to check
		 * arbitrary rows.
		 */
		virtual bool HasColumn(const UScriptStruct* ColumnType) const = 0;
		/*
		 * Returns whether a column matches the requested type or not. This version only applies to the table that's currently set in
		 * the context. This version is faster of checking in the current row or table, but the version using a row is needed to check
		 * arbitrary rows.
		 */
		template<typename Column>
		bool HasColumn() const;
		/*
		 * Return whether a column matches the requested type or not. This can be used for arbitrary rows. If the row is in the
		 * table that's set in the context, for instance because it's the current row, then the version that doesn't take a row
		 * as an argument is recommended.
		 */
		virtual bool HasColumn(RowHandle Row, const UScriptStruct* ColumnType) const = 0;
		/*
		 * Return whether a column matches the requested type or not. This can be used for arbitrary rows. If the row is in the
		 * table that's set in the context, for instance because it's the current row, then the version that doesn't take a row
		 * as an argument is recommended.
		 */
		template<typename Column>
		bool HasColumn(RowHandle Row) const;

		template<TDynamicColumnTemplate DynamicColumnTemplate>
		bool HasColumn(RowHandle Row, const FName& Identifier) const;

		// Sets the Target's Parent if the query has a configured Hierarchy
		virtual void SetParentRow(
			RowHandle Target,
			RowHandle Parent) = 0;

		/**
		 * Establishes a parent relationship between the Target row and a Parent that is not registered in TEDS yet.
		 * Every frame, TEDS will attempt to resolve the missing relation by looking up the parent using TEDS Mapping.
		 * An optional MaxFrameCount can be specified, which will be decremented every frame until the potential relationship is discarded
		 */
		virtual void SetUnresolvedParent(
			RowHandle Target,
			FMapKey ParentId,
			FName MappingDomain) = 0;

		// Gets the Target's Parent if the query has a configured Hierarchy
		virtual RowHandle GetParentRow(
			RowHandle Target) const = 0;

		/**
		 * Defers a relation change until the end of the current chunk loop.
		 * Removes any existing relation of this type where Target is the Subject,
		 * then creates a new one with NewObject as the Object.
		 * Pass InvalidRowHandle for NewObject to remove the existing relation without creating a new one.
		 * Safe to call from within a processor callback — does not call ICoreProvider directly.
		 */
		virtual void SetRelation(
			RelationTypeHandle Type,
			RowHandle Target,
			RowHandle NewObject) = 0;

		/** Returns the amount of time now and the last update in seconds. */
		virtual float GetDeltaTimeSeconds() const = 0;

		/**
		 * Creates a command that will run at the termination of an execution context.
		 * Intended to be used for cases where the query callback requires doing some work that needs to be serialized
		 * with respect to other callback invocations.
		 * Note that commands will be executed in order with respect to the thread that pushed them and will be executed
		 * on the query exection thread serially.
		 *
		 * Usage:
		 *  Define a command struct with a mutable operator() overload
		 *  struct FMyCommand
		 *  {
		 *      void operator()() { DoSomethingWithSideEffects(MyActor) };
		 *      TWeakObjectPtr<AActor> MyActor;
		 *  };
		 *
		 *  Context.PushCommand(FMyCommand{ .MyActor = Actor });
		 */
		template<typename T>
		void PushCommand(T CommandContext);
		
		virtual void PushCommand(void (*CommandFunction)(void* /*CommandData*/), void* InCommandData) = 0;
	protected:
		struct FEmplaceObjectParams
		{
			size_t ObjectSize;
			size_t Alignment;
			void (*Construct)(void*, void*);
			void(*Destroy)(void*);
			void* SourceObject;
		};
		// Required as the interface doesn't have direct access to any temporary handling of memory in the backend.
		virtual void* EmplaceObjectInScratch(const FEmplaceObjectParams& Params) = 0;
	};

	struct ICommonQueryWithEnvironmentContext : public ICommonQueryContext
	{
		using ObjectCopyOrMove = void (*)(const UScriptStruct& TypeInfo, void* Destination, void* Source);

		using ICommonQueryContext::GetColumn;
		using ICommonQueryContext::HasColumn;
		using ICommonQueryContext::GetMutableColumn;
		
		/**
		 * Helper template that returns a const reference to the column array data for the range of rows being processed
		 * Returns nullptr if type not found or column not assigned to row
		 * @warning It is advised to only use this function in a query callback that handles multiple rows
		 *          the returned reference will be to the first column in the table chunk, not necessarily
		 *          the column corresponding to the row that the singular callback form is using.
		 */
		template<typename DynamicColumnTemplate>
		const DynamicColumnTemplate* GetColumn(const FName& Identifier) const;

		/**
		 * Helper template that returns the reference to the column array data for the range of rows being processed
		 * Returns nullptr if type not found or column not assigned to row
		 * @warning It is advised to only use this function in a query callback that handles multiple rows
		 *          the returned reference will be to the first column in the table chunk, not necessarily
		 *          the column corresponding to the row that the singular callback form is using.
		 */
		template<typename DynamicColumnTemplate>
		DynamicColumnTemplate* GetMutableColumn(const FName& Identifier);

		/*
		 * Helper template to return whether a row has a given dynamic column.
		 */
		template<typename DynamicColumnTemplate>
		bool HasColumn(const FName& Identifier) const;
		
		/*
		 * Helper template to return whether a row has a given dynamic column.
		 */
		template<typename DynamicColumnTemplate>
		bool HasColumn(RowHandle Row, const FName& Identifier) const;

		/**
		 * Returns the id for the current update cycle. Every time TEDS goes through a cycle of running query callbacks, this is
		 * incremented by one. This guarantees that all query callbacks in the same run see the same cycle id. This can be useful
		 * in avoiding duplicated work.
		 */
		virtual uint64 GetUpdateCycleId() const = 0;
		
		/** Checks whether or not a row is in use. This is true even if the row has only been reserved. */
		virtual bool IsRowAvailable(RowHandle Row) const = 0;
		/** Checks whether or not a row has been reserved but not yet assigned to a table. */
		virtual bool IsRowAssigned(RowHandle Row) const = 0;

		/**
		 * Triggers all queries registered under the activation name to run for one update cycle. The activatable queries will be activated at
		 * start of the cycle and disabled at the end of the cycle and act like regular queries for that cycle. This includes not running
		 * if there are not columns to match against.
		 */
		virtual void ActivateQueries(FName ActivationName) = 0;

		/**
		 * Adds a row to the given table.
		 */
		virtual RowHandle AddRow(TableHandle Table) = 0;
		
		/**
		 * Removes the row with the provided row handle. The removal will not be immediately done but delayed until the end of the tick
		 * group.
		 */
		virtual void RemoveRow(RowHandle Row) = 0;
		/**
		 * Removes rows with the provided row handles. The removal will not be immediately done but delayed until the end of the tick
		 * group.
		 */
		virtual void RemoveRows(TConstArrayView<RowHandle> Rows) = 0;

		/**
		 * Adds the provided column to the requested row.
		 *
		 * Note: The addition of the column will not be immediately done. Instead it will be deferred until the end of the tick group. Changes
		 * made to the return column will still be applied when the column is added to the row.
		 */
		template<typename ColumnType>
		ColumnType& AddColumn(RowHandle Row, ColumnType&& Column);
		/**
		 * Helper template for adding a dynamic column to the requested row.  The resulting column will be default initialized.
		 * @tparam ColumnTypeTemplate The template layout of the column
		 * @param Row The row to add the column to
		 * @param Identifier Combined with the template type, the dynamic column is uniquely identified with this parameter
		 * @return An ephemeral reference to the data of the column.  The returned column will have been constructed.
		 */
		template<typename ColumnTypeTemplate>
		ColumnTypeTemplate* AddColumn(RowHandle Row, const FName& Identifier);
		/**
		 * Helper template for adding a dynamic column to the requested row
		 * @tparam ColumnTypeTemplate The template layout of the column
		 * @param Row The row to add the column to
		 * @param Identifier Combined with the template type, the dynamic column is uniquely identified with this parameter
		 * @return An ephemeral reference to the data of the column.  The returned column will have been constructed.
		 */
		template<typename ColumnTypeTemplate>
		ColumnTypeTemplate& AddColumn(RowHandle Row, const FName& Identifier, ColumnTypeTemplate&& Column);
		/**
		 * Adds new empty columns to a row of the provided type. The addition will not be immediately done but delayed until the end of the
		 * tick group.
		 */
		template<typename... Columns>
		void AddColumns(RowHandle Row);
		/**
		 * Adds new empty columns to the listed rows of the provided type. The addition will not be immediately done but delayed until the end of the
		 * tick group.
		 */
		template<typename... Columns>
		void AddColumns(TConstArrayView<RowHandle> Rows);
		/**
		 * Adds new empty columns to a row of the provided type. The addition will not be immediately done but delayed until the end of the
		 * tick group.
		 */
		virtual void AddColumns(RowHandle Row, TConstArrayView<const UScriptStruct*> ColumnTypes) = 0;
		/**
		 * Adds new empty columns to the listed rows of the provided type. The addition will not be immediately done but delayed until the end of the
		 * tick group.
		 */
		virtual void AddColumns(TConstArrayView<RowHandle> Rows, TConstArrayView<const UScriptStruct*> ColumnTypes) = 0;
		
		virtual void AddColumns(TConstArrayView<RowHandle> Rows, TConstArrayView<FDynamicColumnDescription> DynamicColumnDescriptions) = 0;
		/**
		 * Add a new uninitialized column of the provided type if one does not exist.
		 * Returns a staged column which is used to copy into the database at a later time via the UStructScript Copy operator at the end
		 * of the tick group.
		 * It is the caller's responsibility to ensure the staged column's constructor is called. The caller may modify other
		 * values in the column.
		 * This function can not be used to add a tag as tags do not contain any data.
		 */
		virtual void* AddColumnUninitialized(RowHandle Row, const UScriptStruct* ColumnType) = 0;
		/**
		 * Add a new uninitialized column of the provided type if one does not exist.
		 * Returns a staged column which is used to copy/move into the database at a later time via the provided relocator at the end of
		 * the tick group.
		 * It is the caller's responsibility to ensure the staged column's constructor is called. The caller may modify other
		 * values in the column.
		 * This function can not be used to add a tag as tags do not contain any data.
		 */
		virtual void* AddColumnUninitialized(RowHandle Row, const UScriptStruct* ObjectType, ObjectCopyOrMove Relocator) = 0;

		virtual void* AddColumnUninitialized(RowHandle Row, const FDynamicColumnDescription& DynamicColumnDescription, ObjectCopyOrMove Relocator) = 0;
		virtual void* AddColumnUninitialized(RowHandle Row, const FDynamicColumnDescription& DynamicColumnDescription) = 0;
		

		/**
		 * Removes columns of the provided types from a row. The removal will not be immediately done but delayed until the end of the
		 * tick group.
		 */
		template<typename... Columns>
		void RemoveColumns(RowHandle Row);
		/**
		 * Removes columns of the provided types from the listed rows. The removal will not be immediately done but delayed until the end of the
		 * tick group.
		 */
		template<typename... Columns>
		void RemoveColumns(TConstArrayView<RowHandle> Rows);
		/**
		 * Removes columns of the provided types from a row. The removal will not be immediately done but delayed until the end of the
		 * tick group.
		 */
		virtual void RemoveColumns(RowHandle Row, TConstArrayView<const UScriptStruct*> ColumnTypes) = 0;
		/**
		 * Removes columns of the provided types from the listed rows. The removal will not be immediately done but delayed until the end of the
		 * tick group.
		 */
		virtual void RemoveColumns(TConstArrayView<RowHandle> Rows, TConstArrayView<const UScriptStruct*> ColumnTypes) = 0;
	};

	enum class EDirectQueryExecutionFlags : uint32
	{
		Default = 0, //< No settings, use the default behavior.
		ParallelizeChunks = 1 << 0, //< If set, each chunk is processed on a separate thread.
		/*
		 * Requires ParallelizeChunks to be set.
		 * If set together with ParallelizeChunks, each chunk will be scheduled individually. This allows work to be better
		 * distributed among threads, but comes at additional scheduling cost. Add this flag if the processing time per chunk varies.
		 */
		AutoBalanceParallelChunkProcessing = 1 << 1,
		IgnoreActiveState = 1 << 2, //< If set, a direct call will not check activatable queries if they're set.
		AllowBoundQueries = 1 << 3, //< Stops checking if queries are bound when set, otherwise bound queries will assert.

		IgnoreActivationCount UE_DEPRECATED("5.7", "Activation counts are no longer supported.") = IgnoreActiveState,
	};
	ENUM_CLASS_FLAGS(EDirectQueryExecutionFlags);

	// For now this is an exact mirror of EDirectQueryExecutionFlags (except deprecated flags) in order to separate the old and the new 
	// query callbacks/context. Please keep both in sync, except for new entries, until the old query callbacks/context has been fully 
	// deprecated.
	enum class ERunQueryFlags : uint32
	{
		Default = 0, //< No settings, use the default behavior.
		ParallelizeChunks = 1 << 0, //< If set, each chunk is processed on a separate thread.
		/*
		 * Requires ParallelizeChunks to be set.
		 * If set together with ParallelizeChunks, each chunk will be scheduled individually. This allows work to be better
		 * distributed among threads, but comes at additional scheduling cost. Add this flag if the processing time per chunk varies.
		 */
		AutoBalanceParallelChunkProcessing = 1 << 1,
		IgnoreActiveState = 1 << 2, //< If set, a direct call will not check activatable queries if they're set.
		AllowBoundQueries = 1 << 3, //< Stops checking if queries are bound when set, otherwise bound queries will assert.
	};
	ENUM_CLASS_FLAGS(ERunQueryFlags);

	/**
	 * Interface to be provided to query callbacks that are directly called through RunQuery from outside a query callback.
	 */
	struct IDirectQueryContext : public ICommonQueryContext
	{
		virtual ~IDirectQueryContext() = default;
	};

	/**
	 * Interface to be provided to query callbacks that are directly called through from a query callback.
	 */
	struct ISubqueryContext : public ICommonQueryWithEnvironmentContext
	{
		virtual ~ISubqueryContext() = default;
	};

	/**
	 * Interface to be provided to query callbacks running with the Data Storage.
	 * Note that at the time of writing only subclasses of Subsystem are supported as dependencies.
	 */
	struct IQueryContext : public ICommonQueryWithEnvironmentContext
	{
		virtual ~IQueryContext() = default;

		/** Returns an immutable instance of the requested dependency or a nullptr if not found. */
		virtual const UObject* GetDependency(const UClass* DependencyClass) = 0;
		/** Returns a mutable instance of the requested dependency or a nullptr if not found. */
		virtual UObject* GetMutableDependency(const UClass* DependencyClass) = 0;
		
		/**
		 * Returns a list of dependencies or nullptrs if a dependency wasn't found. Mutable versions are return and it's up to the
		 * caller to not change immutable dependencies.
		 */
		virtual void GetDependencies(TArrayView<UObject*> RetrievedAddresses, TConstArrayView<TWeakObjectPtr<const UClass>> DependencyTypes,
			TConstArrayView<EQueryAccessType> AccessTypes) = 0;

		/** Retrieves the row for a mapped object in the provided domain. Returns an invalid row handle if the key wasn't found. */
		virtual RowHandle LookupMappedRow(const FName& Domain, const FMapKeyView& Index) const = 0;

		/**
		 * Runs a previously created query. This version takes an arbitrary query, but is limited to running queries that do not directly
		 * access data from rows such as count queries.
		 * The returned result is a snap shot and values may change between phases.
		 */
		virtual FQueryResult RunQuery(QueryHandle Query) = 0;
		/** 
		 * Runs a subquery registered with the current query. The subquery index is in the order of registration with the query. Subqueries
		 * are executed as part of their parent query and are not scheduled separately.
		 */
		virtual FQueryResult RunSubquery(int32 SubqueryIndex) = 0;
		/** 
		 * Runs the provided callback on a subquery registered with the current query. The subquery index is in the order of registration 
		 * with the query. Subqueries are executed as part of their parent query and are not scheduled separately.
		 */
		virtual FQueryResult RunSubquery(int32 SubqueryIndex, SubqueryCallbackRef Callback) = 0;
		/** 
		 * Runs the provided callback on a subquery registered with the current query for the exact provided row. The subquery index is in 
		 * the order of registration with the query. If the row handle is in a table that doesn't match the selected subquery the callback
		 * will not be called. Check the count in the returned results to determine if the callback was called or not. Subqueries
		 * are executed as part of their parent query and are not scheduled separately.
		 */
		virtual FQueryResult RunSubquery(int32 SubqueryIndex, RowHandle Row, SubqueryCallbackRef Callback) = 0;

		/*UE_DEPRECATED(5.6, "Use 'LookUpMappedRow' instead and use the new FMapKey(View) instead of the explicit 'GenerateIndexHash'-functions.")
		inline RowHandle FindIndexedRow(IndexHash Index) const;
		UE_DEPRECATED(5.7, "Use the version of this 'LookupMappedRow' that uses a domain.")
		inline RowHandle LookupMappedRow(const FMapKeyView& Index) const;*/

		
	};
} // namespace UE::Editor::DataStorage




//
// Implementations
//

namespace UE::Editor::DataStorage
{
	//
	// ICommonQueryContext
	//

	template<TDataColumnType Column>
	const Column* ICommonQueryContext::GetColumn() const
	{
		return reinterpret_cast<const Column*>(GetColumn(Column::StaticStruct()));
	}

	template <TDynamicColumnTemplate TemplateType>
	const TemplateType* ICommonQueryContext::GetColumn(const FName& Identifier) const
	{
		const UScriptStruct* DynamicColumnType = FindDynamicColumnType<TemplateType>(Identifier);
		return static_cast<const TemplateType*>(GetColumn(DynamicColumnType));
	}

	template<typename Column>
	Column* ICommonQueryContext::GetMutableColumn()
	{
		return reinterpret_cast<Column*>(GetMutableColumn(Column::StaticStruct()));
	}

	template <TDynamicColumnTemplate TemplateType>
	TemplateType* ICommonQueryContext::GetMutableColumn(const FName& Identifier)
	{
		const UScriptStruct* DynamicColumnType = FindDynamicColumnType<TemplateType>(Identifier);
		return static_cast<TemplateType*>(GetMutableColumn(DynamicColumnType));
	}

	template <typename Column>
	bool ICommonQueryContext::HasColumn() const
	{
		return HasColumn(Column::StaticStruct());
	}

	template <typename Column>
	bool ICommonQueryContext::HasColumn(RowHandle Row) const
	{
		return HasColumn(Row, Column::StaticStruct());
	}

	template <TDynamicColumnTemplate DynamicColumnTemplate>
	bool ICommonQueryContext::HasColumn(RowHandle Row, const FName& Identifier) const
	{
		const UScriptStruct* DynamicColumnType = FindDynamicColumnType<DynamicColumnTemplate>(Identifier);
		return DynamicColumnType ? HasColumn(Row, DynamicColumnType) : false;
	}

	template <typename DynamicColumnTemplate>
	const DynamicColumnTemplate* ICommonQueryWithEnvironmentContext::GetColumn(const FName& Identifier) const
	{
		using namespace UE::Editor::DataStorage;
		const UScriptStruct* StructInfo = FindDynamicColumnType(*DynamicColumnTemplate::StaticStruct(), Identifier);
		if (StructInfo)
		{
			const void* ColumnData = GetColumn(StructInfo);
			return static_cast<const DynamicColumnTemplate*>(ColumnData);
		}
		return nullptr;
	}

	template <typename DynamicColumnTemplate>
	DynamicColumnTemplate* ICommonQueryWithEnvironmentContext::GetMutableColumn(const FName& Identifier)
	{
		using namespace UE::Editor::DataStorage;
		const UScriptStruct* StructInfo = FindDynamicColumnType(*DynamicColumnTemplate::StaticStruct(), Identifier);
		if (StructInfo)
		{
			const void* ColumnData = GetMutableColumn(StructInfo);
			return static_cast<const DynamicColumnTemplate*>(ColumnData);
		}
		return nullptr;
	}

	template <typename DynamicColumnTemplate>
	bool ICommonQueryWithEnvironmentContext::HasColumn(const FName& Identifier) const
	{
		using namespace UE::Editor::DataStorage;
		const UScriptStruct* StructInfo = FindDynamicColumnType(*DynamicColumnTemplate::StaticStruct(), Identifier);
		if (StructInfo)
		{
			return ICommonQueryContext::HasColumn(StructInfo);
		}
		return false;
	}

	template <typename DynamicColumnTemplate>
	bool ICommonQueryWithEnvironmentContext::HasColumn(RowHandle Row, const FName& Identifier) const
	{
		using namespace UE::Editor::DataStorage;
		const UScriptStruct* StructInfo = FindDynamicColumnType(*DynamicColumnTemplate::StaticStruct(), Identifier);
		if (StructInfo)
		{
			return ICommonQueryContext::HasColumn(Row, StructInfo);
		}
		return false;
	}

	//
	// ICommonQueryWithEnvironmentContext
	// 

	template<typename ColumnType>
	ColumnType& ICommonQueryWithEnvironmentContext::AddColumn(RowHandle Row, ColumnType&& Column)
	{
		const UScriptStruct* TypeInfo = ColumnType::StaticStruct();

		if constexpr (std::is_move_constructible_v<ColumnType>)
		{
			void* Address = AddColumnUninitialized(Row, TypeInfo,
				[](const UScriptStruct&, void* Destination, void* Source)
				{
					*reinterpret_cast<ColumnType*>(Destination) = MoveTemp(*reinterpret_cast<ColumnType*>(Source));
				});
			return *(new(Address) ColumnType(Forward<ColumnType>(Column)));
		}
		else
		{
			void* Address = AddColumnUninitialized(Row, TypeInfo);
			TypeInfo->CopyScriptStruct(Address, &Column);
			return *reinterpret_cast<ColumnType*>(Address);
		}
	}

	template <typename ColumnTypeTemplate>
	ColumnTypeTemplate* ICommonQueryWithEnvironmentContext::AddColumn(RowHandle Row, const FName& Identifier)
	{
		const UScriptStruct* TemplateType = ColumnTypeTemplate::StaticStruct();
		const FDynamicColumnDescription Description
		{
			.TemplateType = TemplateType,
			.Identifier = Identifier
		};

		if constexpr (TDataColumnType<ColumnTypeTemplate>)
		{
			ColumnTypeTemplate* ColumnData = static_cast<ColumnTypeTemplate*>(AddColumnUninitialized(Row, Description));
			new (ColumnData) ColumnTypeTemplate();
			return ColumnData;
		}
		if constexpr (TTagColumnType<ColumnTypeTemplate>)
		{
			AddColumns({Row}, {Description});
			return nullptr;
		}

		return  nullptr;
	}

	template <typename ColumnTypeTemplate>
	ColumnTypeTemplate& ICommonQueryWithEnvironmentContext::AddColumn(RowHandle Row, const FName& Identifier, ColumnTypeTemplate&& Column)
	{
		const UScriptStruct* TemplateType = ColumnTypeTemplate::StaticStruct();
		const FDynamicColumnDescription Description
		{
			.TemplateType = TemplateType,
			.Identifier = Identifier
		};

		if constexpr (std::is_move_constructible_v<ColumnTypeTemplate>)
		{
			void* Address = AddColumnUninitialized(Row, Description,
				[](const UScriptStruct&, void* Destination, void* Source)
				{
					*static_cast<ColumnTypeTemplate*>(Destination) = MoveTemp(*static_cast<ColumnTypeTemplate*>(Source));
				});
			return *(new(Address) ColumnTypeTemplate(Forward<ColumnTypeTemplate>(Column)));
		}
		else
		{
			void* Address = AddColumnUninitialized(Row, Description);
			new (Address) ColumnTypeTemplate(Forward<ColumnTypeTemplate>(Column));
			return *static_cast<ColumnTypeTemplate*>(Address);
		}
	}

	template<typename... Columns>
	void ICommonQueryWithEnvironmentContext::AddColumns(RowHandle Row)
	{
		AddColumns(Row, { Columns::StaticStruct()... });
	}

	template<typename... Columns>
	void ICommonQueryWithEnvironmentContext::AddColumns(TConstArrayView<RowHandle> Rows)
	{
		AddColumns(Rows, { Columns::StaticStruct()... });
	}

	template<typename... Columns>
	void ICommonQueryWithEnvironmentContext::RemoveColumns(RowHandle Row)
	{
		RemoveColumns(Row, { Columns::StaticStruct()... });
	}

	template<typename... Columns>
	void ICommonQueryWithEnvironmentContext::RemoveColumns(TConstArrayView<RowHandle> Rows)
	{
		RemoveColumns(Rows, { Columns::StaticStruct()... });
	}

	template <typename T>
	void ICommonQueryContext::PushCommand(T CommandContext)
	{
		if constexpr(std::is_empty_v<T>)
		{
			// The command object doesn't hold any arguments so use a cheaper path that only stores the callback.
			PushCommand(
				[](void*)
				{
					T Instance;
					Instance.operator()();
				}, nullptr);
		}
		else
		{
			// Wrapper lambda to store the type and call operator().
			auto CommandFunction = [](void* InInstanceOfT)
				{
					T* Instance = static_cast<T*>(InInstanceOfT);
					Instance->operator()();
				};

			FEmplaceObjectParams Params;

			Params.ObjectSize = sizeof(T);
			Params.Alignment = alignof(T);
			Params.Construct = [](void* Destination, void* SourceCommandContext)
			{
				T& SourceCommand = *static_cast<T*>(SourceCommandContext);
				new (Destination) T(MoveTemp(SourceCommand));
			};
			if (std::is_trivially_destructible_v<T>)
			{
				Params.Destroy = nullptr;
			}
			else
			{
				Params.Destroy = [](void* EmplacedObject)
				{
					static_cast<T*>(EmplacedObject)->~T();
				};
			}
				
			Params.SourceObject = &CommandContext;
				
			void* EmplacedCommandContext = EmplaceObjectInScratch(Params);
			PushCommand(CommandFunction, EmplacedCommandContext);
		}
	}



	//
	// IQueryContext
	//

	/*RowHandle IQueryContext::FindIndexedRow(IndexHash Index) const
	{
		return LookupMappedRow(NAME_None, FMapKeyView(Index));
	}

	RowHandle IQueryContext::LookupMappedRow(const FMapKeyView& Index) const
	{
		return LookupMappedRow(NAME_None, FMapKeyView(Index));
	}*/

} // namespace UE::Editor::DataStorage
