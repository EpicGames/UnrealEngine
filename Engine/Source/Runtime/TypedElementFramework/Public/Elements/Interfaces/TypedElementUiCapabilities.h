// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Types/ISlateMetaData.h"

/**
 * UI Capabilities are objects that can be stored as meta data on widgets created through TEDS UI and
 * describe the functionality offered by the widget. This allows arbitrary widgets to describe themselves
 * and allows users of TEDS UI to call into or register with the widget.
 */
class UE_DEPRECATED(5.8, "Please store metadata in columns instead of using UI capabilities") ITypedElementUiCapability : public ISlateMetaData
{
public:
	SLATE_METADATA_TYPE(ITypedElementUiCapability, ISlateMetaData)

	~ITypedElementUiCapability() override = default;
};