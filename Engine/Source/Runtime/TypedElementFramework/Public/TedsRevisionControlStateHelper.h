// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DataStorage/Handles.h"
#include "Elements/Columns/TedsRevisionControlColumns.h"

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
}

namespace UE::Editor::RevisionControl
{
	/**
	 * Look up the package row for InFilename in the SourceControl mapping domain,
	 * creating one if it doesn't exist. Returns InvalidRowHandle if the package
	 * table hasn't been registered yet.
	 */
	TYPEDELEMENTFRAMEWORK_API UE::Editor::DataStorage::RowHandle FindOrAddPackageRow(
		UE::Editor::DataStorage::ICoreProvider& DataStorage,
		const FString& InFilename);

	/**
	 * Write NewState to Row, always add FSCCStatusUpdateEndedTag, and activate
	 * UpdateSCCForActors so the viewport overlay refreshes. Callers build NewState
	 * from their provider-specific state (FPerforceSourceControlState /
	 * FSkeinSourceControlState) and hand it off.
	 */
	TYPEDELEMENTFRAMEWORK_API void ApplySccStateUpdate(
		UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::RowHandle Row,
		const FSccStateColumn& NewState);
}
