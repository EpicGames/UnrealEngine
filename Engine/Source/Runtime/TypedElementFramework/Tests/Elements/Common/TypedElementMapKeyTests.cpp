// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "DataStorage/MapKey.h"
#include "Tests/TestHarnessAdapter.h"

namespace UE::Editor::DataStorage::Tests::Private
{
	FName TestFName = FName("FMapKey_TestName");
	FSoftObjectPath TestPath(FTopLevelAssetPath(FName("FMapKey_TestPackage"), FName("FMapKey_TestAsset")), TEXT("FMapKey_TestSubPath"));

	void CopyKeyTest(const FMapKey& Original)
	{
		FMapKey Copy = Original;
		CHECK_MESSAGE(TEXT("Copied key was not empty as expected."), Copy.IsSet());
		CHECK_MESSAGE(TEXT("Original and copied key don't have the same value."), Original == Copy);
	}

	void MoveKeyTest(FMapKey&& Original)
	{
		FMapKey Moved = MoveTemp(Original);
		CHECK_MESSAGE(TEXT("Original is still set after moving."), !Original.IsSet());
		CHECK_MESSAGE(TEXT("Moved key is empty."), Moved.IsSet());
		CHECK_MESSAGE(TEXT("Original and moved shouldn't have the same value."), Original != Moved);
	}

	void ConvertKeyToKeyViewTest(const FMapKey& Key)
	{
		FMapKeyView View(Key);
		CHECK_MESSAGE(TEXT("Key to View comparison failed"), Key == View);
		CHECK_MESSAGE(TEXT("View to Key comparison failed"), View == Key);
	}

	TEST_CASE_NAMED(FMapKey_Tests, "Editor::DataStorage::Map Key (FMapKey)", "[ApplicationContextMask][EngineFilter]")
	{
		using namespace UE::Editor::DataStorage;
		using namespace UE::Editor::DataStorage::Tests::Private;

		//
		// Construction
		//

		SECTION("Empty key")
		{
			FMapKey Key;
			CHECK_MESSAGE(TEXT("Key was not empty as expected."), !Key.IsSet());
			CHECK_MESSAGE(TEXT("Hash is not zero"), Key.CalculateHash() == 0);
			CHECK_MESSAGE(TEXT("Expected string wasn't returned"), Key.ToString() == TEXT("Empty"));
			CHECK_MESSAGE(TEXT("Self comparison failed"), Key == Key);
		}
		
		SECTION("Construct key (void*)")
		{
			int* Object = reinterpret_cast<int*>(0x1234);
			FMapKey Key(Object);
			CHECK_MESSAGE(TEXT("Key was not set."), Key.IsSet());
			CHECK_MESSAGE(TEXT("Hash is was zero"), Key.CalculateHash() != 0);
			FString KeyString = Key.ToString();
			CHECK_MESSAGE(TEXT("Expected string wasn't returned"), 
				KeyString.StartsWith(TEXT("Pointer(0x"))
				&& KeyString.EndsWith(TEXT("1234)")));
			CHECK_MESSAGE(TEXT("Self comparison failed"), Key == Key);
		}

		SECTION("Construct key (UObject*)")
		{
			UObject* Object = reinterpret_cast<UObject*>(0x1234);
			FMapKey Key(Object);
			CHECK_MESSAGE(TEXT("Key was not set."), Key.IsSet());
			CHECK_MESSAGE(TEXT("Hash is was zero"), Key.CalculateHash() != 0);
			FString KeyString = Key.ToString();
			CHECK_MESSAGE(TEXT("Expected string wasn't returned"), 
				KeyString.StartsWith(TEXT("UObject(0x"))
				&& KeyString.EndsWith(TEXT("1234)")));
			CHECK_MESSAGE(TEXT("Self comparison failed"), Key == Key);
		}

		SECTION("Construct key (int64)")
		{
			int64 Object = 42;
			FMapKey Key(Object);
			CHECK_MESSAGE(TEXT("Key was not set."), Key.IsSet());
			CHECK_MESSAGE(TEXT("Hash is was zero"), Key.CalculateHash() != 0);
			CHECK_MESSAGE(TEXT("Expected string wasn't returned"), Key.ToString() == TEXT("42"));
			CHECK_MESSAGE(TEXT("Self comparison failed"), Key == Key);
		}

		SECTION("Construct key (uint64)")
		{
			uint64 Object = 42;
			FMapKey Key(Object);
			CHECK_MESSAGE(TEXT("Key was not set."), Key.IsSet());
			CHECK_MESSAGE(TEXT("Hash is was zero"), Key.CalculateHash() != 0);
			CHECK_MESSAGE(TEXT("Expected string wasn't returned"), Key.ToString() == TEXT("42"));
			CHECK_MESSAGE(TEXT("Self comparison failed"), Key == Key);
		}

		SECTION("Construct key (float)")
		{
			float Object = 42.0f;
			FMapKey Key(Object);
			CHECK_MESSAGE(TEXT("Key was not set."), Key.IsSet());
			CHECK_MESSAGE(TEXT("Hash is was zero"), Key.CalculateHash() != 0);
			CHECK_MESSAGE(TEXT("Expected string wasn't returned"), Key.ToString() == TEXT("42.0"));
			CHECK_MESSAGE(TEXT("Self comparison failed"), Key == Key);
		}

		SECTION("Construct key (double)")
		{
			double Object = 42.0;
			FMapKey Key(Object);
			CHECK_MESSAGE(TEXT("Key was not set."), Key.IsSet());
			CHECK_MESSAGE(TEXT("Hash is was zero"), Key.CalculateHash() != 0);
			CHECK_MESSAGE(TEXT("Expected string wasn't returned"), Key.ToString() == TEXT("42.0"));
			CHECK_MESSAGE(TEXT("Self comparison failed"), Key == Key);
		}

		SECTION("Construct key (FString)")
		{
			FString Object = "TestString";
			FMapKey Key(Object);
			CHECK_MESSAGE(TEXT("Key was not set."), Key.IsSet());
			CHECK_MESSAGE(TEXT("Hash is was zero"), Key.CalculateHash() != 0);
			CHECK_MESSAGE(TEXT("Expected string wasn't returned"), Key.ToString() == TEXT("TestString"));
			CHECK_MESSAGE(TEXT("Self comparison failed"), Key == Key);
		}

		SECTION("Construct key (FName)")
		{
			FMapKey Key(TestFName);
			CHECK_MESSAGE(TEXT("Key was not set."), Key.IsSet());
			CHECK_MESSAGE(TEXT("Hash is was zero"), Key.CalculateHash() != 0);
			CHECK_MESSAGE(TEXT("Expected string wasn't returned"), Key.ToString() == TEXT("FMapKey_TestName"));
			CHECK_MESSAGE(TEXT("Self comparison failed"), Key == Key);
		}

		SECTION("Construct key (FSoftObjectPath)")
		{
			FMapKey Key(TestPath);
			CHECK_MESSAGE(TEXT("Key was not set."), Key.IsSet());
			CHECK_MESSAGE(TEXT("Hash is was zero"), Key.CalculateHash() != 0);
			CHECK_MESSAGE(TEXT("Expected string wasn't returned"), Key.ToString() == TestPath.ToString());
			CHECK_MESSAGE(TEXT("Self comparison failed"), Key == Key);
		}

		//
		// Copy
		//

		SECTION("Copy empty key")
		{
			FMapKey Key0;
			FMapKey Key1 = Key0;
			CHECK_MESSAGE(TEXT("Original key was not empty as expected."), !Key1.IsSet());
			CHECK_MESSAGE(TEXT("Copied key was not empty as expected."), !Key1.IsSet());
			CHECK_MESSAGE(TEXT("Original and copied key don't have the same value."), Key0 == Key1);
		}

		SECTION("Copy key (void*)")
		{
			int* Object = nullptr;
			FMapKey Key(Object);
			CopyKeyTest(Key);
		}

		SECTION("Copy key (UObject*)")
		{
			UObject* Object = nullptr;
			FMapKey Key(Object);
			CopyKeyTest(Key);
		}

		SECTION("Copy key (int64)")
		{
			int64 Object = 42;
			FMapKey Key(Object);
			CopyKeyTest(Key);
		}

		SECTION("Copy key (uint64)")
		{
			uint64 Object = 42;
			FMapKey Key(Object);
			CopyKeyTest(Key);
		}

		SECTION("Copy key (float)")
		{
			float Object = 42.0f;
			FMapKey Key(Object);
			CopyKeyTest(Key);
		}

		SECTION("Copy key (double)")
		{
			double Object = 42.0;
			FMapKey Key(Object);
			CopyKeyTest(Key);
		}

		SECTION("Copy key (FString)")
		{
			FString Object = "TestString";
			FMapKey Key(Object);
			CopyKeyTest(Key);
		}

		SECTION("Copy key (FName)")
		{
			FMapKey Key(TestFName);
			CopyKeyTest(Key);
		}

		SECTION("Copy key (FSoftObjectPath)")
		{
			FMapKey Key(TestPath);
			CopyKeyTest(Key);
		}

		//
		// Move
		//

		SECTION("Move empty key")
		{
			FMapKey Key0;
			FMapKey Key1 = MoveTemp(Key0);
			CHECK_MESSAGE(TEXT("Original key was not empty as expected."), !Key1.IsSet());
			CHECK_MESSAGE(TEXT("Copied key was not empty as expected."), !Key1.IsSet());
			CHECK_MESSAGE(TEXT("Original and copied key don't have the same value."), Key0 == Key1);
		}

		SECTION("Move key (void*)")
		{
			int* Object = nullptr;
			FMapKey Key(Object);
			MoveKeyTest(MoveTemp(Key));
		}

		SECTION("Move key (UObject*)")
		{
			UObject* Object = nullptr;
			FMapKey Key(Object);
			MoveKeyTest(MoveTemp(Key));
		}

		SECTION("Move key (int64)")
		{
			int64 Object = 42;
			FMapKey Key(Object);
			MoveKeyTest(MoveTemp(Key));
		}

		SECTION("Move key (uint64)")
		{
			uint64 Object = 42;
			FMapKey Key(Object);
			MoveKeyTest(MoveTemp(Key));
		}

		SECTION("Move key (float)")
		{
			float Object = 42.0f;
			FMapKey Key(Object);
			MoveKeyTest(MoveTemp(Key));
		}

		SECTION("Move key (double)")
		{
			double Object = 42.0;
			FMapKey Key(Object);
			MoveKeyTest(MoveTemp(Key));
		}

		SECTION("Move key (FString)")
		{
			FString Object = "TestString";
			FMapKey Key(Object);
			MoveKeyTest(MoveTemp(Key));
		}

		SECTION("Move key (FName)")
		{
			FMapKey Key(TestFName);
			MoveKeyTest(MoveTemp(Key));
		}

		SECTION("Move key (FSoftObjectPath)")
		{
			FMapKey Key(TestPath);
			MoveKeyTest(MoveTemp(Key));
		}
	}

	TEST_CASE_NAMED(FMapKeyView_Tests, "Editor::DataStorage::Map Key View (FMapKeyView)", "[ApplicationContextMask][EngineFilter]")
	{
		using namespace UE::Editor::DataStorage;
		using namespace UE::Editor::DataStorage::Tests::Private;

		//
		// Construction
		//

		SECTION("Empty key")
		{
			FMapKeyView View;
			CHECK_MESSAGE(TEXT("Hash is not zero"), View.CalculateHash() == 0);
			CHECK_MESSAGE(TEXT("Expected string wasn't returned"), View.ToString() == TEXT("Empty"));
			CHECK_MESSAGE(TEXT("Self comparison failed"), View == View);
		}

		SECTION("Construct key (void*)")
		{
			int* Object = reinterpret_cast<int*>(0x1234);
			FMapKeyView View(Object);
			CHECK_MESSAGE(TEXT("Hash is was zero"), View.CalculateHash() != 0);
			FString ViewString = View.ToString();
			CHECK_MESSAGE(TEXT("Expected string wasn't returned"),
				ViewString.StartsWith(TEXT("Pointer(0x"))
				&& ViewString.EndsWith(TEXT("1234)")));
			CHECK_MESSAGE(TEXT("Self comparison failed"), View == View);
		}

		SECTION("Construct key (UObject*)")
		{
			UObject* Object = reinterpret_cast<UObject*>(0x1234);
			FMapKeyView View(Object);
			CHECK_MESSAGE(TEXT("Hash is was zero"), View.CalculateHash() != 0);
			FString ViewString = View.ToString();
			CHECK_MESSAGE(TEXT("Expected string wasn't returned"),
				ViewString.StartsWith(TEXT("UObject(0x"))
				&& ViewString.EndsWith(TEXT("1234)")));
			CHECK_MESSAGE(TEXT("Self comparison failed"), View == View);
		}

		SECTION("Construct key (int64)")
		{
			int64 Object = 42;
			FMapKeyView View(Object);
			CHECK_MESSAGE(TEXT("Hash is was zero"), View.CalculateHash() != 0);
			CHECK_MESSAGE(TEXT("Expected string wasn't returned"), View.ToString() == TEXT("42"));
			CHECK_MESSAGE(TEXT("Self comparison failed"), View == View);
		}

		SECTION("Construct key (uint64)")
		{
			uint64 Object = 42;
			FMapKeyView View(Object);
			CHECK_MESSAGE(TEXT("Hash is was zero"), View.CalculateHash() != 0);
			CHECK_MESSAGE(TEXT("Expected string wasn't returned"), View.ToString() == TEXT("42"));
			CHECK_MESSAGE(TEXT("Self comparison failed"), View == View);
		}

		SECTION("Construct key (float)")
		{
			float Object = 42.0f;
			FMapKeyView View(Object);
			CHECK_MESSAGE(TEXT("Hash is was zero"), View.CalculateHash() != 0);
			CHECK_MESSAGE(TEXT("Expected string wasn't returned"), View.ToString() == TEXT("42.0"));
			CHECK_MESSAGE(TEXT("Self comparison failed"), View == View);
		}

		SECTION("Construct key (double)")
		{
			double Object = 42.0;
			FMapKeyView View(Object);
			CHECK_MESSAGE(TEXT("Hash is was zero"), View.CalculateHash() != 0);
			CHECK_MESSAGE(TEXT("Expected string wasn't returned"), View.ToString() == TEXT("42.0"));
			CHECK_MESSAGE(TEXT("Self comparison failed"), View == View);
		}

		SECTION("Construct key (FString)")
		{
			FString Object = "TestString";
			FMapKeyView View(Object);
			CHECK_MESSAGE(TEXT("Hash is was zero"), View.CalculateHash() != 0);
			CHECK_MESSAGE(TEXT("Expected string wasn't returned"), View.ToString() == TEXT("TestString"));
			CHECK_MESSAGE(TEXT("Self comparison failed"), View == View);
		}

		SECTION("Construct key (FStringView)")
		{
			FString Object = "TestString";
			FMapKeyView View = FMapKeyView(FStringView(Object));
			CHECK_MESSAGE(TEXT("Hash is was zero"), View.CalculateHash() != 0);
			CHECK_MESSAGE(TEXT("Expected string wasn't returned"), View.ToString() == TEXT("TestString"));
			CHECK_MESSAGE(TEXT("Self comparison failed"), View == View);
		}

		SECTION("Construct key (FName)")
		{
			FMapKeyView View(TestFName);
			CHECK_MESSAGE(TEXT("Hash is was zero"), View.CalculateHash() != 0);
			CHECK_MESSAGE(TEXT("Expected string wasn't returned"), View.ToString() == TEXT("FMapKey_TestName"));
			CHECK_MESSAGE(TEXT("Self comparison failed"), View == View);
		}

		SECTION("Construct key (FSoftObjectPath)")
		{
			FMapKeyView View(TestPath);
			CHECK_MESSAGE(TEXT("Hash is was zero"), View.CalculateHash() != 0);
			CHECK_MESSAGE(TEXT("Expected string wasn't returned"), View.ToString() == TestPath.ToString());
			CHECK_MESSAGE(TEXT("Self comparison failed"), View == View);
		}

		//
		// Conversion
		//

		SECTION("Convert key (Empty)")
		{
			FMapKey Key;
			ConvertKeyToKeyViewTest(Key);
		}

		SECTION("Convert key (void*)")
		{
			int* Object = reinterpret_cast<int*>(0x1234);
			FMapKey Key(Object);
			ConvertKeyToKeyViewTest(Key);
		}

		SECTION("Convert key (UObject*)")
		{
			UObject* Object = reinterpret_cast<UObject*>(0x1234);
			FMapKey Key(Object);
			ConvertKeyToKeyViewTest(Key);
		}

		SECTION("Convert key (int64)")
		{
			int64 Object = 42;
			FMapKey Key(Object);
			ConvertKeyToKeyViewTest(Key);
		}

		SECTION("Convert key (uint64)")
		{
			uint64 Object = 42;
			FMapKey Key(Object);
			ConvertKeyToKeyViewTest(Key);
		}

		SECTION("Convert key (float)")
		{
			float Object = 42.0f;
			FMapKey Key(Object);
			ConvertKeyToKeyViewTest(Key);
		}

		SECTION("Convert key (double)")
		{
			double Object = 42.0;
			FMapKey Key(Object);
			ConvertKeyToKeyViewTest(Key);
		}

		SECTION("Convert key (FString)")
		{
			FString Object = "TestString";
			FMapKey Key(Object);
			ConvertKeyToKeyViewTest(Key);
		}

		SECTION("Convert key (FName)")
		{
			FMapKey Key(TestFName);
			ConvertKeyToKeyViewTest(Key);
		}

		SECTION("Convert key (FSoftObjectPath)")
		{
			FMapKey Key(TestPath);
			ConvertKeyToKeyViewTest(Key);
		}
	}
}
// namespace UE::Editor::DataStorage::Tests::Private

#endif // WITH_TESTS
