// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataStorage/Scope/EditorDataScope.h"
#include "DataStorage/Scope/EditorDataScopeColumns.h"
#include "Elements/Framework/TypedElementTestColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#if WITH_TESTS

#include "DataStorage/Features.h"
#include "Misc/AutomationTest.h"

namespace UE::Editor::DataStorage::Scope::Tests
{

BEGIN_DEFINE_SPEC(FEditorDataScopeTestFixture,
	"Editor.DataStorage.Scope",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	ICoreProvider* Storage = nullptr;
	TArray<RowHandle> CreatedRows;

	RowHandle CreateRow()
	{
		RowHandle Row = Storage->AddScopeRow();
		CreatedRows.Add(Row);
		return Row;
	}

	void SetupParentChild(RowHandle Parent, RowHandle Child)
	{
		Storage->SetParentScope(Child, Parent);
	}

END_DEFINE_SPEC(FEditorDataScopeTestFixture)

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
void FEditorDataScopeTestFixture::Define()
{
	BeforeEach([this]()
	{
		Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		TestTrue(TEXT("TEDS ICoreProvider must be available"), Storage != nullptr);
		if (!Storage) return;

		// Reset TLS context
		SetCurrentScope(InvalidRowHandle);
	});

	AfterEach([this]()
	{
		if (Storage)
		{
			for (RowHandle Row : CreatedRows)
			{
				Storage->RemoveScopeRow(Row);
			}
		}
		CreatedRows.Empty();
		SetCurrentScope(InvalidRowHandle);
	});

	// ========================================================================
	// TLS Tests
	// ========================================================================

	Describe("TLS", [this]
	{
		It(TEXT("DefaultIsInvalid"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			TestEqual(TEXT("Default current context should be InvalidRowHandle"),
				GetCurrentScope(), InvalidRowHandle);
		});

		It(TEXT("SetAndGet"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = CreateRow();
			SetCurrentScope(Row);
			TestEqual(TEXT("GetCurrentScope should return the row we set"),
				GetCurrentScope(), Row);
			SetCurrentScope(InvalidRowHandle);
		});

		It(TEXT("ScopedPushRestore"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle RowA = CreateRow();
			RowHandle RowB = CreateRow();

			SetCurrentScope(RowA);
			{
				FPushScopeGuard Guard(RowB);
				TestEqual(TEXT("Inside scope: context should be RowB"),
					GetCurrentScope(), RowB);
			}
			TestEqual(TEXT("After scope: context should be restored to RowA"),
				GetCurrentScope(), RowA);
			SetCurrentScope(InvalidRowHandle);
		});

		It(TEXT("NestedScoped"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle RowA = CreateRow();
			RowHandle RowB = CreateRow();
			RowHandle RowC = CreateRow();

			SetCurrentScope(RowA);
			{
				FPushScopeGuard Guard1(RowB);
				TestEqual(TEXT("Level 1: context should be RowB"), GetCurrentScope(), RowB);
				{
					FPushScopeGuard Guard2(RowC);
					TestEqual(TEXT("Level 2: context should be RowC"), GetCurrentScope(), RowC);
				}
				TestEqual(TEXT("Back to level 1: context should be RowB"), GetCurrentScope(), RowB);
			}
			TestEqual(TEXT("Back to original: context should be RowA"), GetCurrentScope(), RowA);
			SetCurrentScope(InvalidRowHandle);
		});
	});

	// ========================================================================
	// Hierarchy Tests
	// ========================================================================

	Describe("Hierarchy", [this]
	{
		It(TEXT("AddScopeRow"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = Storage->AddScopeRow();
			CreatedRows.Add(Row);
			TestTrue(TEXT("AddScopeRow should return valid handle"),
				Row != InvalidRowHandle);
		});

		It(TEXT("ParentChild"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Parent = CreateRow();
			RowHandle Child = CreateRow();
			Storage->SetParentScope(Child, Parent);

			RowHandle FoundParent = Storage->GetParentScope(Child);
			TestEqual(TEXT("GetParentScope should return the parent"),
				FoundParent, Parent);
		});
	});

	// ========================================================================
	// Scope Data Lookup Tests
	// ========================================================================

	Describe("Lookup", [this]
	{
		It(TEXT("DirectColumn"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = CreateRow();
			Storage->SetScopeData<FTestColumnInt>(Row, FTestColumnInt{.TestInt = 42});

			const FTestColumnInt* Data = Storage->GetScopeData<FTestColumnInt>(Row);
			TestNotNull(TEXT("Should find column on same row"), Data);
			if (Data)
			{
				TestEqual(TEXT("Column value should match"), Data->TestInt, 42);
			}
		});

		It(TEXT("InheritedColumn"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Parent = CreateRow();
			RowHandle Child = CreateRow();
			SetupParentChild(Parent, Child);

			Storage->SetScopeData<FTestColumnInt>(Parent, FTestColumnInt{.TestInt = 100});

			const FTestColumnInt* Data = Storage->GetScopeData<FTestColumnInt>(Child);
			TestNotNull(TEXT("Should find inherited column from parent"), Data);
			if (Data)
			{
				TestEqual(TEXT("Inherited value should match parent"), Data->TestInt, 100);
			}
		});

		It(TEXT("OverrideColumn"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Parent = CreateRow();
			RowHandle Child = CreateRow();
			SetupParentChild(Parent, Child);

			Storage->SetScopeData<FTestColumnInt>(Parent, FTestColumnInt{.TestInt = 100});
			Storage->SetScopeData<FTestColumnInt>(Child, FTestColumnInt{.TestInt = 200});

			const FTestColumnInt* Data = Storage->GetScopeData<FTestColumnInt>(Child);
			TestNotNull(TEXT("Should find overridden column"), Data);
			if (Data)
			{
				TestEqual(TEXT("Should return child value, not parent"), Data->TestInt, 200);
			}
		});

		It(TEXT("MissingColumn"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = CreateRow();
			const FTestColumnString* Data = Storage->GetScopeData<FTestColumnString>(Row);
			TestNull(TEXT("Missing column should return nullptr"), Data);
		});

		It(TEXT("UsesCurrentScope"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = CreateRow();
			Storage->SetScopeData<FTestColumnInt>(Row, FTestColumnInt{.TestInt = 55});

			SetCurrentScope(Row);
			const FTestColumnInt* Data = GetScopeData<FTestColumnInt>(*Storage);
			TestNotNull(TEXT("Should find data via current context"), Data);
			if (Data)
			{
				TestEqual(TEXT("Value should match"), Data->TestInt, 55);
			}
			SetCurrentScope(InvalidRowHandle);
		});

		It(TEXT("ThreeLevelInheritance"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Grandparent = CreateRow();
			RowHandle Parent = CreateRow();
			RowHandle Child = CreateRow();
			SetupParentChild(Grandparent, Parent);
			SetupParentChild(Parent, Child);

			Storage->SetScopeData<FTestColumnInt>(Grandparent, FTestColumnInt{.TestInt = 999});

			const FTestColumnInt* Data = Storage->GetScopeData<FTestColumnInt>(Child);
			TestNotNull(TEXT("Should find column from grandparent"), Data);
			if (Data)
			{
				TestEqual(TEXT("Should be grandparent value"), Data->TestInt, 999);
			}
		});
	});

	// ========================================================================
	// Versioning Tests
	// ========================================================================

	Describe("Version", [this]
	{
		It(TEXT("SetReturnsValid"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = CreateRow();
			FScopeDataVersion V = Storage->SetScopeData<FTestColumnInt>(Row, FTestColumnInt{.TestInt = 1});
			TestTrue(TEXT("Version from SetScopeData should be valid"), V.IsValid());
		});

		It(TEXT("SetIncrements"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = CreateRow();
			FScopeDataVersion V1 = Storage->SetScopeData<FTestColumnInt>(Row, FTestColumnInt{.TestInt = 1});
			FScopeDataVersion V2 = Storage->SetScopeData<FTestColumnInt>(Row, FTestColumnInt{.TestInt = 2});
			TestTrue(TEXT("Two updates should return different versions"), V1 != V2);
		});

		It(TEXT("ReadVersionMatches"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = CreateRow();
			FScopeDataVersion V1 = Storage->SetScopeData<FTestColumnInt>(Row, FTestColumnInt{.TestInt = 1});
			FScopeDataVersion V2 = Storage->GetScopeDataVersion<FTestColumnInt>(Row);
			TestTrue(TEXT("GetScopeDataVersion should match last update"), V1 == V2);
		});

		It(TEXT("DeleteReturnsTrue"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = CreateRow();
			Storage->SetScopeData<FTestColumnInt>(Row, FTestColumnInt{.TestInt = 1});
			bool Deleted = Storage->RemoveScopeData<FTestColumnInt>(Row);
			TestTrue(TEXT("RemoveScopeData should return true when column existed"), Deleted);
		});

		It(TEXT("DeleteReturnsFalse"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = CreateRow();
			bool Deleted = Storage->RemoveScopeData<FTestColumnInt>(Row);
			TestFalse(TEXT("RemoveScopeData should return false when column absent"), Deleted);
		});

		It(TEXT("DeleteClearsVersion"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = CreateRow();
			Storage->SetScopeData<FTestColumnInt>(Row, FTestColumnInt{.TestInt = 1});
			Storage->RemoveScopeData<FTestColumnInt>(Row);

			FScopeDataVersion V = Storage->GetScopeDataVersion<FTestColumnInt>(Row);
			TestFalse(TEXT("Version should be invalid after delete (column gone, walk finds nothing)"), V.IsValid());
		});

		It(TEXT("DeleteVersionDiffers"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = CreateRow();
			FScopeDataVersion VUpdate = Storage->SetScopeData<FTestColumnInt>(Row, FTestColumnInt{.TestInt = 1});
			Storage->RemoveScopeData<FTestColumnInt>(Row);
			FScopeDataVersion VDelete = Storage->GetScopeDataVersion<FTestColumnInt>(Row);

			TestTrue(TEXT("Version after delete should differ from update version"), VUpdate != VDelete);
		});

		It(TEXT("SetAfterDelete"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = CreateRow();
			FScopeDataVersion V1 = Storage->SetScopeData<FTestColumnInt>(Row, FTestColumnInt{.TestInt = 1});
			Storage->RemoveScopeData<FTestColumnInt>(Row);
			FScopeDataVersion V2 = Storage->GetScopeDataVersion<FTestColumnInt>(Row);
			FScopeDataVersion V3 = Storage->SetScopeData<FTestColumnInt>(Row, FTestColumnInt{.TestInt = 2});

			TestTrue(TEXT("V1 != V2 (update vs delete)"), V1 != V2);
			TestTrue(TEXT("V2 != V3 (delete vs re-update)"), V2 != V3);
			TestTrue(TEXT("V3 should be valid"), V3.IsValid());

			// Verify the data is actually back
			const FTestColumnInt* Data = Storage->GetScopeData<FTestColumnInt>(Row);
			TestNotNull(TEXT("Column should exist after re-update"), Data);
			if (Data)
			{
				TestEqual(TEXT("Re-updated value should match"), Data->TestInt, 2);
			}
		});

		It(TEXT("NeverSetIsInvalid"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = CreateRow();
			FScopeDataVersion V = Storage->GetScopeDataVersion<FTestColumnInt>(Row);
			TestFalse(TEXT("Version for never-set column should be invalid"), V.IsValid());
		});
	});

	// ========================================================================
	// Hierarchy + Versioning Integration Tests
	// ========================================================================

	Describe("HierVersion", [this]
	{
		It(TEXT("InheritedVersion"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Parent = CreateRow();
			RowHandle Child = CreateRow();
			SetupParentChild(Parent, Child);

			FScopeDataVersion VParent = Storage->SetScopeData<FTestColumnInt>(Parent, FTestColumnInt{.TestInt = 10});
			FScopeDataVersion VFromChild = Storage->GetScopeDataVersion<FTestColumnInt>(Child);

			TestTrue(TEXT("Hierarchy version from child should be valid"), VFromChild.IsValid());
			TestTrue(TEXT("Hierarchy version should match parent's version"), VParent == VFromChild);
		});

		It(TEXT("OverriddenVersion"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Parent = CreateRow();
			RowHandle Child = CreateRow();
			SetupParentChild(Parent, Child);

			Storage->SetScopeData<FTestColumnInt>(Parent, FTestColumnInt{.TestInt = 10});
			FScopeDataVersion VChild = Storage->SetScopeData<FTestColumnInt>(Child, FTestColumnInt{.TestInt = 20});

			FScopeDataVersion VFromChild = Storage->GetScopeDataVersion<FTestColumnInt>(Child);
			TestTrue(TEXT("Should get child's version, not parent's"), VChild == VFromChild);
		});

		It(TEXT("DeleteOnChildFallsToParent"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Parent = CreateRow();
			RowHandle Child = CreateRow();
			SetupParentChild(Parent, Child);

			FScopeDataVersion VParent = Storage->SetScopeData<FTestColumnInt>(Parent, FTestColumnInt{.TestInt = 10});
			Storage->SetScopeData<FTestColumnInt>(Child, FTestColumnInt{.TestInt = 20});
			Storage->RemoveScopeData<FTestColumnInt>(Child);

			// The child's version map entry was removed on delete, so the hierarchy walk
			// falls through to the parent.
			FScopeDataVersion VFromChild = Storage->GetScopeDataVersion<FTestColumnInt>(Child);
			TestTrue(TEXT("After delete, hierarchy version should still be valid"), VFromChild.IsValid());

			// The actual data lookup should now fall through to parent
			const FTestColumnInt* Data = Storage->GetScopeData<FTestColumnInt>(Child);
			TestNotNull(TEXT("GetScopeData should fall through to parent"), Data);
			if (Data)
			{
				TestEqual(TEXT("Data should come from parent"), Data->TestInt, 10);
			}
		});
	});

	// ========================================================================
	// Integration Tests
	// ========================================================================

	Describe("Integration", [this]
	{
		It(TEXT("FullWorkflow"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			// Build 3-level hierarchy: Root -> Mid -> Leaf
			RowHandle Root = CreateRow();
			RowHandle Mid = CreateRow();
			RowHandle Leaf = CreateRow();
			SetupParentChild(Root, Mid);
			SetupParentChild(Mid, Leaf);

			// Publish on root
			FScopeDataVersion V1 = Storage->SetScopeData<FTestColumnInt>(Root, FTestColumnInt{.TestInt = 42});
			TestTrue(TEXT("V1 should be valid"), V1.IsValid());

			// Read from leaf
			const FTestColumnInt* Data = Storage->GetScopeData<FTestColumnInt>(Leaf);
			TestNotNull(TEXT("Leaf should inherit from root"), Data);
			if (Data)
			{
				TestEqual(TEXT("Inherited value should be 42"), Data->TestInt, 42);
			}

			// Detect version change on update
			FScopeDataVersion V2 = Storage->SetScopeData<FTestColumnInt>(Root, FTestColumnInt{.TestInt = 99});
			TestTrue(TEXT("Version should change after update"), V1 != V2);

			// Verify new value propagates
			Data = Storage->GetScopeData<FTestColumnInt>(Leaf);
			TestNotNull(TEXT("Leaf should still inherit"), Data);
			if (Data)
			{
				TestEqual(TEXT("Updated value should be 99"), Data->TestInt, 99);
			}
		});

		It(TEXT("ScopedCallStack"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle RowA = CreateRow();
			RowHandle RowB = CreateRow();
			Storage->SetScopeData<FTestColumnInt>(RowA, FTestColumnInt{.TestInt = 1});
			Storage->SetScopeData<FTestColumnInt>(RowB, FTestColumnInt{.TestInt = 2});

			{
				FPushScopeGuard GuardA(RowA);
				const FTestColumnInt* DataA = GetScopeData<FTestColumnInt>(*Storage);
				TestNotNull(TEXT("Level A: should find data"), DataA);
				if (DataA)
				{
					TestEqual(TEXT("Level A value"), DataA->TestInt, 1);
				}

				{
					FPushScopeGuard GuardB(RowB);
					const FTestColumnInt* DataB = GetScopeData<FTestColumnInt>(*Storage);
					TestNotNull(TEXT("Level B: should find data"), DataB);
					if (DataB)
					{
						TestEqual(TEXT("Level B value"), DataB->TestInt, 2);
					}
				}

				const FTestColumnInt* DataAfter = GetScopeData<FTestColumnInt>(*Storage);
				TestNotNull(TEXT("After B scope: should find A's data"), DataAfter);
				if (DataAfter)
				{
					TestEqual(TEXT("Should be A's value again"), DataAfter->TestInt, 1);
				}
			}
		});

		It(TEXT("MultipleColumnTypes"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Parent = CreateRow();
			RowHandle Child = CreateRow();
			SetupParentChild(Parent, Child);

			Storage->SetScopeData<FTestColumnInt>(Parent, FTestColumnInt{.TestInt = 10});
			FTestColumnString StrColumn;
			StrColumn.TestString = TEXT("hello");
			Storage->SetScopeData<FTestColumnString>(Child, MoveTemp(StrColumn));

			const FTestColumnInt* IntData = Storage->GetScopeData<FTestColumnInt>(Child);
			const FTestColumnString* StrData = Storage->GetScopeData<FTestColumnString>(Child);

			TestNotNull(TEXT("Int from parent should be found"), IntData);
			TestNotNull(TEXT("String from child should be found"), StrData);
			if (IntData)
			{
				TestEqual(TEXT("Int value from parent"), IntData->TestInt, 10);
			}
			if (StrData)
			{
				TestEqual(TEXT("String value from child"), StrData->TestString, FString(TEXT("hello")));
			}
		});

		It(TEXT("DeleteAndReAdd"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = CreateRow();
			FScopeDataVersion V1 = Storage->SetScopeData<FTestColumnInt>(Row, FTestColumnInt{.TestInt = 1});
			bool Deleted = Storage->RemoveScopeData<FTestColumnInt>(Row);
			TestTrue(TEXT("Delete should succeed"), Deleted);
			FScopeDataVersion V2 = Storage->GetScopeDataVersion<FTestColumnInt>(Row);
			FScopeDataVersion V3 = Storage->SetScopeData<FTestColumnInt>(Row, FTestColumnInt{.TestInt = 3});

			TestTrue(TEXT("V1 != V2"), V1 != V2);
			TestTrue(TEXT("V2 != V3"), V2 != V3);
			TestTrue(TEXT("V1 != V3"), V1 != V3);
		});
	});

	// ========================================================================
	// FEditingVerseScope Tests
	// ========================================================================

	Describe("EditingVerseScope", [this]
	{
		It(TEXT("SetAndReadOnSameRow"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = CreateRow();
			FEditingVerseScope Scope;
			Scope.EditingScopeVersePath = TEXT("/TestDomain/TestScope");
			Storage->SetScopeData<FEditingVerseScope>(Row, MoveTemp(Scope));

			const FEditingVerseScope* Data = Storage->GetScopeData<FEditingVerseScope>(Row);
			TestNotNull(TEXT("Should find FEditingVerseScope on same row"), Data);
			if (Data)
			{
				TestEqual(TEXT("VersePath should match"), Data->EditingScopeVersePath, FString(TEXT("/TestDomain/TestScope")));
			}
		});

		It(TEXT("InheritedFromParent"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Parent = CreateRow();
			RowHandle Child = CreateRow();
			SetupParentChild(Parent, Child);

			FEditingVerseScope Scope;
			Scope.EditingScopeVersePath = TEXT("/Game/Maps/TestLevel");
			Storage->SetScopeData<FEditingVerseScope>(Parent, MoveTemp(Scope));

			const FEditingVerseScope* Data = Storage->GetScopeData<FEditingVerseScope>(Child);
			TestNotNull(TEXT("Child should inherit FEditingVerseScope from parent"), Data);
			if (Data)
			{
				TestEqual(TEXT("Inherited path should match parent"),
					Data->EditingScopeVersePath, FString(TEXT("/Game/Maps/TestLevel")));
			}
		});

		It(TEXT("ChildOverridesParent"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Root = CreateRow();
			RowHandle Child = CreateRow();
			SetupParentChild(Root, Child);

			FEditingVerseScope RootScope;
			RootScope.EditingScopeVersePath = TEXT("/Game/Maps/TestLevel");
			Storage->SetScopeData<FEditingVerseScope>(Root, MoveTemp(RootScope));

			FEditingVerseScope ChildScope;
			ChildScope.EditingScopeVersePath = TEXT("/Game/Prefabs/TestPrefab");
			Storage->SetScopeData<FEditingVerseScope>(Child, MoveTemp(ChildScope));

			const FEditingVerseScope* Data = Storage->GetScopeData<FEditingVerseScope>(Child);
			TestNotNull(TEXT("Should find overridden FEditingVerseScope"), Data);
			if (Data)
			{
				TestEqual(TEXT("Should return child scope, not parent"),
					Data->EditingScopeVersePath, FString(TEXT("/Game/Prefabs/TestPrefab")));
			}

			// Parent should still have its own scope
			const FEditingVerseScope* ParentData = Storage->GetScopeData<FEditingVerseScope>(Root);
			TestNotNull(TEXT("Parent should still have its scope"), ParentData);
			if (ParentData)
			{
				TestEqual(TEXT("Parent scope should be unchanged"),
					ParentData->EditingScopeVersePath, FString(TEXT("/Game/Maps/TestLevel")));
			}
		});

		It(TEXT("ReadViaTLS"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = CreateRow();
			FEditingVerseScope Scope;
			Scope.EditingScopeVersePath = TEXT("/TLS/TestScope");
			Storage->SetScopeData<FEditingVerseScope>(Row, MoveTemp(Scope));

			FPushScopeGuard Guard(Row);
			const FEditingVerseScope* Data = GetScopeData<FEditingVerseScope>(*Storage);
			TestNotNull(TEXT("Should find FEditingVerseScope via TLS current context"), Data);
			if (Data)
			{
				TestEqual(TEXT("TLS-based read should return correct path"),
					Data->EditingScopeVersePath, FString(TEXT("/TLS/TestScope")));
			}
		});

		It(TEXT("ThreeLevelHierarchy"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			// Simulate Root -> LevelEditor -> PrefabEditor hierarchy
			RowHandle Root = CreateRow();
			RowHandle LevelEditor = CreateRow();
			RowHandle PrefabEditor = CreateRow();
			SetupParentChild(Root, LevelEditor);
			SetupParentChild(LevelEditor, PrefabEditor);

			FEditingVerseScope LevelScope;
			LevelScope.EditingScopeVersePath = TEXT("/Game/Maps/MyLevel");
			Storage->SetScopeData<FEditingVerseScope>(LevelEditor, MoveTemp(LevelScope));

			FEditingVerseScope PrefabScope;
			PrefabScope.EditingScopeVersePath = TEXT("/Game/Prefabs/MyPrefab");
			Storage->SetScopeData<FEditingVerseScope>(PrefabEditor, MoveTemp(PrefabScope));

			// PrefabEditor should see its own scope
			const FEditingVerseScope* PrefabData = Storage->GetScopeData<FEditingVerseScope>(PrefabEditor);
			TestNotNull(TEXT("PrefabEditor should have its own scope"), PrefabData);
			if (PrefabData)
			{
				TestEqual(TEXT("PrefabEditor scope"), PrefabData->EditingScopeVersePath, FString(TEXT("/Game/Prefabs/MyPrefab")));
			}

			// LevelEditor should see its own scope
			const FEditingVerseScope* LevelData = Storage->GetScopeData<FEditingVerseScope>(LevelEditor);
			TestNotNull(TEXT("LevelEditor should have its own scope"), LevelData);
			if (LevelData)
			{
				TestEqual(TEXT("LevelEditor scope"), LevelData->EditingScopeVersePath, FString(TEXT("/Game/Maps/MyLevel")));
			}

			// Root has no scope set -- should return nullptr
			const FEditingVerseScope* RootData = Storage->GetScopeData<FEditingVerseScope>(Root);
			TestNull(TEXT("Root should have no FEditingVerseScope"), RootData);
		});
	});

	// ========================================================================
	// Edge Cases
	// ========================================================================

	Describe("Edge", [this]
	{
		It(TEXT("RowNotInHierarchy"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = CreateRow();
			Storage->SetScopeData<FTestColumnInt>(Row, FTestColumnInt{.TestInt = 77});

			// Row is not part of the scope hierarchy, should still check its own columns
			const FTestColumnInt* Data = Storage->GetScopeData<FTestColumnInt>(Row);
			TestNotNull(TEXT("Should find data on the row itself"), Data);
			if (Data)
			{
				TestEqual(TEXT("Value should match"), Data->TestInt, 77);
			}
		});

		It(TEXT("EmptyHierarchy"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = CreateRow();
			// No parent, no data
			const FTestColumnInt* Data = Storage->GetScopeData<FTestColumnInt>(Row);
			TestNull(TEXT("No data on isolated row should return nullptr"), Data);
		});
	});

	// ========================================================================
	// InvalidRowHandle Robustness
	// ========================================================================

	Describe("InvalidRow", [this]
	{
		It(TEXT("GetScopeData"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			const FTestColumnInt* Data = Storage->GetScopeData<FTestColumnInt>(InvalidRowHandle);
			TestNull(TEXT("GetScopeData on InvalidRowHandle should return nullptr"), Data);
		});

		It(TEXT("GetScopeDataVersion"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			FScopeDataVersion V = Storage->GetScopeDataVersion<FTestColumnInt>(InvalidRowHandle);
			TestFalse(TEXT("GetScopeDataVersion on InvalidRowHandle should return invalid version"), V.IsValid());
		});

		It(TEXT("SetScopeData"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			FScopeDataVersion V = Storage->SetScopeData<FTestColumnInt>(InvalidRowHandle, FTestColumnInt{.TestInt = 1});
			TestFalse(TEXT("SetScopeData on InvalidRowHandle should return invalid version"), V.IsValid());
		});

		It(TEXT("RemoveScopeData"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			bool Deleted = Storage->RemoveScopeData<FTestColumnInt>(InvalidRowHandle);
			TestFalse(TEXT("RemoveScopeData on InvalidRowHandle should return false"), Deleted);
		});

		It(TEXT("GetParentScope"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Parent = Storage->GetParentScope(InvalidRowHandle);
			TestEqual(TEXT("GetParentScope on InvalidRowHandle should return InvalidRowHandle"),
				Parent, InvalidRowHandle);
		});

		It(TEXT("SetParentScope"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			// Should not crash
			Storage->SetParentScope(InvalidRowHandle, InvalidRowHandle);
		});

		It(TEXT("RemoveScopeRow"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			// Should not crash
			Storage->RemoveScopeRow(InvalidRowHandle);
		});

		It(TEXT("GetAllVisibleScopeColumns"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			TArray<const UScriptStruct*> Columns = Storage->GetAllVisibleScopeColumns(InvalidRowHandle);
			TestEqual(TEXT("GetAllVisibleScopeColumns on InvalidRowHandle should return empty"),
				Columns.Num(), 0);
		});

		It(TEXT("ReadViaTLSWhenUnset"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			SetCurrentScope(InvalidRowHandle);
			const FTestColumnInt* Data = GetScopeData<FTestColumnInt>(*Storage);
			TestNull(TEXT("GetScopeData via unset TLS should return nullptr"), Data);
		});
	});

	// ========================================================================
	// Destroyed Row Robustness
	// ========================================================================

	Describe("DestroyedRow", [this]
	{
		It(TEXT("GetScopeData"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = Storage->AddScopeRow();
			Storage->SetScopeData<FTestColumnInt>(Row, FTestColumnInt{.TestInt = 42});
			Storage->RemoveScopeRow(Row);

			const FTestColumnInt* Data = Storage->GetScopeData<FTestColumnInt>(Row);
			TestNull(TEXT("GetScopeData on destroyed row should return nullptr"), Data);
		});

		It(TEXT("GetScopeDataVersion"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = Storage->AddScopeRow();
			Storage->SetScopeData<FTestColumnInt>(Row, FTestColumnInt{.TestInt = 1});
			Storage->RemoveScopeRow(Row);

			FScopeDataVersion V = Storage->GetScopeDataVersion<FTestColumnInt>(Row);
			TestFalse(TEXT("GetScopeDataVersion on destroyed row should return invalid version"), V.IsValid());
		});

		It(TEXT("SetScopeData"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = Storage->AddScopeRow();
			Storage->RemoveScopeRow(Row);

			FScopeDataVersion V = Storage->SetScopeData<FTestColumnInt>(Row, FTestColumnInt{.TestInt = 1});
			TestFalse(TEXT("SetScopeData on destroyed row should return invalid version"), V.IsValid());
		});

		It(TEXT("RemoveScopeData"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = Storage->AddScopeRow();
			Storage->SetScopeData<FTestColumnInt>(Row, FTestColumnInt{.TestInt = 1});
			Storage->RemoveScopeRow(Row);

			bool Deleted = Storage->RemoveScopeData<FTestColumnInt>(Row);
			TestFalse(TEXT("RemoveScopeData on destroyed row should return false"), Deleted);
		});

		It(TEXT("GetParentScope"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = Storage->AddScopeRow();
			Storage->RemoveScopeRow(Row);

			RowHandle Parent = Storage->GetParentScope(Row);
			TestEqual(TEXT("GetParentScope on destroyed row should return InvalidRowHandle"),
				Parent, InvalidRowHandle);
		});

		It(TEXT("SetParentScope"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = Storage->AddScopeRow();
			RowHandle Other = CreateRow();
			Storage->RemoveScopeRow(Row);

			// Should not crash
			Storage->SetParentScope(Row, Other);
		});

		It(TEXT("RemoveScopeRowTwice"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = Storage->AddScopeRow();
			Storage->RemoveScopeRow(Row);

			// Should not crash
			Storage->RemoveScopeRow(Row);
		});

		It(TEXT("GetAllVisibleScopeColumns"), EAsyncExecution::TaskGraphMainTick, [this]()
		{
			RowHandle Row = Storage->AddScopeRow();
			Storage->SetScopeData<FTestColumnInt>(Row, FTestColumnInt{.TestInt = 1});
			Storage->RemoveScopeRow(Row);

			TArray<const UScriptStruct*> Columns = Storage->GetAllVisibleScopeColumns(Row);
			TestEqual(TEXT("GetAllVisibleScopeColumns on destroyed row should return empty"),
				Columns.Num(), 0);
		});
	});
}

} // namespace UE::Editor::DataStorage::Scope::Tests

PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

#endif // WITH_TESTS
