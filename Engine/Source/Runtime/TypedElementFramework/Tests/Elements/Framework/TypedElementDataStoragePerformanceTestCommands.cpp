// Copyright Epic Games, Inc. All Rights Reserved.

#include "TypedElementDataStoragePerformanceTestCommands.h"

#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementDataStoragePerformanceTestCommands)

namespace UE::Editor::DataStorage
{
	namespace Private
	{
		TableHandle PerformanceTestCommandTable = InvalidTableHandle;
	} // namespace Private

	FAutoConsoleCommand CVarAddDebugRows = FAutoConsoleCommand(
		TEXT("Teds.Debug.PerformanceTest.AddRows"),
		TEXT("Teds.Debug.PerformanceTest.AddRows <NumRows>;  NumRows = number of rows to add"),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			if (Args.Num() > 0)
			{
				int32 EntitiesToAdd;
				LexFromString(EntitiesToAdd, *Args[0]);

				ICoreProvider* DataStorageInterface = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
			
				DataStorageInterface->BatchAddRow(Private::PerformanceTestCommandTable, EntitiesToAdd, [DataStorageInterface](RowHandle Row)
					{
						FTest_PingPongPrePhys* Column = DataStorageInterface->GetColumn<FTest_PingPongPrePhys>(Row);
						Column->Value = 0;
					});
			}
		}
	));

	FAutoConsoleCommand CVarResetDebugEntities = FAutoConsoleCommand(
		TEXT("Teds.Debug.PerformanceTest.RemoveAllRows"),
		TEXT("Removes all added rows for the performance test"),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			using namespace UE::Editor::DataStorage::Queries;

			ICoreProvider* DataStorageInterface = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		
			QueryHandle Query = DataStorageInterface->RegisterQuery(
				Select().Where().All<FTest_PingPongPrePhys>().Compile());

			TArray<RowHandle> RowsToDelete;

			DataStorageInterface->RunQuery(Query, CreateDirectQueryCallbackBinding([&RowsToDelete](const IDirectQueryContext& Context, const RowHandle* RowArray)
			{
				RowsToDelete.Insert(RowArray, Context.GetRowCount(), RowsToDelete.Num());
			}));

			for (RowHandle Row : RowsToDelete)
			{
				DataStorageInterface->RemoveRow(Row);
			}

			DataStorageInterface->UnregisterQuery(Query);
		}
	));
} // namespace UE::Editor::DataStorage

void UTest_PingPongBetweenPhaseFactory::RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	UE::Editor::DataStorage::Private::PerformanceTestCommandTable = DataStorage.RegisterTable({FTest_PingPongPrePhys::StaticStruct()}, TEXT("Test_PingPongPrePhys"));
}

// Performance test is a small benchmark to better characterize the performance of adding and removing columns during the processor phases
// The test comprises of three processors, A, B and C.  The run in consecutive phases: A in PrePhys, B in DuringPhys and C in PostPhys
// A is sensitive to tag PingPongPrePhys.  It adds a PingPongDurPhys tag and removes the PingPonPrePhys tag.  This causes the processed row
// to then be processed by processor B.  Processor B does a similar thing to ensure it is processed by C.  C then ensures it is processed by A
// the next time A is run.
void UTest_PingPongBetweenPhaseFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;

	Super::RegisterQueries(DataStorage);

	DataStorage.RegisterQuery(
		Select(TEXT("PingPong PrePhysics->DurPhysics"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::Default))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, const RowHandle* RowPtr, FTest_PingPongPrePhys* PingPongAPtr)
			{
				QUICK_SCOPE_CYCLE_COUNTER(PingPong_Pre_During);
				TArrayView<const RowHandle> Rows(RowPtr, Context.GetRowCount());
				TArrayView<FTest_PingPongPrePhys> PingPongAs(PingPongAPtr, Context.GetRowCount());

				const UScriptStruct* RemovedColumnStruct = FTest_PingPongPrePhys::StaticStruct();
				for (int32 Index = 0, End = Context.GetRowCount(); Index < End; ++Index)
				{
					uint64 Value = PingPongAs[Index].Value;
					++Value;
					Context.AddColumn(Rows[Index], FTest_PingPongDurPhys{.Value = Value});
					Context.RemoveColumns(Rows[Index], {RemovedColumnStruct});
				}
			})
		.Where()
		.Compile());

	DataStorage.RegisterQuery(
		Select(TEXT("PingPong DurPhysics->PostPhysics"),
			FProcessor(EQueryTickPhase::DuringPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::Default))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, const RowHandle* RowPtr, FTest_PingPongDurPhys* PingPongBPtr)
			{
				QUICK_SCOPE_CYCLE_COUNTER(PingPong_During_Post);
				TArrayView<const RowHandle> Rows(RowPtr, Context.GetRowCount());
				TArrayView<FTest_PingPongDurPhys> PingPongAs(PingPongBPtr, Context.GetRowCount());
	
				const UScriptStruct* RemovedColumnStruct = FTest_PingPongDurPhys::StaticStruct();
				for (int32 Index = 0, End = Context.GetRowCount(); Index < End; ++Index)
				{
					uint64 Value = PingPongAs[Index].Value;
					++Value;
					Context.AddColumn(Rows[Index], FTest_PingPongPostPhys{.Value = Value});
					Context.RemoveColumns(Rows[Index], {RemovedColumnStruct});
				}
			})
		.Where()
		.Compile());

	DataStorage.RegisterQuery(
	Select(TEXT("PingPong PostPhysics->PrePhysics"),
		FProcessor(EQueryTickPhase::PostPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::Default))
			.SetExecutionMode(EExecutionMode::GameThread),
		[](IQueryContext& Context, const RowHandle* RowPtr, FTest_PingPongPostPhys* PingPongBPtr)
		{
			QUICK_SCOPE_CYCLE_COUNTER(PingPong_Post_Pre);
			TArrayView<const RowHandle> Rows(RowPtr, Context.GetRowCount());
			TArrayView<FTest_PingPongPostPhys> PingPongAs(PingPongBPtr, Context.GetRowCount());

			const UScriptStruct* RemovedColumnStruct = FTest_PingPongPostPhys::StaticStruct();
			for (int32 Index = 0, End = Context.GetRowCount(); Index < End; ++Index)
			{
				uint64 Value = PingPongAs[Index].Value;
				++Value;
				Context.AddColumn(Rows[Index], FTest_PingPongPrePhys{.Value = Value});
				Context.RemoveColumns(Rows[Index], {RemovedColumnStruct});
			}
		})
	.Where()
	.Compile());
}
