// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Modules/ModuleManager.h"

#include "TypedElementDataStoragePerformanceTestCommands.generated.h"

/**
 * Column to represent that a row is selected
 */
USTRUCT(meta = (DisplayName = ""))
struct FTest_PingPongPrePhys final : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	
	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	uint64 Value;
};

USTRUCT(meta = (DisplayName = ""))
struct FTest_PingPongDurPhys final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	uint64 Value;
};

USTRUCT(meta = (DisplayName = ""))
struct FTest_PingPongPostPhys final : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	
	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	uint64 Value;
};

/**
 * The heads up transform display provides at a glance view in a scene outliner row of abnormal transform characteristics, including:
 *		1. Non-uniform scale
 *		2. Negative scaling on X, Y, or Z axis
 *		3. Unnormalized rotation
 */
UCLASS()
class UTest_PingPongBetweenPhaseFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UTest_PingPongBetweenPhaseFactory() override = default;

	void RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
};