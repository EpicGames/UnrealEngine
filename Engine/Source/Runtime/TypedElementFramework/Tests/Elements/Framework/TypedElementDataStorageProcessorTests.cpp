// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS
#include "DataStorage/Features.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "HAL/IConsoleManager.h"
#include "TypedElementTestColumns.h"

namespace UE::Editor::DataStorage::Debug::ProcessorTests
{
	void OnProcessorTestsEnabled(IConsoleVariable*);

	bool GProcessorTestsEnabled = false;
	FAutoConsoleVariableRef CVarAllowUnversionedContentInEditor(
		TEXT("TEDS.Tests.ProcessorTestsEnabled"),
		GProcessorTestsEnabled,
		TEXT("If true, registers processors and additional commands with TEDS to test processors."),
		FConsoleVariableDelegate::CreateStatic(&OnProcessorTestsEnabled),
		ECVF_Default
	);

	TableHandle PrimaryTable = InvalidTableHandle;
	TableHandle SecondaryTable = InvalidTableHandle;

	TArray<QueryHandle> RegisteredQueries;
	TArray<IConsoleCommand*> RegisteredCommands;

	void RegisterProcessors()
	{
		using namespace UE::Editor::DataStorage::Queries;

		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

		if (PrimaryTable == InvalidTableHandle)
		{
			PrimaryTable =
				DataStorage->RegisterTable<FTEDSProcessorTestsReferenceColumn, FTEDSProcessorTests_PrimaryTag>(FName("ProcessorTests Primary Table"));
		}
		if (SecondaryTable == InvalidTableHandle)
		{
			SecondaryTable =
				DataStorage->RegisterTable<FTEDSProcessorTestsReferenceColumn, FTEDSProcessorTests_SecondaryTag>(FName("ProcessorTests Secondary Table"));
		}

		// Test creation of a row from within a query processor
		QueryHandle PrimaryRowQuery = DataStorage->RegisterQuery(
			Select(
				TEXT("TEST: Creating a row for primary reference column"),
				FProcessor(EQueryTickPhase::PrePhysics, DataStorage->GetQueryTickGroupName(EQueryTickGroups::Default)),
				[](IQueryContext& Context, const RowHandle* Rows, FTEDSProcessorTestsReferenceColumn* ReferenceColumns)
				{
					const int32 RowCount = Context.GetRowCount();
					TConstArrayView<RowHandle> RowsView = MakeArrayView(Rows, RowCount);
					TArrayView<FTEDSProcessorTestsReferenceColumn> ReferenceColumnsView = MakeArrayView(ReferenceColumns, RowCount);

					for (int32 Index = 0; Index < RowCount; ++Index)
					{
						FTEDSProcessorTestsReferenceColumn& PrimaryReferenceColumn = ReferenceColumnsView[Index];
						// Auto-create a secondary row if this points to no row
						if (!Context.IsRowAvailable(PrimaryReferenceColumn.Reference))
						{
							RowHandle SecondaryRow = Context.AddRow(SecondaryTable);
							// Initialize bi-directional row references
							PrimaryReferenceColumn.Reference = SecondaryRow;
							Context.AddColumn(SecondaryRow, FTEDSProcessorTestsReferenceColumn{.Reference = RowsView[Index]});
						}
					}
				})
			.Where()
				.All<FTEDSProcessorTests_PrimaryTag>()
				.None<FTEDSProcessorTests_Linked>()
			.Compile());

		RegisteredQueries.Emplace(PrimaryRowQuery);

		QueryHandle UpdateTransformWidget = DataStorage->RegisterQuery(
			Select()
				.ReadOnly<FTEDSProcessorTestsReferenceColumn>()
			.Where()
				.All<FTEDSProcessorTests_PrimaryTag>()
				.None<FTEDSProcessorTests_Linked>()
			.Compile());
	
		RegisteredQueries.Emplace(UpdateTransformWidget);

		QueryHandle SecondaryRowQuery = DataStorage->RegisterQuery(
			Select(TEXT("TEST: Creating a row for secondary reference column"),
				FProcessor(EQueryTickPhase::DuringPhysics, DataStorage->GetQueryTickGroupName(EQueryTickGroups::Default)),
				[](IQueryContext& Context, const RowHandle* Rows, FTEDSProcessorTestsReferenceColumn* ReferenceColumns)
				{
					const int32 RowCount = Context.GetRowCount();
					TConstArrayView<RowHandle> RowsView = MakeArrayView(Rows, RowCount);
					TArrayView<FTEDSProcessorTestsReferenceColumn> ReferenceColumnsView = MakeArrayView(ReferenceColumns, RowCount);

					for (int32 Index = 0; Index < RowCount; ++Index)
					{
						const RowHandle& SecondaryRow = RowsView[Index];
						Context.RunSubquery(
							0,
							ReferenceColumnsView[Index].Reference,
							CreateSubqueryCallbackBinding(
							[SecondaryRow](ISubqueryContext& SubqueryContext, RowHandle PrimaryRow, const FTEDSProcessorTestsReferenceColumn& ReferenceColumn)
							{
								if (ReferenceColumn.Reference == SecondaryRow)
								{
									// Add these tags to prevent further processing
									SubqueryContext.AddColumns<FTEDSProcessorTests_Linked>(SecondaryRow);
									SubqueryContext.AddColumns<FTEDSProcessorTests_Linked>(PrimaryRow);
								}
							}
						));
					}
				})
			.Where()
				.All<FTEDSProcessorTests_SecondaryTag>()
				.None<FTEDSProcessorTests_Linked>()
			.DependsOn()
				.SubQuery(UpdateTransformWidget)
			.Compile());

		RegisteredQueries.Emplace(SecondaryRowQuery);
	}

	void UnregisterProcessors()
	{
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		for (const UE::Editor::DataStorage::QueryHandle& Handle : RegisteredQueries)
		{
			DataStorage->UnregisterQuery(Handle);
		}
		RegisteredQueries.Empty();
	}

	void RegisterCommands()
	{
		IConsoleCommand* Command = IConsoleManager::Get().RegisterConsoleCommand(TEXT("TEDS.Tests.ProcessorTests.AddPrimaryRows"), TEXT(""), FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			int32 RowsToCreate = 0;
			if (Args.Num() != 1)
			{
				return;
			}
			TTypeFromString<int32>::FromString(RowsToCreate, *Args[0]);
			if (RowsToCreate <= 0)
			{
				return;
			}

			ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
			DataStorage->BatchAddRow(PrimaryTable, RowsToCreate, [](UE::Editor::DataStorage::RowHandle Row)
			{
			});
		}), ECVF_Default);
	
		RegisteredCommands.Emplace(Command);
	}

	void UnregisterCommands()
	{
		for (IConsoleCommand* Command : RegisteredCommands)
		{
			IConsoleManager::Get().UnregisterConsoleObject(Command);
		}
		RegisteredCommands.Empty();
	}

	void OnProcessorTestsEnabled(IConsoleVariable* Variable)
	{
		if (Variable->GetBool())
		{
			RegisterProcessors();
			RegisterCommands();
		}
		else
		{
			UnregisterProcessors();
			UnregisterCommands();
		}
	};
} // namespace UE::Editor::DataStorage::Debug::ProcessorTests
#endif
