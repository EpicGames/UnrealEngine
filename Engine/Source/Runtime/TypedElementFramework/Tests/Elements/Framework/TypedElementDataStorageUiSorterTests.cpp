// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Algo/Sort.h"
#include "Containers/AnsiString.h"
#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Math/NumericLimits.h"
#include "Templates/Tuple.h"
#include "Tests/TestHarnessAdapter.h"

namespace UE::Editor::DataStorage::Tests
{
	template<typename T>
	class FSingleValueTestSorter : public FColumnSorterInterface
	{
	public:
		virtual ~FSingleValueTestSorter() override = default;

		virtual int32 Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const override
		{
			if (Values[static_cast<int32>(Left)] == Values[static_cast<int32>(Right)])
			{
				return 0;
			}
			else
			{
				return Values[static_cast<int32>(Left)] < Values[static_cast<int32>(Right)] ? -1 : 1;
			}
		}

		virtual FPrefixInfo CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const override
		{
			return CreateSortPrefix(ByteIndex, Values[static_cast<int32>(Row)]);
		}

		virtual ESortType GetSortType() const override
		{
			return ESortType::HybridSort;
		}

		virtual FText GetShortName() const override
		{
			return FText::GetEmpty();
		}

		void SortByPrefix(const ICoreProvider& Storage, uint32 ByteIndex)
		{
			struct FData
			{
				uint64 Prefix;
				T Value;
			};
			TArray<FData> Data;
			Data.Reserve(Values.Num());

			for (int32 Index = 0; Index < Values.Num(); ++Index)
			{
				Data.Add(FData
					{
						.Prefix = CalculatePrefix(Storage, Index, ByteIndex).Prefix,
						.Value = MoveTemp(Values[Index])
					});
			}

			RadixSort64(Data.GetData(), Data.Num(), [](const FData& Entry) { return Entry.Prefix; });

			Values.Reset();
			for (FData& Entry : Data)
			{
				Values.Add(MoveTemp(Entry.Value));
			}
		}

		bool Verify(const ICoreProvider& Storage)
		{
			for (int32 Index = 0; Index < Values.Num() - 1; ++Index)
			{
				if (Compare(Storage, Index, Index + 1) > 0)
				{
					return false;
				}
			}
			return true;
		}

		TArray<T> Values;
	};

	template<typename... Types>
	class FMultiValueTestSorter : public FColumnSorterInterface
	{
	public:
		virtual ~FMultiValueTestSorter() override = default;

		virtual int32 Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const override
		{
			if (Values[static_cast<int32>(Left)] == Values[static_cast<int32>(Right)])
			{
				return 0;
			}
			else
			{
				return Values[static_cast<int32>(Left)] < Values[static_cast<int32>(Right)] ? -1 : 1;
			}
		}

		virtual FPrefixInfo CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const override
		{
			return Values[static_cast<int32>(Row)].ApplyAfter([this, ByteIndex](auto&&... TupleValues)
				{
					return CreateSortPrefix(ByteIndex, TupleValues...);
				});
		}

		virtual ESortType GetSortType() const override
		{
			return ESortType::HybridSort;
		}

		virtual FText GetShortName() const override
		{
			return FText::GetEmpty();
		}

		void SortByPrefix(const ICoreProvider& Storage, uint32 ByteIndex, int32 RangeStart, int32 RangeEnd)
		{
			struct FData
			{
				uint64 Prefix;
				TTuple<Types...> ValueTuple;
			};
			TArray<FData> Data;
			Data.Reserve(RangeEnd - RangeStart + 1);

			for (int32 Index = RangeStart; Index <= RangeEnd; ++Index)
			{
				uint64 Prefix = Values[Index].ApplyAfter([this, ByteIndex](auto&&... TupleValues)
					{
						return CreateSortPrefix(ByteIndex, TupleValues...);
					}).Prefix;
				Data.Add(FData
					{
						.Prefix = Prefix,
						.ValueTuple = MoveTemp(Values[Index])
					});
			}

			RadixSort64(Data.GetData(), Data.Num(), [](const FData& Entry) { return Entry.Prefix; });

			for (int32 Index = 0; Index <= RangeEnd - RangeStart; ++Index)
			{
				Values[RangeStart + Index] = MoveTemp(Data[Index].ValueTuple);
			}
		}

		bool Verify(const ICoreProvider& Storage)
		{
			for (int32 Index = 0; Index < Values.Num() - 1; ++Index)
			{
				if (Compare(Storage, Index, Index + 1) > 0)
				{
					return false;
				}
			}
			return true;
		}

		TArray<TTuple<Types...>> Values;
	};

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_UInt64, 
		"Editor::DataStorage::FColumnSorterInterface_Prefix_UInt64", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			FSingleValueTestSorter<uint64> Sorter;
			Sorter.Values.Append({ 1, 42, 7, 33, 1024 });
			Sorter.SortByPrefix(*DataStorage, 0);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_UInt32, 
		"Editor::DataStorage::FColumnSorterInterface_Prefix_UInt32", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			FSingleValueTestSorter<uint32> Sorter;
			Sorter.Values.Append({ 1, 42, 7, 33, 1024 });
			Sorter.SortByPrefix(*DataStorage, 0);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_UInt16, 
		"Editor::DataStorage::FColumnSorterInterface_Prefix_UInt16", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			FSingleValueTestSorter<uint16> Sorter;
			Sorter.Values.Append({ 1, 42, 7, 33, 1024 });
			Sorter.SortByPrefix(*DataStorage, 0);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_UInt8, 
		"Editor::DataStorage::FColumnSorterInterface_Prefix_UInt8", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			FSingleValueTestSorter<uint8> Sorter;
			Sorter.Values.Append({ 1, 42, 7, 33, 255 });
			Sorter.SortByPrefix(*DataStorage, 0);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_Int64, 
		"Editor::DataStorage::FColumnSorterInterface_Prefix_Int64", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			FSingleValueTestSorter<int64> Sorter;
			Sorter.Values.Append({ 1, -1, -8, 42, TNumericLimits<int64>::Min(), TNumericLimits<int64>::Max(), 7, -53221, 33, 1024 });
			Sorter.SortByPrefix(*DataStorage, 0);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_Int32, 
		"Editor::DataStorage::FColumnSorterInterface_Prefix_Int32", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			FSingleValueTestSorter<int32> Sorter;
			Sorter.Values.Append({ 1, -1, -8, 42, TNumericLimits<int32>::Min(), TNumericLimits<int32>::Max(), 7, -53221, 33, 1024 });
			Sorter.SortByPrefix(*DataStorage, 0);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_Int16, 
		"Editor::DataStorage::FColumnSorterInterface_Prefix_Int16", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			FSingleValueTestSorter<int16> Sorter;
			Sorter.Values.Append({ 1, -1, -8, 42, TNumericLimits<int16>::Min(), TNumericLimits<int16>::Max(), 7, -3221, 33, 1024 });
			Sorter.SortByPrefix(*DataStorage, 0);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_Int8, 
		"Editor::DataStorage::FColumnSorterInterface_Prefix_Int8", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			FSingleValueTestSorter<int8> Sorter;
			Sorter.Values.Append({ 1, -1, -8, 42, TNumericLimits<int8>::Min(), TNumericLimits<int8>::Max(), 7, -128, 33, 127 });
			Sorter.SortByPrefix(*DataStorage, 0);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_Float, 
		"Editor::DataStorage::FColumnSorterInterface_Prefix_Float", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			FSingleValueTestSorter<float> Sorter;
			Sorter.Values.Append({ 1.44f, -1.32f, -8.8675463f, 42.145165f, 7.0f, -53221542.331f, 33.8763f, 1024.45290625f });
			Sorter.SortByPrefix(*DataStorage, 0);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_Double, 
		"Editor::DataStorage::FColumnSorterInterface_Prefix_Double", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			FSingleValueTestSorter<double> Sorter;
			Sorter.Values.Append({ 1.44, -1.32, -8.8675463, 42.145165, 7.0, -53221542.331, 33.8763, 1024.45290625 });
			Sorter.SortByPrefix(*DataStorage, 0);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_Boolean, 
		"Editor::DataStorage::FColumnSorterInterface_Prefix_Boolean", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			FSingleValueTestSorter<bool> Sorter;
			Sorter.Values.Append({ true, false, false, true, false, false, true, true });
			Sorter.SortByPrefix(*DataStorage, 0);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_MultiVariableSizedValues,
		"Editor::DataStorage::FColumnSorterInterface_Prefix_MultiVariableSizedValues", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			{
				FMultiValueTestSorter<uint32> Sorter;
				Sorter.Values.Add(
					{
						static_cast<uint32>(0x11223344)
					});

				FPrefixInfo Result;
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 0);
				CHECK(Result.Prefix == 0x1122334400000000);
				CHECK(Result.bHasRemainingBytes == false);
			}

			{
				FMultiValueTestSorter<uint32, uint8> Sorter;
				Sorter.Values.Add(
					{
						static_cast<uint32>(0x11223344),
						static_cast<uint8>(0x55)
					});

				FPrefixInfo Result;
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 0);
				CHECK(Result.Prefix == 0x1122334455000000);
				CHECK(Result.bHasRemainingBytes == false);
			}

			{
				FMultiValueTestSorter<uint32, uint8, uint16> Sorter;
				Sorter.Values.Add(
					{
						static_cast<uint32>(0x11223344),
						static_cast<uint8>(0x55),
						static_cast<uint16>(0x6677)
					});

				FPrefixInfo Result;
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 0);
				CHECK(Result.Prefix == 0x1122334455667700);
				CHECK(Result.bHasRemainingBytes == false);
			}

			{
				FMultiValueTestSorter<uint32, uint8, uint16, uint32> Sorter;
				Sorter.Values.Add(
					{
						static_cast<uint32>(0x11223344),
						static_cast<uint8>(0x55),
						static_cast<uint16>(0x6677),
						static_cast<uint32>(0x8899aabb)
					});

				FPrefixInfo Result;
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 0);
				CHECK(Result.Prefix == 0x1122334455667788);
				CHECK(Result.bHasRemainingBytes == true);

				Result = Sorter.CalculatePrefix(*DataStorage, 0, 8);
				CHECK(Result.Prefix == 0x99aabb0000000000);
				CHECK(Result.bHasRemainingBytes == false);
			}

			{
				FMultiValueTestSorter<uint32, uint8, uint16, uint32, uint64> Sorter;
				Sorter.Values.Add(
					{
						static_cast<uint32>(0x11223344),
						static_cast<uint8>(0x55),
						static_cast<uint16>(0x6677),
						static_cast<uint32>(0x8899aabb),
						static_cast<uint64>(0xccddeeff11223344)
					});

				FPrefixInfo Result;
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 0);
				CHECK(Result.Prefix == 0x1122334455667788);
				CHECK(Result.bHasRemainingBytes == true);

				Result = Sorter.CalculatePrefix(*DataStorage, 0, 8);
				CHECK(Result.Prefix == 0x99aabbccddeeff11);
				CHECK(Result.bHasRemainingBytes == true);

				Result = Sorter.CalculatePrefix(*DataStorage, 0, 16);
				CHECK(Result.Prefix == 0x2233440000000000);
				CHECK(Result.bHasRemainingBytes == false);
			}

			{
				FMultiValueTestSorter<uint32, uint8, uint16, uint32, uint64, uint16> Sorter;
				Sorter.Values.Add(
					{
						static_cast<uint32>(0x11223344),
						static_cast<uint8>(0x55),
						static_cast<uint16>(0x6677),
						static_cast<uint32>(0x8899aabb),
						static_cast<uint64>(0xccddeeff11223344),
						static_cast<uint16>(0x5566)
					});

				FPrefixInfo Result;
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 0);
				CHECK(Result.Prefix == 0x1122334455667788);
				CHECK(Result.bHasRemainingBytes == true);

				Result = Sorter.CalculatePrefix(*DataStorage, 0, 8);
				CHECK(Result.Prefix == 0x99aabbccddeeff11);
				CHECK(Result.bHasRemainingBytes == true);

				Result = Sorter.CalculatePrefix(*DataStorage, 0, 16);
				CHECK(Result.Prefix == 0x2233445566000000);
				CHECK(Result.bHasRemainingBytes == false);
			}

			{
				FMultiValueTestSorter<uint32, uint8, uint16, uint32, uint64, uint16, uint8> Sorter;
				Sorter.Values.Add(
					{
						static_cast<uint32>(0x11223344),
						static_cast<uint8>(0x55),
						static_cast<uint16>(0x6677),
						static_cast<uint32>(0x8899aabb),
						static_cast<uint64>(0xccddeeff11223344),
						static_cast<uint16>(0x5566),
						static_cast<uint8>(0x77)
					});

				FPrefixInfo Result;
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 0);
				CHECK(Result.Prefix == 0x1122334455667788);
				CHECK(Result.bHasRemainingBytes == true);

				Result = Sorter.CalculatePrefix(*DataStorage, 0, 8);
				CHECK(Result.Prefix == 0x99aabbccddeeff11);
				CHECK(Result.bHasRemainingBytes == true);

				Result = Sorter.CalculatePrefix(*DataStorage, 0, 16);
				CHECK(Result.Prefix == 0x2233445566770000);
				CHECK(Result.bHasRemainingBytes == false);
			}

			{
				FMultiValueTestSorter<uint32, uint8, uint16, uint32, uint64, uint16, uint8, uint16> Sorter;
				Sorter.Values.Add(
					{
						static_cast<uint32>(0x11223344),
						static_cast<uint8>(0x55),
						static_cast<uint16>(0x6677),
						static_cast<uint32>(0x8899aabb),
						static_cast<uint64>(0xccddeeff11223344),
						static_cast<uint16>(0x5566),
						static_cast<uint8>(0x77),
						static_cast<uint16>(0x8899)
					});

				FPrefixInfo Result;
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 0);
				CHECK(Result.Prefix == 0x1122334455667788);
				CHECK(Result.bHasRemainingBytes == true);

				Result = Sorter.CalculatePrefix(*DataStorage, 0, 8);
				CHECK(Result.Prefix == 0x99aabbccddeeff11);
				CHECK(Result.bHasRemainingBytes == true);

				Result = Sorter.CalculatePrefix(*DataStorage, 0, 16);
				CHECK(Result.Prefix == 0x2233445566778899);
				CHECK(Result.bHasRemainingBytes == false);
			}
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_StringIndexCheck,
		"Editor::DataStorage::FColumnSorterInterface_Prefix_StringIndexCheck", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			using SortStringView = TSortStringView<FSortCaseSensitive, FWideStringView>;
			FSingleValueTestSorter<SortStringView> Sorter;
			FPrefixInfo Result;
			{
				FWideString String = TEXT("a");
				Sorter.Values.Add(SortStringView(String));
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 0);
				CHECK(Result.Prefix == 0x0061000000000000);
				CHECK(Result.bHasRemainingBytes == false);
				Sorter.Values.Empty();
			}
			{
				FWideString String = TEXT("aa");
				Sorter.Values.Add(SortStringView(String));
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 0);
				CHECK(Result.Prefix == 0x0061006100000000);
				CHECK(Result.bHasRemainingBytes == false);
				Sorter.Values.Empty();
			}
			{
				FWideString String = TEXT("aaa");
				Sorter.Values.Add(SortStringView(String));
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 0);
				CHECK(Result.Prefix == 0x0061006100610000);
				CHECK(Result.bHasRemainingBytes == false);
				Sorter.Values.Empty();
			}
			{
				FWideString String = TEXT("aaaa");
				Sorter.Values.Add(SortStringView(String));
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 0);
				CHECK(Result.Prefix == 0x0061006100610061);
				CHECK(Result.bHasRemainingBytes == false);
				Sorter.Values.Empty();
			}
			{
				FWideString String = TEXT("aaaab");
				Sorter.Values.Add(SortStringView(String));
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 0);
				CHECK(Result.Prefix == 0x0061006100610061);
				CHECK(Result.bHasRemainingBytes == true);
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 8);
				CHECK(Result.Prefix == 0x0062000000000000);
				CHECK(Result.bHasRemainingBytes == false);
				Sorter.Values.Empty();
			}
			{
				FWideString String = TEXT("aaaabb");
				Sorter.Values.Add(SortStringView(String));
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 0);
				CHECK(Result.Prefix == 0x0061006100610061);
				CHECK(Result.bHasRemainingBytes == true);
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 8);
				CHECK(Result.Prefix == 0x0062006200000000);
				CHECK(Result.bHasRemainingBytes == false);
				Sorter.Values.Empty();
			}
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_StringIndexCheckWithOffset,
		"Editor::DataStorage::FColumnSorterInterface_Prefix_StringIndexCheckWithOffset", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			using SortStringView = TSortStringView<FSortCaseSensitive, FWideStringView>;
			FMultiValueTestSorter<uint16, uint32, SortStringView> Sorter;
			FPrefixInfo Result;
			{
				FWideString String = TEXT("a");
				Sorter.Values.Add({ static_cast<uint16>(0xaaaa), 0xbbbbcccc, SortStringView(String) });
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 0);
				CHECK(Result.Prefix == 0xaaaabbbbcccc0061);
				CHECK(Result.bHasRemainingBytes == false);
				Sorter.Values.Empty();
			}
			{
				FWideString String = TEXT("ab");
				Sorter.Values.Add({ static_cast<uint16>(0xaaaa), 0xbbbbcccc, SortStringView(String) });
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 0);
				CHECK(Result.Prefix == 0xaaaabbbbcccc0061);
				CHECK(Result.bHasRemainingBytes == true);
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 8);
				CHECK(Result.Prefix == 0x0062000000000000);
				CHECK(Result.bHasRemainingBytes == false);
				Sorter.Values.Empty();
			}
			{
				FWideString String = TEXT("abc");
				Sorter.Values.Add({ static_cast<uint16>(0xaaaa), 0xbbbbcccc, SortStringView(String) });
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 0);
				CHECK(Result.Prefix == 0xaaaabbbbcccc0061);
				CHECK(Result.bHasRemainingBytes == true);
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 8);
				CHECK(Result.Prefix == 0x0062006300000000);
				CHECK(Result.bHasRemainingBytes == false);
				Sorter.Values.Empty();
			}
			{
				FWideString String = TEXT("abcd");
				Sorter.Values.Add({ static_cast<uint16>(0xaaaa), 0xbbbbcccc, SortStringView(String) });
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 0);
				CHECK(Result.Prefix == 0xaaaabbbbcccc0061);
				CHECK(Result.bHasRemainingBytes == true);
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 8);
				CHECK(Result.Prefix == 0x0062006300640000);
				CHECK(Result.bHasRemainingBytes == false);
				Sorter.Values.Empty();
			}
			{
				FWideString String = TEXT("abcde");
				Sorter.Values.Add({ static_cast<uint16>(0xaaaa), 0xbbbbcccc, SortStringView(String) });
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 0);
				CHECK(Result.Prefix == 0xaaaabbbbcccc0061);
				CHECK(Result.bHasRemainingBytes == true);
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 8);
				CHECK(Result.Prefix == 0x0062006300640065);
				CHECK(Result.bHasRemainingBytes == false);
				Sorter.Values.Empty();
			}
			{
				FWideString String = TEXT("abcdef");
				Sorter.Values.Add({ static_cast<uint16>(0xaaaa), 0xbbbbcccc, SortStringView(String) });
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 0);
				CHECK(Result.Prefix == 0xaaaabbbbcccc0061);
				CHECK(Result.bHasRemainingBytes == true);
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 8);
				CHECK(Result.Prefix == 0x0062006300640065);
				CHECK(Result.bHasRemainingBytes == true);
				Result = Sorter.CalculatePrefix(*DataStorage, 0, 16);
				CHECK(Result.Prefix == 0x0066000000000000);
				CHECK(Result.bHasRemainingBytes == false);
				Sorter.Values.Empty();
			}
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_StringView_CaseSensitive,
		"Editor::DataStorage::FColumnSorterInterface_Prefix_StringView_CaseSensitive", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			TArray<FString> Strings{ TEXT("a"), TEXT("bba"), TEXT("Aba"), TEXT("cba"), TEXT("ABac"), TEXT("longstring") };
			FSingleValueTestSorter<TSortStringView<FSortCaseSensitive, FStringView>> Sorter;
			Sorter.Values.Append(Strings);
			Sorter.SortByPrefix(*DataStorage, 0);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_StringView_CaseInsensitive,
		"Editor::DataStorage::FColumnSorterInterface_Prefix_StringView_CaseInsensitive", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			TArray<FString> Strings{ TEXT("a"), TEXT("bba"), TEXT("Aba"), TEXT("cba"), TEXT("ABac"), TEXT("longstring") };
			FSingleValueTestSorter<TSortStringView<FSortCaseInsensitive, FStringView>> Sorter;
			Sorter.Values.Append(Strings);
			Sorter.SortByPrefix(*DataStorage, 0);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_WideStringView_CaseSensitive,
		"Editor::DataStorage::FColumnSorterInterface_Prefix_WideStringView_CaseSensitive", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			TArray<FWideString> Strings
			{ 
				TEXT("a"), 
				TEXT("bba"), 
				TEXT("Aba"), 
				TEXT("cba"), 
				TEXT("ABac"), 
				TEXT("longstring")
			};
			FSingleValueTestSorter<TSortStringView<FSortCaseSensitive, FWideStringView>> Sorter;
			Sorter.Values.Append(Strings);
			Sorter.SortByPrefix(*DataStorage, 0);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_WideStringView_CaseInsensitive,
		"Editor::DataStorage::FColumnSorterInterface_Prefix_WideStringView_CaseInsensitive", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			TArray<FWideString> Strings
			{ 
				TEXT("a"), 
				TEXT("bba"), 
				TEXT("Aba"), 
				TEXT("cba"), 
				TEXT("ABac"), 
				TEXT("longstring")
			};
			FSingleValueTestSorter<TSortStringView<FSortCaseInsensitive, FWideStringView>> Sorter;
			Sorter.Values.Append(Strings);
			Sorter.SortByPrefix(*DataStorage, 0);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_AnsiStringView_CaseSensitive,
		"Editor::DataStorage::FColumnSorterInterface_Prefix_AnsiStringView_CaseSensitive", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			TArray<FAnsiString> Strings{ "a", "bba", "Aba", "cba", "ABac", "longstring" };
			FSingleValueTestSorter<TSortStringView<FSortCaseSensitive, FAnsiStringView>> Sorter;
			Sorter.Values.Append(Strings);
			Sorter.SortByPrefix(*DataStorage, 0);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_AnsiStringView_CaseInsensitive,
		"Editor::DataStorage::FColumnSorterInterface_Prefix_AnsiStringView_CaseInsensitive", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			TArray<FAnsiString> Strings{ "a", "bba", "Aba", "cba", "ABac", "longstring" };
			FSingleValueTestSorter<TSortStringView<FSortCaseInsensitive, FAnsiStringView>> Sorter;
			Sorter.Values.Append(Strings);
			Sorter.SortByPrefix(*DataStorage, 0);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_TextView_CaseInsensitive,
		"Editor::DataStorage::FColumnSorterInterface_Prefix_TextView_CaseInsensitive", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			TArray<FText> Strings
			{ 
				FText::FromString("a"), 
				FText::FromString("bba"), 
				FText::FromString("Aba"), 
				FText::FromString("cba"), 
				FText::FromString("ABac"), 
				FText::FromString("longstring")
			};
			FSingleValueTestSorter<TSortStringView<FSortCaseInsensitive, FStringView>> Sorter;
			Sorter.Values.Append(Strings);
			Sorter.SortByPrefix(*DataStorage, 0);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_TextView_CaseSensitive,
		"Editor::DataStorage::FColumnSorterInterface_Prefix_TextView_CaseSensitive", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			TArray<FText> Strings
			{
				FText::FromString("a"),
				FText::FromString("bba"),
				FText::FromString("Aba"),
				FText::FromString("cba"),
				FText::FromString("ABac"),
				FText::FromString("longstring")
			};
			FSingleValueTestSorter<TSortStringView<FSortCaseSensitive, FStringView>> Sorter;
			Sorter.Values.Append(Strings);
			Sorter.SortByPrefix(*DataStorage, 0);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_FNameByString,
		"Editor::DataStorage::FColumnSorterInterface_Prefix_FNameByString", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			TArray<FName> Names{ "a", "bba", "Aba", "cba", "ABac", "LongName" };
			FSingleValueTestSorter<TSortNameView<TSortByName<>>> Sorter;
			Sorter.Values.Append(Names);
			Sorter.SortByPrefix(*DataStorage, 0);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_FNameById,
		"Editor::DataStorage::FColumnSorterInterface_Prefix_FNameById", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			TArray<FName> Names{ "a", "bba", "Aba", "cba", "ABac", "LongName" };
			FSingleValueTestSorter<TSortNameView<FSortById>> Sorter;
			Sorter.Values.Append(Names);
			Sorter.SortByPrefix(*DataStorage, 0);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_8ByteMultiValue, 
		"Editor::DataStorage::FColumnSorterInterface_Prefix_8ByteMultiValue", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			FMultiValueTestSorter<uint64, uint64, uint64> Sorter;
			Sorter.Values.Append(
				{
					{ 42, 88, 31 },
					{ 3, 102, 88 },
					{ 88, 42, 400 },
					{ 88, 16, 1 },
					{ 88, 32, 24 },
					{ 88, 32, 12 },
					{ 4, 502998, 87 },
					{ 502998, 17, 99}
				});
			// Mimic the steps of a bucket sort.
			Sorter.SortByPrefix(*DataStorage, 0, 0, 7);
			Sorter.SortByPrefix(*DataStorage, 8, 3, 6);
			Sorter.SortByPrefix(*DataStorage, 16, 4, 5);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_4ByteMultiValue, 
		"Editor::DataStorage::FColumnSorterInterface_Prefix_4ByteMultiValue", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			FMultiValueTestSorter<uint32, uint32, uint32> Sorter;
			Sorter.Values.Append(
				{
					{ 42, 88, 31 },
					{ 3, 102, 88 },
					{ 88, 42, 400 },
					{ 88, 16, 1 },
					{ 88, 32, 24 },
					{ 88, 32, 12 },
					{ 4, 502998, 87 },
					{ 502998, 17, 99}
				});
			// Mimic the steps of a bucket sort.
			Sorter.SortByPrefix(*DataStorage, 0, 0, 7);
			Sorter.SortByPrefix(*DataStorage, 8, 4, 5);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_MultiValueMisMatchedSizes, 
		"Editor::DataStorage::FColumnSorterInterface_Prefix_MultiValueMisMatchedSizes", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			FMultiValueTestSorter<int8, uint16, uint64> Sorter;
			Sorter.Values.Append(
				{
					{ static_cast<int8>(42), static_cast<uint16>(88), 31ull },
					{ static_cast<int8>(-3), static_cast<uint16>(102), 88ull },
					{ static_cast<int8>(88), static_cast<uint16>(42), 400ull },
					{ static_cast<int8>(88), static_cast<uint16>(16), 1ull },
					{ static_cast<int8>(88), static_cast<uint16>(32), 0xaabbccdd11223333ull },
					{ static_cast<int8>(88), static_cast<uint16>(32), 0xaabbccdd11223344ull },
					{ static_cast<int8>(88), static_cast<uint16>(32), 0xaabbccdd11223322ull },
					{ static_cast<int8>( 4), static_cast<uint16>(502998), 87ull },
					{ static_cast<int8>(-1), static_cast<uint16>(17), 99ull}
				});
			// Mimic the steps of a bucket sort.
			Sorter.SortByPrefix(*DataStorage, 0, 0, 8);
			Sorter.SortByPrefix(*DataStorage, 8, 5, 7);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_MultiValueStrings,
		"Editor::DataStorage::FColumnSorterInterface_Prefix_MultiValueStrings", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			TArray<FString> Strings{ TEXT("a"), TEXT("bba"), TEXT("cb"), TEXT("aba"), TEXT("abac"), TEXT("longstring") };
			
			FMultiValueTestSorter<int16, TSortStringView<FSortCaseInsensitive, FStringView>> Sorter;
			Sorter.Values.Append(
				{
					{ static_cast<int16>(42), TSortStringView(FSortCaseInsensitive{}, Strings[0]) },
					{ static_cast<int16>(-3), TSortStringView(FSortCaseInsensitive{}, Strings[1]) },
					{ static_cast<int16>(88), TSortStringView(FSortCaseInsensitive{}, Strings[2]) },
					{ static_cast<int16>(88), TSortStringView(FSortCaseInsensitive{}, Strings[3]) },
					{ static_cast<int16>( 4), TSortStringView(FSortCaseInsensitive{}, Strings[4]) },
					{ static_cast<int16>(-1), TSortStringView(FSortCaseInsensitive{}, Strings[5]) }
				});
			// Mimic the steps of a bucket sort.
			Sorter.SortByPrefix(*DataStorage, 0, 0, 5);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}

	TEST_CASE_NAMED(FEditorDataStorageUiSorter_Prefix_MultiValueStringsMatchingNumbers,
		"Editor::DataStorage::FColumnSorterInterface_Prefix_MultiValueStringsMatchingNumbers", "[ApplicationContextMask][EngineFilter]")
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			TArray<FString> Strings{ TEXT("a"), TEXT("bb_a"), TEXT("cb"), TEXT("ab_ca"), TEXT("ab_ac"), TEXT("longstring") };

			FMultiValueTestSorter<uint32, TSortStringView<FSortCaseInsensitive, FStringView>> Sorter;
			Sorter.Values.Append(
				{
					{ 42u, TSortStringView(FSortCaseInsensitive{}, Strings[0]) },
					{ 42u, TSortStringView(FSortCaseInsensitive{}, Strings[1]) },
					{ 42u, TSortStringView(FSortCaseInsensitive{}, Strings[2]) },
					{ 42u, TSortStringView(FSortCaseInsensitive{}, Strings[3]) },
					{ 42u, TSortStringView(FSortCaseInsensitive{}, Strings[4]) },
					{ 42u, TSortStringView(FSortCaseInsensitive{}, Strings[5]) }
				});
			// Mimic the steps of a bucket sort.
			Sorter.SortByPrefix(*DataStorage, 0, 0, 5);
			Sorter.SortByPrefix(*DataStorage, 8, 1, 2);
			CHECK(Sorter.Verify(*DataStorage));
		}
	}
} // namespace UE::Editor::DataStorage::Tests

#endif // WITH_TESTS