// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS
#include "TypedElementTestColumns.h"

#include "Algo/Sort.h"
#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "DataStorage/Features.h"
#include "DataStorage/Queries/Conditions.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Tests/TestHarnessAdapter.h"
#include "UObject/Class.h"

namespace UE::Editor::DataStorage::Tests
{
	using namespace UE::Editor::DataStorage::Queries;

static void AppendColumnName(FString& Output, TWeakObjectPtr<const UScriptStruct> TypeInfo)
{
#if WITH_EDITORONLY_DATA
	static FName DisplayNameName("DisplayName");
	if (const FString* Name = TypeInfo->FindMetaData(DisplayNameName))
	{
		Output += *Name;
	}
#else
	Output += TEXT("<Unavailable>");
#endif
}

static bool TestMatching(FConditions& TestQuery, const TArray<FColumnBase>& RequestedColumns, bool Expected, bool Sort = false)
{
	ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

	TestQuery.Compile(FEditorStorageQueryConditionCompileContext(Storage));
	
	if (Sort)
	{
		Algo::SortBy(RequestedColumns,
			[](const FColumnBase& Column) { return Column.TypeInfo.Get(); });
	}

	TArray<TWeakObjectPtr<const UScriptStruct>> Matches;
	bool Result = TestQuery.Verify(Matches, RequestedColumns);

	FString Description = (Result == Expected) ? TEXT("[Pass] ") : TEXT("[Fail] ");
	TestQuery.AppendToString(Description);

#if WITH_EDITORONLY_DATA
	{
		Description += " -> { ";
		auto It = RequestedColumns.begin();
		AppendColumnName(Description, (*It).TypeInfo);
		++It;
		for (; It != RequestedColumns.end(); ++It)
		{
			Description += ", ";
			AppendColumnName(Description, (*It).TypeInfo);
		}
		Description += " } ";
	}
#endif

	if (Expected)
	{
#if WITH_EDITORONLY_DATA
		if (!Matches.IsEmpty())
		{
			Description += " -> { ";
			auto It = Matches.begin();
			AppendColumnName(Description, *It);
			++It;
			for (; It != Matches.end(); ++It)
			{
				Description += ", ";
				AppendColumnName(Description, *It);
			}
			Description += " } ";
		}
#endif

		for (TWeakObjectPtr<const UScriptStruct> Match : Matches)
		{
			bool Found = false;
			for (const FColumnBase& Requested : RequestedColumns)
			{
				if (Match == Requested.TypeInfo)
				{
					Found = true;
					break;
				}
			}

			if (!Found)
			{
				Result = false;
				Description += " [Match failed]";
				break;
			}
		}
	}

	INFO(MoveTemp(Description));
	return (Result == Expected);
}

TEST_CASE_NAMED(FTypedElementQueryConditions_NoColumn, "Editor::DataStorage::QueryBuilder::FTypedElementQueryConditions_NoColumn", "[ApplicationContextMask][EngineFilter]")
{
	FConditions Example;
	
	CHECK(Example.MinimumColumnMatchRequired() == 0);
	// Since there are no restrictions provided in the query, all input passes.
	CHECK(TestMatching(Example, { TColumn<FTestColumnA>() }, true));
}

TEST_CASE_NAMED(FTypedElementQueryConditions_OneColumn, "Editor::DataStorage::Queries::FTypedElementQueryConditions_OneColumn", "[ApplicationContextMask][EngineFilter]")
{
	FConditions Example{ TColumn<FTestColumnA>() };

	CHECK(Example.MinimumColumnMatchRequired() == 1);
	CHECK(TestMatching(Example, { TColumn<FTestColumnA>() }, true));
}

TEST_CASE_NAMED(FTypedElementQueryConditions1, "Editor::DataStorage::QueryBuilder::FTypedElementQueryConditions A && B && C", "[ApplicationContextMask][EngineFilter]")
{
	TColumn<FTestColumnA> TestA;
	
	FConditions Example = TColumn<FTestColumnA>() && TColumn<FTestColumnB>() && TColumn<FTestColumnC>();
	
	CHECK(Example.MinimumColumnMatchRequired() == 3);
	CHECK(TestMatching(Example, { TColumn<FTestColumnA>(), TColumn<FTestColumnB>(), TColumn<FTestColumnC>() }, true));
	CHECK(TestMatching(Example, { TColumn<FTestColumnA>(), TColumn<FTestColumnB>(), TColumn<FTestColumnD>() }, false));
}

TEST_CASE_NAMED(FTypedElementQueryConditions2, "Editor::DataStorage::QueryBuilder::FTypedElementQueryConditions A || B || C", "[ApplicationContextMask][EngineFilter]")
{
	FConditions Example = TColumn<FTestColumnA>() || TColumn<FTestColumnB>() || TColumn<FTestColumnC>();
	
	CHECK(Example.MinimumColumnMatchRequired() == 1);
	CHECK(TestMatching(Example, { TColumn<FTestColumnB>() }, true));
	CHECK(TestMatching(Example, { TColumn<FTestColumnB>(), TColumn<FTestColumnC>() }, true));
	CHECK(TestMatching(Example, { TColumn<FTestColumnD>() }, false));
}

TEST_CASE_NAMED(FTypedElementQueryConditions3, "Editor::DataStorage::QueryBuilder::FTypedElementQueryConditions A && (B || C)", "[ApplicationContextMask][EngineFilter]")
{
	FConditions Example = TColumn<FTestColumnA>() && (TColumn<FTestColumnB>() || TColumn<FTestColumnC>());
	
	CHECK(Example.MinimumColumnMatchRequired() == 2);
	CHECK(TestMatching(Example, { TColumn<FTestColumnA>(), TColumn<FTestColumnB>() }, true));
	CHECK(TestMatching(Example, { TColumn<FTestColumnA>(), TColumn<FTestColumnC>() }, true));
	CHECK(TestMatching(Example, { TColumn<FTestColumnA>(), TColumn<FTestColumnD>() }, false));
	CHECK(TestMatching(Example, { TColumn<FTestColumnD>(), TColumn<FTestColumnB>() }, false));
}

TEST_CASE_NAMED(FTypedElementQueryConditions4, "Editor::DataStorage::QueryBuilder::FTypedElementQueryConditions A && (B || C) && (D || E)", "[ApplicationContextMask][EngineFilter]")
{
	FConditions Example = 
		TColumn<FTestColumnA>() && 
		(TColumn<FTestColumnB>() || TColumn<FTestColumnC>()) &&
		(TColumn<FTestColumnD>() || TColumn<FTestColumnE>());
	
	CHECK(Example.MinimumColumnMatchRequired() == 3);
	
	CHECK(TestMatching(Example, { TColumn<FTestColumnA>() }, false));
	CHECK(TestMatching(Example, { TColumn<FTestColumnA>(), TColumn<FTestColumnB>() }, false));

	CHECK(TestMatching(Example, { TColumn<FTestColumnA>(), TColumn<FTestColumnB>(), TColumn<FTestColumnD>() }, true));
	CHECK(TestMatching(Example, { TColumn<FTestColumnA>(), TColumn<FTestColumnB>(), TColumn<FTestColumnE>() }, true));
	CHECK(TestMatching(Example, { TColumn<FTestColumnA>(), TColumn<FTestColumnC>(), TColumn<FTestColumnD>() }, true));

	CHECK(TestMatching(Example, { TColumn<FTestColumnA>(), TColumn<FTestColumnC>(), TColumn<FTestColumnF>() }, false));
	CHECK(TestMatching(Example, { TColumn<FTestColumnA>(), TColumn<FTestColumnF>(), TColumn<FTestColumnD>() }, false));
	CHECK(TestMatching(Example, { TColumn<FTestColumnB>(), TColumn<FTestColumnC>(), TColumn<FTestColumnD>() }, false));
}

TEST_CASE_NAMED(FTypedElementQueryConditions5, "Editor::DataStorage::QueryBuilder::FTypedElementQueryConditions (A || B) && (C || D) && (E || F)", "[ApplicationContextMask][EngineFilter]")
{
	FConditions Example =
		(TColumn<FTestColumnA>() || TColumn<FTestColumnB>()) &&
		(TColumn<FTestColumnC>() || TColumn<FTestColumnD>()) &&
		(TColumn<FTestColumnE>() || TColumn<FTestColumnF>());
	
	CHECK(Example.MinimumColumnMatchRequired() == 3); 
	
	CHECK(TestMatching(Example, { TColumn<FTestColumnA>() }, false));
	CHECK(TestMatching(Example, { TColumn<FTestColumnA>(), TColumn<FTestColumnC>() }, false));
	
	CHECK(TestMatching(Example, { TColumn<FTestColumnA>(), TColumn<FTestColumnC>(), TColumn<FTestColumnE>() }, true));
	CHECK(TestMatching(Example, { TColumn<FTestColumnB>(), TColumn<FTestColumnC>(), TColumn<FTestColumnE>() }, true));
	
	CHECK(TestMatching(Example, { TColumn<FTestColumnA>(), TColumn<FTestColumnC>(), TColumn<FTestColumnG>() }, false));
	CHECK(TestMatching(Example, { TColumn<FTestColumnA>(), TColumn<FTestColumnG>(), TColumn<FTestColumnD>() }, false));
	CHECK(TestMatching(Example, { TColumn<FTestColumnG>(), TColumn<FTestColumnC>(), TColumn<FTestColumnD>() }, false));
}

TEST_CASE_NAMED(FTypedElementQueryConditions6, "Editor::DataStorage::QueryBuilder::FTypedElementQueryConditions ((A || B) && (C || D)) || (E && F)", "[ApplicationContextMask][EngineFilter]")
{
	FConditions Example =
		(
			(TColumn<FTestColumnA>() || TColumn<FTestColumnB>()) &&
			(TColumn<FTestColumnC>() || TColumn<FTestColumnD>())
		) ||
		(TColumn<FTestColumnE>() && TColumn<FTestColumnF>());
	
	CHECK(Example.MinimumColumnMatchRequired() == 2);

	CHECK(TestMatching(Example, { TColumn<FTestColumnA>() }, false));
	CHECK(TestMatching(Example, { TColumn<FTestColumnA>(), TColumn<FTestColumnC>() }, true));

	CHECK(TestMatching(Example, { TColumn<FTestColumnE>(), TColumn<FTestColumnF>() }, true));
	CHECK(TestMatching(Example, { TColumn<FTestColumnG>() }, false));
}

TEST_CASE_NAMED(FTypedElementQueryConditions7, "Editor::DataStorage::QueryBuilder::FTypedElementQueryConditions (A && B) || (C && D) || (E && F)", "[ApplicationContextMask][EngineFilter]")
{
	FConditions Example =
		(TColumn<FTestColumnA>() && TColumn<FTestColumnB>()) ||
		(TColumn<FTestColumnC>() && TColumn<FTestColumnD>()) ||
		(TColumn<FTestColumnE>() && TColumn<FTestColumnF>());

	CHECK(Example.MinimumColumnMatchRequired() == 2);

	CHECK(TestMatching(Example, { TColumn<FTestColumnA>() }, false));
	CHECK(TestMatching(Example, { TColumn<FTestColumnA>(), TColumn<FTestColumnB>() }, true));
	CHECK(TestMatching(Example, { TColumn<FTestColumnC>(), TColumn<FTestColumnD>() }, true));
	CHECK(TestMatching(Example, { TColumn<FTestColumnE>(), TColumn<FTestColumnF>() }, true));

	CHECK(TestMatching(Example, { TColumn<FTestColumnC>(), TColumn<FTestColumnD>(), TColumn<FTestColumnE>(), TColumn<FTestColumnF>() }, true));


	CHECK(TestMatching(Example, { TColumn<FTestColumnE>() }, false));
	CHECK(TestMatching(Example, { TColumn<FTestColumnG>() }, false));
}

TEST_CASE_NAMED(FTypedElementQueryConditions_MultiMatch, "Editor::DataStorage::QueryBuilder::FTypedElementQueryConditions_MultiMatch", "[ApplicationContextMask][EngineFilter]")
{
	FConditions Example =
		(TColumn<FTestColumnA>() || TColumn<FTestColumnB>()) &&
		(TColumn<FTestColumnC>() || TColumn<FTestColumnD>()) &&
		(TColumn<FTestColumnE>() || TColumn<FTestColumnF>());

	CHECK(TestMatching(Example, 
		{ 
			TColumn<FTestColumnA>(), 
			TColumn<FTestColumnB>(), 
			TColumn<FTestColumnC>(), 
			TColumn<FTestColumnD>(),
			TColumn<FTestColumnE>(),
			TColumn<FTestColumnF>()
		}, true));

	CHECK(TestMatching(Example,
		{
			TColumn<FTestColumnA>(),
			TColumn<FTestColumnC>(),
			TColumn<FTestColumnE>(),
			TColumn<FTestColumnG>()
		}, true));
}

TEST_CASE_NAMED(FTypedElementQueryConditions_Sorted, "Editor::DataStorage::QueryBuilder::FTypedElementQueryConditions_Sorted", "[ApplicationContextMask][EngineFilter]")
{
	FConditions Example =
		(TColumn<FTestColumnA>() && TColumn<FTestColumnB>()) ||
		(TColumn<FTestColumnC>() && TColumn<FTestColumnD>()) ||
		(TColumn<FTestColumnE>() && TColumn<FTestColumnF>());

	CHECK(Example.MinimumColumnMatchRequired() == 2);

	CHECK(TestMatching(Example, { TColumn<FTestColumnA>(), TColumn<FTestColumnB>() }, true, true));
	CHECK(TestMatching(Example, { TColumn<FTestColumnC>(), TColumn<FTestColumnD>() }, true, true));
	CHECK(TestMatching(Example, { TColumn<FTestColumnE>(), TColumn<FTestColumnF>() }, true, true));

	CHECK(TestMatching(Example, { TColumn<FTestColumnC>(), TColumn<FTestColumnD>(), TColumn<FTestColumnE>(), TColumn<FTestColumnF>() }, true, true));
}
} // namespace UE::Editor::DataStorage::Tests

#endif // WITH_TESTS
