// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "Elements/Framework/TypedElementQueryCapabilities.h"
#include "Elements/Framework/TypedElementQueryContextImplementation.h"
#include "Elements/Framework/TypedElementQueryFunctionArgumentTypes.h"
#include "Templates/FunctionFwd.h"

namespace UE::Editor::DataStorage::Queries
{
	//
	// Composite the environment. This will hold all the function implementations and the calls to set them.
	//

	struct FMockContextEnvironment
	{
#define Concat2(a, b) a##b
#define Concat(a, b) Concat2(a, b)

#define ArgTypeName(Type, Name) Type Name
#define ArgName(Type, Name) Name
#define MockFuncVar Concat(MockFunction_, __LINE__)

#define CapabilityStart(Capability, Flags)\
		friend struct F##Capability##Mock;

#define Function0(Capability, Return, Function) \
		private: \
			TFunction<Return ()> MockFuncVar; \
		public: \
			void Assign_##Function(TFunction<Return ()> Callback) \
			{ \
				MockFuncVar = MoveTemp(Callback); \
			}

#define Function1(Capability, Return, Function, Arg1) \
		private: \
			TFunction<Return (ArgTypeName Arg1 )> MockFuncVar; \
		public: \
			void Assign_##Function(TFunction<Return (ArgTypeName Arg1 )> Callback) \
			{ \
				MockFuncVar = MoveTemp(Callback); \
			}

#define Function2(Capability, Return, Function, Arg1, Arg2) \
		private: \
			TFunction<Return (ArgTypeName Arg1, ArgTypeName Arg2 )> MockFuncVar; \
		public: \
			void Assign_##Function(TFunction<Return (ArgTypeName Arg1, ArgTypeName Arg2 )> Callback) \
			{ \
				MockFuncVar = MoveTemp(Callback); \
			}

#define Function3(Capability, Return, Function, Arg1, Arg2, Arg3) \
		private: \
			TFunction<Return (ArgTypeName Arg1, ArgTypeName Arg2, ArgTypeName Arg3 )> MockFuncVar; \
		public: \
			void Assign_##Function(TFunction<Return (ArgTypeName Arg1, ArgTypeName Arg2, ArgTypeName Arg3 )> Callback) \
			{ \
				MockFuncVar = MoveTemp(Callback); \
			}

#define Function4(Capability, Return, Function, Arg1, Arg2, Arg3, Arg4) \
		private: \
			TFunction<Return (ArgTypeName Arg1, ArgTypeName Arg2, ArgTypeName Arg3, ArgTypeName Arg4 )> MockFuncVar; \
		public: \
			void Assign_##Function(TFunction<Return (ArgTypeName Arg1, ArgTypeName Arg2, ArgTypeName Arg3, ArgTypeName Arg4 )> Callback) \
			{ \
				MockFuncVar = MoveTemp(Callback); \
			}

#define ConstFunction0(Capability, Return, Function) \
		private: \
			TFunction<Return ()> MockFuncVar; \
		public: \
			void Assign_##Function##_Const(TFunction<Return ()> Callback) \
			{ \
				MockFuncVar = MoveTemp(Callback); \
			}

#define ConstFunction1(Capability, Return, Function, Arg1) \
		private: \
			TFunction<Return (ArgTypeName Arg1 )> MockFuncVar; \
		public: \
			void Assign_##Function##_Const(TFunction<Return (ArgTypeName Arg1 )> Callback) \
			{ \
				MockFuncVar = MoveTemp(Callback); \
			}

#define ConstFunction2(Capability, Return, Function, Arg1, Arg2) \
		private: \
			TFunction<Return (ArgTypeName Arg1, ArgTypeName Arg2 )> MockFuncVar; \
		public: \
			void Assign_##Function##_Const(TFunction<Return (ArgTypeName Arg1, ArgTypeName Arg2 )> Callback) \
			{ \
				MockFuncVar = MoveTemp(Callback); \
			}

#define ConstFunction3(Capability, Return, Function, Arg1, Arg2, Arg3) \
		private: \
			TFunction<Return (ArgTypeName Arg1, ArgTypeName Arg2, ArgTypeName Arg3 )> MockFuncVar; \
		public: \
			void Assign_##Function##_Const(TFunction<Return (ArgTypeName Arg1, ArgTypeName Arg2, ArgTypeName Arg3 )> Callback) \
			{ \
				MockFuncVar = MoveTemp(Callback); \
			}

#define ConstFunction4(Capability, Return, Function, Arg1, Arg2, Arg3, Arg4) \
		private: \
			TFunction<Return (ArgTypeName Arg1, ArgTypeName Arg2, ArgTypeName Arg3, ArgTypeName Arg4 )> MockFuncVar; \
		public: \
			void Assign_##Function##_Const(TFunction<Return (ArgTypeName Arg1, ArgTypeName Arg2, ArgTypeName Arg3, ArgTypeName Arg4 )> Callback) \
			{ \
				MockFuncVar = MoveTemp(Callback); \
			}

#define CapabilityEnd(Capability)

#define DeprecatedFunction(Version, Msg)

#include "Elements/Framework/TypedElementQueryCapabilities.inl"

#undef CapabilityStart
#undef DeprecatedFunction
#undef Function0
#undef Function1
#undef Function2
#undef Function3
#undef Function4
#undef ConstFunction0
#undef ConstFunction1
#undef ConstFunction2
#undef ConstFunction3
#undef ConstFunction4
#undef CapabilityEnd
#undef Concat2
#undef Concat
#undef ArgTypeName
#undef ArgName
#undef MockFunction
#undef MockFuncVar
	};

	//
	// Create a mock implementation for each capability. These will calls the functions users can provide in the environment.
	//

#define Concat2(a, b) a##b
#define Concat(a, b) Concat2(a, b)

#define ArgTypeName(Type, Name) Type Name
#define ArgName(Type, Name) Name
#define ForwardArg(Type, Name) Forward< Type >( Name )
#define MockFuncVar Concat(Environment.MockFunction_, __LINE__)

#define CapabilityStart(Capability, Flags) \
		struct F##Capability##Mock : ImplementsContextCapability< Capability > \
		{

#define Function0(Capability, Return, Function) \
			static Return Function (FMockContextEnvironment& Environment) \
			{ \
				checkf(MockFuncVar, TEXT( "Context function " #Function " in capability " #Capability " was unexpectedly called." )); \
				return MockFuncVar ? MockFuncVar () : UE::MakeDefault< Return >(); \
			}
		
#define Function1(Capability, Return, Function, Arg1) \
			static Return Function (FMockContextEnvironment& Environment, ArgTypeName Arg1) \
			{ \
				checkf(MockFuncVar, TEXT( "Context function " #Function " in capability " #Capability " was unexpectedly called." )); \
				return MockFuncVar ? MockFuncVar (ForwardArg Arg1) : UE::MakeDefault< Return >(); \
			}

#define Function2(Capability, Return, Function, Arg1, Arg2) \
			static Return Function (FMockContextEnvironment& Environment, ArgTypeName Arg1, ArgTypeName Arg2) \
			{ \
				checkf(MockFuncVar, TEXT( "Context function " #Function " in capability " #Capability " was unexpectedly called." )); \
				return MockFuncVar ? MockFuncVar(ForwardArg Arg1, ForwardArg Arg2) : UE::MakeDefault< Return >(); \
			}

#define Function3(Capability, Return, Function, Arg1, Arg2, Arg3) \
			static Return Function (FMockContextEnvironment& Environment, ArgTypeName Arg1, ArgTypeName Arg2, ArgTypeName Arg3) \
			{ \
				checkf(MockFuncVar, TEXT( "Context function " #Function " in capability " #Capability " was unexpectedly called." )); \
				return MockFuncVar ? MockFuncVar(ForwardArg Arg1, ForwardArg Arg2, ForwardArg Arg3) : UE::MakeDefault< Return >(); \
			}

#define Function4(Capability, Return, Function, Arg1, Arg2, Arg3, Arg4) \
			static Return Function (FMockContextEnvironment& Environment, ArgTypeName Arg1, ArgTypeName Arg2, ArgTypeName Arg3, ArgTypeName Arg4) \
			{ \
				checkf(MockFuncVar, TEXT( "Context function " #Function " in capability " #Capability " was unexpectedly called." )); \
				return MockFuncVar ? MockFuncVar(ForwardArg Arg1, ForwardArg Arg2, ForwardArg Arg3, ForwardArg Arg4) : UE::MakeDefault< Return >(); \
			}

#define ConstFunction0(Capability, Return, Function) \
			static Return Function (const FMockContextEnvironment& Environment) \
			{ \
				checkf(MockFuncVar, TEXT( "Context function " #Function " in capability " #Capability " was unexpectedly called." )); \
				return MockFuncVar ? MockFuncVar () : UE::MakeDefault< Return >(); \
			}

#define ConstFunction1(Capability, Return, Function, Arg1) \
			static Return Function (const FMockContextEnvironment& Environment, ArgTypeName Arg1) \
			{ \
				checkf(MockFuncVar, TEXT( "Context function " #Function " in capability " #Capability " was unexpectedly called." )); \
				return MockFuncVar ? MockFuncVar (ForwardArg Arg1) : UE::MakeDefault< Return >(); \
			}

#define ConstFunction2(Capability, Return, Function, Arg1, Arg2) \
			static Return Function (const FMockContextEnvironment& Environment, ArgTypeName Arg1, ArgTypeName Arg2) \
			{ \
				checkf(MockFuncVar, TEXT( "Context function " #Function " in capability " #Capability " was unexpectedly called." )); \
				return MockFuncVar ? MockFuncVar(ForwardArg Arg1, ForwardArg Arg2) : UE::MakeDefault< Return >(); \
			}

#define ConstFunction3(Capability, Return, Function, Arg1, Arg2, Arg3) \
			static Return Function (const FMockContextEnvironment& Environment, ArgTypeName Arg1, ArgTypeName Arg2, ArgTypeName Arg3) \
			{ \
				checkf(MockFuncVar, TEXT( "Context function " #Function " in capability " #Capability " was unexpectedly called." )); \
				return MockFuncVar ? MockFuncVar(ForwardArg Arg1, ForwardArg Arg2, ForwardArg Arg3) : UE::MakeDefault< Return >(); \
			}

#define ConstFunction4(Capability, Return, Function, Arg1, Arg2, Arg3, Arg4) \
			static Return Function (const FMockContextEnvironment& Environment, ArgTypeName Arg1, ArgTypeName Arg2, ArgTypeName Arg3, ArgTypeName Arg4) \
			{ \
				checkf(MockFuncVar, TEXT( "Context function " #Function " in capability " #Capability " was unexpectedly called." )); \
				return MockFuncVar ? MockFuncVar(ForwardArg Arg1, ForwardArg Arg2, ForwardArg Arg3, ForwardArg Arg4) : UE::MakeDefault< Return >(); \
			}

#define CapabilityEnd(Capability) \
		};

#define DeprecatedFunction(Version, Msg) UE_DEPRECATED(Version, Msg)

#include "Elements/Framework/TypedElementQueryCapabilities.inl"

#undef CapabilityStart
#undef DeprecatedFunction
#undef Function0
#undef Function1
#undef Function2
#undef Function3
#undef Function4
#undef ConstFunction0
#undef ConstFunction1
#undef ConstFunction2
#undef ConstFunction3
#undef ConstFunction4
#undef CapabilityEnd
#undef Concat2
#undef Concat
#undef ArgTypeName
#undef ArgName
#undef ForwardArg
#undef MockFunction
#undef MockFuncVar

	//
	// Combine all mock implementations together into a final implementation.
	//

using QueryContextMock = TQueryContextImpl<false, FMockContextEnvironment

#define Function0(Capability, Return, Function)
#define Function1(Capability, Return, Function, Arg1)
#define Function2(Capability, Return, Function, Arg1, Arg2)
#define Function3(Capability, Return, Function, Arg1, Arg2, Arg3)
#define Function4(Capability, Return, Function, Arg1, Arg2, Arg3, Arg4)

#define ConstFunction0(Capability, Return, Function)
#define ConstFunction1(Capability, Return, Function, Arg1)
#define ConstFunction2(Capability, Return, Function, Arg1, Arg2)
#define ConstFunction3(Capability, Return, Function, Arg1, Arg2, Arg3)
#define ConstFunction4(Capability, Return, Function, Arg1, Arg2, Arg3, Arg4)

#define DeprecatedFunction(Version, Msg)

#define CapabilityStart(Capability, Flags) , F##Capability##Mock
#define CapabilityEnd(Capability)

#include "Elements/Framework/TypedElementQueryCapabilities.inl"
	>;

#undef Function0
#undef Function1
#undef Function2
#undef Function3
#undef Function4
#undef ConstFunction0
#undef ConstFunction1
#undef ConstFunction2
#undef ConstFunction3
#undef ConstFunction4
#undef DeprecatedFunction
#undef CapabilityStart
#undef CapabilityEnd

} // namespace UE::Editor::DataStorage::Queries
