// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Containers/StaticArray.h"
#include "Elements/Framework/TypedElementQueryContext.h"
#include "Elements/Framework/TypedElementQueryContextMock.h"
#include "Elements/Framework/TypedElementQueryFunctions.h"
#include "Elements/Framework/TypedElementTestColumns.h"
#include "Math/UnrealMathUtility.h"
#include "Tests/TestHarnessAdapter.h"

namespace UE::Editor::DataStorage::Queries::Tests
{
	struct FBoolAndResultAccumulator : TResult<bool>
	{
		bool Value = true;

		virtual void Add(RowHandle Row, bool ResultValue) override
		{
			Value = Value && ResultValue;
		}
	};

	struct FBoolOrResultAccumulator : TResult<bool>
	{
		bool Value = false;

		virtual void Add(RowHandle Row, bool ResultValue) override
		{
			Value = Value || ResultValue;
		}
	};

	struct FInt32ResultAccumulator : TResult<int32>
	{
		int32 Value = 0;

		virtual void Add(RowHandle Row, int32 ResultValue) override
		{
			Value += ResultValue;
		}
	};
	
	struct FTestQueryFunctionResponse : IQueryFunctionResponse
	{
		virtual ~FTestQueryFunctionResponse() override = default;

		virtual FRowHandleArrayView GetBatchRows() const override
		{
			CHECK_MESSAGE(TEXT("Row count was not set in test."), !FakeRows.IsEmpty());
			return FRowHandleArrayView(FakeRows.GetData(), FakeRows.Num(), FRowHandleArrayView::EFlags::None);
		}

		virtual void GetConstColumns(TArrayView<const void*> ColumnsData, TConstArrayView<const UScriptStruct*> ColumnTypes) const override
		{
			CHECK_MESSAGE(TEXT("Row count was not set in test."), !FakeRows.IsEmpty());
			for (int32 Index = 0, Num = ColumnTypes.Num(); Index < Num; ++Index)
			{
				const void* const* Data = ConstColumns.Find(ColumnTypes[Index]);
				if (Data == nullptr)
				{
					Data = MutableColumns.Find(ColumnTypes[Index]);
				}
				CHECK_MESSAGE(TEXT("Unable to retrieve const column data."), Data != nullptr && *Data != nullptr);
				if (Data != nullptr && *Data != nullptr)
				{
					ColumnsData[Index] = *Data;
				}
				else
				{
					ColumnsData[Index] = nullptr;
				}
			}
		}

		virtual void GetMutableColumns(TArrayView<void*> ColumnsData, TConstArrayView<const UScriptStruct*> ColumnTypes) override
		{
			CHECK_MESSAGE(TEXT("Row count was not set in test."), !FakeRows.IsEmpty());
			for (int32 Index = 0, Num = ColumnTypes.Num(); Index < Num; ++Index)
			{
				void** Data = MutableColumns.Find(ColumnTypes[Index]);
				CHECK_MESSAGE(TEXT("Unable to retrieve mutable column data."), Data != nullptr && *Data != nullptr);
				if (Data != nullptr && *Data != nullptr)
				{
					ColumnsData[Index] = *Data;
				}
				else
				{
					ColumnsData[Index] = nullptr;
				}
			}
		}

		virtual void SetCurrentRow(RowHandle Row) override
		{
			CurrentRow = Row;
		}

		void SetRowCount(int32 InRowCount)
		{
			CHECK_MESSAGE(TEXT("Row count was not set in test."), InRowCount > 0);
			FakeRows.Reset();
			FakeRows.Reserve(InRowCount);
			for (RowHandle Row = 0; Row < InRowCount; ++Row)
			{
				FakeRows.Add(Row);
			}
		}

		template<TColumnType... ColumnType>
		void SetConstColumns(const ColumnType*... Columns)
		{
			(ConstColumns.Add(ColumnType::StaticStruct(), Columns), ...);
		}

		template<TColumnType... ColumnType>
		void SetMutableColumns(ColumnType*... Columns)
		{
			(MutableColumns.Add(ColumnType::StaticStruct(), Columns), ...);
		}

		TMap<const UScriptStruct*, const void*> ConstColumns;
		TMap<const UScriptStruct*, void*> MutableColumns;
		TArray<RowHandle> FakeRows;
		RowHandle CurrentRow = InvalidRowHandle;
	};

	TEST_CASE_NAMED(TQueryFunction_Tests, "Editor::DataStorage::Queries::Query Function (TQueryFunction)", "[ApplicationContextMask][EngineFilter]")
	{
		using namespace UE::Editor::DataStorage;
		using namespace UE::Editor::DataStorage::Queries;
		
		SECTION("Empty function (void)")
		{
			TQueryFunction<void> Result = BuildQueryFunction<void>([](){});
			
			CHECK_MESSAGE(TEXT("Empty query still has capabilities."), Result.CapabilityMask == 0);
			CHECK_MESSAGE(TEXT("Empty query still has const columns."), Result.ConstColumnTypes.IsEmpty());
			CHECK_MESSAGE(TEXT("Empty query still has mutable columns."), Result.MutableColumnTypes.IsEmpty());
		}

		SECTION("Empty function (bool)")
		{
			TQueryFunction<bool> Result = BuildQueryFunction<bool>([]() { return false; });

			CHECK_MESSAGE(TEXT("Empty query still has capabilities."), Result.CapabilityMask == 0);
			CHECK_MESSAGE(TEXT("Empty query still has const columns."), Result.ConstColumnTypes.IsEmpty());
			CHECK_MESSAGE(TEXT("Empty query still has mutable columns."), Result.MutableColumnTypes.IsEmpty());
		}

		SECTION("Empty function with capture")
		{
			bool bResult = false;
			TQueryFunction<bool> Result = BuildQueryFunction<bool>([&bResult]() { return bResult; });

			CHECK_MESSAGE(TEXT("Empty query still has capabilities."), Result.CapabilityMask == 0);
			CHECK_MESSAGE(TEXT("Empty query still has const columns."), Result.ConstColumnTypes.IsEmpty());
			CHECK_MESSAGE(TEXT("Empty query still has mutable columns."), Result.MutableColumnTypes.IsEmpty());
		}

		SECTION("Function with context")
		{
			TQueryFunction<void> Result = BuildQueryFunction<void>([](TQueryContext<SingleRowInfo> Context) {});

			CHECK_MESSAGE(TEXT("Expected exactly one capability."), FMath::CountBits64(Result.CapabilityMask) == 1);
			CHECK_MESSAGE(TEXT("Query with only context shouldn't have const columns."), Result.ConstColumnTypes.IsEmpty());
			CHECK_MESSAGE(TEXT("Query with only context shouldn't have mutable columns."), Result.MutableColumnTypes.IsEmpty());
		}

		SECTION("Function with columns")
		{
			TQueryFunction<void> Result = BuildQueryFunction<void>(
				[](FTestColumnInt& ColumnA, const FTestColumnString& ColumnB)
				{
				});

			CHECK_MESSAGE(TEXT("Query with only columns shouldn't have capabilities."), Result.CapabilityMask == 0);
			CHECK_MESSAGE(TEXT("Expected exactly one const column."), Result.ConstColumnTypes.Num() == 1);
			CHECK_MESSAGE(TEXT("Expected exactly one mutable column."), Result.MutableColumnTypes.Num() == 1);
		}

		SECTION("Function with context and columns")
		{
			TQueryFunction<void> Result = BuildQueryFunction<void>(
				[](TQueryContext<SingleRowInfo> Context, FTestColumnInt& ColumnA, const FTestColumnString& ColumnB)
				{
				});

			CHECK_MESSAGE(TEXT("Expected exactly one capability."), FMath::CountBits64(Result.CapabilityMask) == 1);
			CHECK_MESSAGE(TEXT("Expected exactly one const column."), Result.ConstColumnTypes.Num() == 1);
			CHECK_MESSAGE(TEXT("Expected exactly one mutable column."), Result.MutableColumnTypes.Num() == 1);

			CHECK_MESSAGE(TEXT("Incorrect const column set."), Result.ConstColumnTypes[0] == FTestColumnString::StaticStruct());
			CHECK_MESSAGE(TEXT("Incorrect mutable column set."), Result.MutableColumnTypes[0] == FTestColumnInt::StaticStruct());
		}

		SECTION("Function with context and columns at random location")
		{
			TQueryFunction<void> Result = BuildQueryFunction<void>(
				[](FTestColumnInt& ColumnA, TQueryContext<SingleRowInfo> Context, const FTestColumnString& ColumnB)
				{
				});

			CHECK_MESSAGE(TEXT("Expected exactly one capability."), FMath::CountBits64(Result.CapabilityMask) == 1);
			CHECK_MESSAGE(TEXT("Expected exactly one const column."), Result.ConstColumnTypes.Num() == 1);
			CHECK_MESSAGE(TEXT("Expected exactly one mutable column."), Result.MutableColumnTypes.Num() == 1);

			CHECK_MESSAGE(TEXT("Incorrect const column set."), Result.ConstColumnTypes[0] == FTestColumnString::StaticStruct());
			CHECK_MESSAGE(TEXT("Incorrect mutable column set."), Result.MutableColumnTypes[0] == FTestColumnInt::StaticStruct());
		}

		SECTION("Function with context and column batch")
		{
			TQueryFunction<void> Result = BuildQueryFunction<void>(
				[](TQueryContext<RowBatchInfo> Context, TBatch<FTestColumnInt> ColumnsA, TConstBatch<FTestColumnString> ColumnB)
				{
				});

			CHECK_MESSAGE(TEXT("Expected exactly one capability."), FMath::CountBits64(Result.CapabilityMask) == 1);
			CHECK_MESSAGE(TEXT("Expected exactly one const column."), Result.ConstColumnTypes.Num() == 1);
			CHECK_MESSAGE(TEXT("Expected exactly one mutable column."), Result.MutableColumnTypes.Num() == 1);

			CHECK_MESSAGE(TEXT("Incorrect const column set."), Result.ConstColumnTypes[0] == FTestColumnString::StaticStruct());
			CHECK_MESSAGE(TEXT("Incorrect mutable column set."), Result.MutableColumnTypes[0] == FTestColumnInt::StaticStruct());
		}

		SECTION("Check context compatibility without arguments")
		{
			TQueryFunction<void> Function = BuildQueryFunction<void>([](){});

			FMockContextEnvironment Environment;
			QueryContextMock Context(Environment);
			bool bResult = Context.CheckCompatibility(Function.CapabilityMask);
			CHECK_MESSAGE(TEXT("Function without context incorrectly found to not be matching."), bResult);
		}

		SECTION("Check context compatibility with context")
		{
			TQueryFunction<void> Function = BuildQueryFunction<void>([](TQueryContext<SingleRowInfo> Context) {});

			FMockContextEnvironment Environment;
			QueryContextMock Context(Environment);
			bool bResult = Context.CheckCompatibility(Function.CapabilityMask);
			CHECK_MESSAGE(TEXT("Function with context incorrectly found to not be matching."), bResult);
		}

		SECTION("Call without arguments")
		{
			TQueryFunction<bool> Function = BuildQueryFunction<bool>(
				[]()
				{
					return true;
				});
			
			FMockContextEnvironment Environment;
			QueryContextMock Context(Environment);
			FTestQueryFunctionResponse Response;
			Response.SetRowCount(1);
			
			FBoolOrResultAccumulator Result;
			Function.Call<EFunctionCallConfig::VerifyColumns>(Result, Context, Response);
			CHECK_MESSAGE(TEXT("Query function wasn't called."), Result.Value);
		}

		SECTION("Call setting column value")
		{
			TQueryFunction<void> Function = BuildQueryFunction<void>(
				[](FTestColumnString& Column)
				{
					Column.TestString = TEXT("Callback");
				});

			FMockContextEnvironment Environment;
			QueryContextMock Context(Environment);
			FTestColumnString Column;
			Column.TestString = TEXT("Not set");
			
			FTestQueryFunctionResponse Response;
			Response.SetRowCount(1);
			Response.SetMutableColumns(&Column);
			Function.Call<EFunctionCallConfig::VerifyColumns>(Context, Response);
			
			CHECK_MESSAGE(TEXT("Query function wasn't called."), Column.TestString == TEXT("Callback"));
		}

		SECTION("Call setting const and mutable columns")
		{
			TQueryFunction<void> Function = BuildQueryFunction<void>(
				[](FTestColumnString& ColumnA, const FTestColumnInt& ColumnB)
				{
					ColumnA.TestString = FString::Printf(TEXT("Test: %i"), ColumnB.TestInt);
				});

			FMockContextEnvironment Environment;
			QueryContextMock Context(Environment);
			FTestColumnString ColumnA;
			FTestColumnInt ColumnB;
			ColumnB.TestInt = 42;

			FTestQueryFunctionResponse Response;
			Response.SetRowCount(1);
			Response.SetMutableColumns(&ColumnA);
			Response.SetConstColumns(&ColumnB);
			Function.Call<EFunctionCallConfig::VerifyColumns>(Context, Response);

			CHECK_MESSAGE(TEXT("Query function wasn't called."), ColumnA.TestString == TEXT("Test: 42"));
		}

		SECTION("Call setting const and mutable batch columns")
		{
			TQueryFunction<void> Function = BuildQueryFunction<void>(
				[](TQueryContext<RowBatchInfo> Context, TBatch<FTestColumnString> ColumnsA, TConstBatch<FTestColumnInt> ColumnsB)
				{
					Context.ForEachRow([](RowHandle Row, FTestColumnString& ColumnA, const FTestColumnInt& ColumnB)
						{
							ColumnA.TestString = FString::Printf(TEXT("Test: %i"), ColumnB.TestInt);
						}, ColumnsA, ColumnsB);
				});

			static constexpr uint32 RowCount = 2;
			TStaticArray<RowHandle, 2> Rows = { 0, 1 };
			FRowHandleArrayView RowsView(Rows.GetData(), Rows.Num(), 
				FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique);

			FMockContextEnvironment Environment;
			QueryContextMock Context(Environment);
			Environment.Assign_GetBatchRowCount_Const([]() { return RowCount; });
			Environment.Assign_GetBatchRowHandles_Const([RowsView](){ return RowsView; });
			FTestColumnString ColumnsA[RowCount];
			FTestColumnInt ColumnsB[] =
			{
				FTestColumnInt{ .TestInt = 42 },
				FTestColumnInt{ .TestInt = 88 }
			};

			FTestQueryFunctionResponse Response;
			Response.SetRowCount(RowCount);
			Response.SetMutableColumns(ColumnsA);
			Response.SetConstColumns(ColumnsB);
			Function.Call<EFunctionCallConfig::VerifyColumns>(Context, Response);

			CHECK_MESSAGE(TEXT("Query function wasn't called."), ColumnsA[0].TestString == TEXT("Test: 42"));
			CHECK_MESSAGE(TEXT("Query function wasn't called."), ColumnsA[1].TestString == TEXT("Test: 88"));
		}

		SECTION("Call and use context")
		{
			TQueryFunction<void> Function = BuildQueryFunction<void>(
				[](TQueryContext<SingleRowInfo, CurrentTableInfo> Context)
				{
					Context.GetCurrentRow();
					Context.CurrentTableHasColumns<FTestColumnString>();
				});

			FMockContextEnvironment Environment;
			QueryContextMock Context(Environment);

			bool bGetCurrentRowCalled = false;
			Environment.Assign_GetCurrentRow_Const(
				[&bGetCurrentRowCalled]()
				{
					bGetCurrentRowCalled = true;
					return 0;
				});

			bool bCurrentTableHasColumnsCalled = false;
			Environment.Assign_CurrentTableHasColumns_Const(
				[&bCurrentTableHasColumnsCalled](TConstArrayView<const UScriptStruct*>)
				{
					bCurrentTableHasColumnsCalled = true;
					return false;
				});

			FTestQueryFunctionResponse Response;
			Response.SetRowCount(1);
			Function.Call(Context, Response);

			CHECK_MESSAGE(TEXT("GetCurrentRow not called on context."), bGetCurrentRowCalled);
			CHECK_MESSAGE(TEXT("CurrentTableHasColumns not called on context."), bCurrentTableHasColumnsCalled);
		}

		SECTION("Call and check provided row handles")
		{
			constexpr int32 CheckRange = 5;

			TArray<RowHandle> RowCheck;
			RowCheck.Reserve(CheckRange);
			TQueryFunction<void> Function = BuildQueryFunction<void>(
				[&RowCheck](TQueryContext<SingleRowInfo> Context)
				{
					RowCheck.Add(Context.GetCurrentRow());
				});

			FMockContextEnvironment Environment;
			QueryContextMock Context(Environment);
			FTestQueryFunctionResponse Response;

			Environment.Assign_GetCurrentRow_Const(
				[&Response]()
				{
					return Response.CurrentRow;
				});

			Response.SetRowCount(CheckRange);
			Function.Call(Context, Response);

			CHECK_MESSAGE(TEXT("Not all Rows were added."), RowCheck.Num() == CheckRange);
			for (int32 FakeRow = 0; FakeRow < CheckRange; ++FakeRow)
			{
				CHECK_MESSAGE(TEXT("One of the rows wasn't called or calle with the wrong row handle."), RowCheck[FakeRow] == FakeRow);
			}
		}

		SECTION("Call with result from callback")
		{
			TQueryFunction<bool> Function = BuildQueryFunction<bool>([]() { return true; });

			FMockContextEnvironment Environment;
			QueryContextMock Context(Environment);
			FTestColumnString Column;

			FBoolOrResultAccumulator Result;

			FTestQueryFunctionResponse Response;
			Response.SetRowCount(1);
			Response.SetMutableColumns(&Column);
			Function.Call<EFunctionCallConfig::VerifyColumns>(Result, Context, Response);

			CHECK_MESSAGE(TEXT("Result wasn't passed back in."), Result.Value);
		}

		SECTION("Call with result through TResult")
		{
			TQueryFunction<bool> Function = BuildQueryFunction<bool>(
				[](TQueryContext<SingleRowInfo> Context, TResult<bool>& Result)
				{ 
					Result.Add(Context.GetCurrentRow(), true);
				});

			FMockContextEnvironment Environment;
			Environment.Assign_GetCurrentRow_Const([]() { return RowHandle(1); });
			QueryContextMock Context(Environment);
			FTestColumnString Column;

			FBoolOrResultAccumulator Result;

			FTestQueryFunctionResponse Response;
			Response.SetRowCount(1);
			Response.SetMutableColumns(&Column);
			Function.Call<EFunctionCallConfig::VerifyColumns>(Result, Context, Response);

			CHECK_MESSAGE(TEXT("Result wasn't passed back in."), Result.Value);
		}

		SECTION("Call with batching results.")
		{
			TQueryFunction<int32> Function = BuildQueryFunction<int32>(
				[](TQueryContext<RowBatchInfo> Context, TResult<int32>& Result, TConstBatch<FTestColumnInt> Columns)
				{
					Context.ForEachRow([&Result](RowHandle Row, const FTestColumnInt& Column)
						{
							Result.Add(Row, Column.TestInt);
						}, Columns);
				});

			static constexpr int32 RowCount = 4;
			TStaticArray<RowHandle, 4> Rows = { 0, 1, 2, 3 };
			FRowHandleArrayView RowsView(Rows.GetData(), Rows.Num(),
				FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique);

			FMockContextEnvironment Environment;
			QueryContextMock Context(Environment);
			Environment.Assign_GetBatchRowCount_Const([]() { return RowCount; });
			Environment.Assign_GetBatchRowHandles_Const([RowsView]() { return RowsView; });

			FTestColumnInt Columns[] =
			{
				FTestColumnInt{.TestInt = 1 },
				FTestColumnInt{.TestInt = 2 },
				FTestColumnInt{.TestInt = 4 },
				FTestColumnInt{.TestInt = 8 }
			};

			FInt32ResultAccumulator Result;

			FTestQueryFunctionResponse Response;
			Response.SetRowCount(RowCount);
			Response.SetConstColumns(Columns);
			Function.Call<EFunctionCallConfig::VerifyColumns>(Result, Context, Response);

			CHECK_MESSAGE(TEXT("Result wasn't passed back in."), Result.Value == 1 + 2 + 4 + 8);
		}

		SECTION("Call and break on first row")
		{
			int32 LastValue = 0;
			TQueryFunction<void> Function = BuildQueryFunction<void>(
				[&LastValue](EFlowControl& Flow, const FTestColumnInt& Column)
				{
					LastValue = Column.TestInt;
					Flow = EFlowControl::Break;
				});

			FMockContextEnvironment Environment;
			QueryContextMock Context(Environment);

			FTestColumnInt Columns[] =
			{
				FTestColumnInt{.TestInt = 1 },
				FTestColumnInt{.TestInt = 2 }
			};

			FTestQueryFunctionResponse Response;
			Response.SetRowCount(2);
			Response.SetConstColumns(Columns);

			Function.Call(Context, Response);

			CHECK_MESSAGE(TEXT("Iteration didn't stop on the first row."), LastValue == 1);
		}

		SECTION("Forward context")
		{
			TQueryFunction<void> Function = BuildQueryFunction<void>(
				[](TQueryContext<SingleRowInfo, CurrentTableInfo> Context, const FTestColumnInt& Column)
				{
					Context.GetCurrentRow();
					Context.GetCurrentTable();
				});

			FMockContextEnvironment Environment;
			QueryContextMock Context(Environment);

			bool bCurrentRowCalled = false;
			bool bCurrentTableCalled = false;
			Environment.Assign_GetCurrentRow_Const(
				[&bCurrentRowCalled]()
				{
					bCurrentRowCalled = true;
					return 1;
				});
			Environment.Assign_GetCurrentTable_Const(
				[&bCurrentTableCalled]()
				{
					bCurrentTableCalled = true;
					return 1;
				});


			FTestColumnInt Columns[] = { FTestColumnInt{.TestInt = 1 } };

			FTestQueryFunctionResponse Response;
			Response.SetRowCount(1);
			Response.SetConstColumns(Columns);

			// This is an order that would typically happen inside a query callback that calls a utility function.
			TQueryContext<SingleRowInfo, CurrentTableInfo> ContextRequest(Context);
			TForwardingQueryContext<SingleRowInfo> ForwardedContext(ContextRequest);

			Function.Call(ForwardedContext, Response);

			CHECK_MESSAGE(TEXT("GetCurrentRow wasn't called."), bCurrentRowCalled);
			CHECK_MESSAGE(TEXT("GetCurrentTable wasn't called."), bCurrentTableCalled);
		}
	}
} // namespace UE::Editor::DataStorage::Queries::Tests

#endif // WITH_TEST