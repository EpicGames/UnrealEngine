// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/TypedElementTestColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#if WITH_TESTS

#include "DataStorage/Features.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#include "Misc/AutomationTest.h"

namespace UE::Editor::DataStorage::Tests
{
	using namespace UE::Editor::DataStorage::Queries;
	
	BEGIN_DEFINE_SPEC(QueryMergeTestFixture, "Editor.DataStorage.QueryMerge", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		ICoreProvider* TedsInterface = nullptr;
		const FName TestTableName = "TestTable_QueryMergeTest";

		TableHandle TestTable;
		TArray<RowHandle> Rows;
		TArray<QueryHandle> QueryHandles;
		FName Identifier1;
		FName Identifier2;
		FName Identifier3;

		TableHandle RegisterTestTable() const
		{
			const TableHandle Table = TedsInterface->FindTable(TestTableName);
				
			if (Table != InvalidTableHandle)
			{
				return Table;
			}
				
			return TedsInterface->RegisterTable(
			{
				FTestColumnA::StaticStruct(),
				FTestTagColumnA::StaticStruct()
			},
			TestTableName);
		}

		RowHandle CreateTestRow(TableHandle InTableHandle)
		{
			RowHandle Row = TedsInterface->AddRow(InTableHandle);
			Rows.Add(Row);
			return Row;
		}

		QueryHandle RegisterQuery(FQueryDescription&& Query)
		{
			QueryHandle QueryHandle = TedsInterface->RegisterQuery(MoveTemp(Query));
			QueryHandles.Add(QueryHandle);
			return QueryHandle;
		}

		void RegisterTestTableRows()
		{
			RowHandle RowA = CreateTestRow(TestTable);
			TedsInterface->AddColumns<FTestColumnB, FTestColumnC, FTestColumnD>(RowA);
			TedsInterface->AddColumns<FTestTagColumnB, FTestTagColumnC, FTestTagColumnD>(RowA);
			TedsInterface->AddColumn<FTestColumnDynamic>(RowA, Identifier1);
			TedsInterface->AddColumn<FTestColumnDynamic>(RowA, Identifier2);
			TedsInterface->AddColumn<FTestColumnDynamic>(RowA, Identifier3);

			RowHandle RowB = CreateTestRow(TestTable);
			TedsInterface->AddColumns<FTestColumnB, FTestColumnD>(RowB);
			TedsInterface->AddColumns<FTestTagColumnB, FTestTagColumnC>(RowB);
			TedsInterface->AddColumn<FTestColumnDynamic>(RowB, Identifier1);

			RowHandle RowC = CreateTestRow(TestTable);
			TedsInterface->AddColumn<FTestColumnDynamic>(RowC, Identifier2);

			RowHandle RowD = CreateTestRow(TestTable);
			TedsInterface->AddColumns<FTestColumnC, FTestColumnD>(RowD);
			TedsInterface->AddColumns<FTestTagColumnD>(RowD);
			TedsInterface->AddColumn<FTestColumnDynamic>(RowD, Identifier3);

			RowHandle RowE = CreateTestRow(TestTable);
			TedsInterface->AddColumns<FTestColumnB, FTestColumnC, FTestColumnD>(RowE);
			TedsInterface->AddColumn<FTestColumnDynamic>(RowE, Identifier2);
			TedsInterface->AddColumn<FTestColumnDynamic>(RowE, Identifier3);
		}

		TSet<RowHandle> RegisterAndRunQuery(FQueryDescription& Query)
		{
			QueryHandle QueryHandle = RegisterQuery(MoveTemp(Query));
			TSet<RowHandle> OutputRows;
			
			TedsInterface->RunQuery(QueryHandle, CreateDirectQueryCallbackBinding([&OutputRows](const IDirectQueryContext& Context, const RowHandle* RowArray)
			{
				OutputRows.Append(Context.GetRowHandles());
			}));

			return OutputRows;
		}

		void TestQueryMerge_Success(FQueryDescription& Query1, FQueryDescription& Query2)
		{
			// Copy Query1 since the merge operates directly on the first query
			FQueryDescription MergedQuery = Query1;
			FText Error;

			bool bMerged = MergeQueries(MergedQuery, Query2, &Error);
				
			TestTrue(TEXT("Expected successful merge between simple queries"), bMerged);

			TSet<RowHandle> Rows1 = RegisterAndRunQuery(Query1);
			TSet<RowHandle> Rows2 = RegisterAndRunQuery(Query2);
			TSet<RowHandle> RowsMerged = RegisterAndRunQuery(MergedQuery);

			TestTrue(TEXT("Expected Query1 to have valid results"), !Rows1.IsEmpty());
			TestTrue(TEXT("Expected Query2 to have valid results"), !Rows2.IsEmpty());

			TSet<RowHandle> RowsMerged_Expected = Rows1.Intersect(Rows2);
			bool bResultsSame = RowsMerged.Difference(RowsMerged_Expected).IsEmpty();

			TestTrue(TEXT("Expected merge query output to be the same as the intersection of the individual query outputs"), bResultsSame);
		}
	
		void TestQueryMerge_Failure(FQueryDescription& Query1, FQueryDescription& Query2, const FString& InvalidReason)
		{
			// Copy Query1 since the merge operates directly on the first query
			FQueryDescription MergedQuery = Query1;
			FText Error;

			bool bMerged = MergeQueries(MergedQuery, Query2, &Error);

			TestFalse(FString::Printf(TEXT("Expected merge to fail because of %s"), *InvalidReason), bMerged);
		}

	
	END_DEFINE_SPEC(QueryMergeTestFixture)

	void QueryMergeTestFixture::Define()
	{
		BeforeEach([this]()
		{
			Identifier1 = FName("Identifier1");
			Identifier2 = FName("Identifier2");
			Identifier3 = FName("Identifier3");
			TedsInterface = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
			TestTrue("", TedsInterface != nullptr);
			
			TestTable = RegisterTestTable();
			RegisterTestTableRows();
		});
			
		Describe("QueryMerge", [this]
		{
			Describe("ValidMerge", [this]
			{
				Describe("Where()", [this]
				{
					It(TEXT("Simple"), EAsyncExecution::TaskGraphMainTick, [this]()
					{
						FQueryDescription Query1 =
							Select()
							.Where()
								.All<FTestColumnA, FTestTagColumnA>()
							.Compile();

						FQueryDescription Query2 =
							Select()
							.Where()
								.All<FTestColumnB, FTestTagColumnB>()
							.Compile();

						TestQueryMerge_Success(Query1, Query2);
					});
					
					It(TEXT("Complex"), EAsyncExecution::TaskGraphMainTick, [this]()
					{
						FQueryDescription Query1 =
							Select()
							.Where()
								.Any<FTestColumnB, FTestColumnD>()
								.None<FTestColumnC>()
							.Compile();

						FQueryDescription Query2 =
							Select()
							.Where()
								.All<FTestColumnB>()
								.None<FTestTagColumnC>()
							.Compile();

						FQueryDescription MergedQuery = Query1;
						FText Error;

						TestQueryMerge_Success(Query1, Query2);
					});
				});
				
				Describe("Select()", [this]
				{
					It(TEXT("Simple"), EAsyncExecution::TaskGraphMainTick, [this]()
					{
						FQueryDescription Query1 =
							Select()
								.ReadOnly<FTestColumnA>()
							.Compile();

						FQueryDescription Query2 =
							Select()
								.ReadOnly<FTestColumnB>()
							.Compile();

						TestQueryMerge_Success(Query1, Query2);
					});
						
					It(TEXT("Complex"), EAsyncExecution::TaskGraphMainTick, [this]()
					{
						FQueryDescription Query1 =
							Select()
								.ReadOnly<FTestColumnA>()
								.ReadWrite<FTestColumnB>()
							.Compile();

						FQueryDescription Query2 =
							Select()
								.ReadOnly<FTestColumnB>()
								.ReadWrite<FTestColumnD>()
							.Compile();
						
						TestQueryMerge_Success(Query1, Query2);
					});
				});
				
				Describe("FConditions", [this]
                {
					It(TEXT("Simple"), EAsyncExecution::TaskGraphMainTick, [this]()
					{
						FQueryDescription Query1 =
							Select()
							.Where(TColumn<FTestColumnA>() && TColumn<FTestTagColumnA>())
							.Compile();

						FQueryDescription Query2 =
							Select()
							.Where(TColumn<FTestColumnB>() && TColumn<FTestTagColumnB>())
							.Compile();

						TestQueryMerge_Success(Query1, Query2);
					});

					It(TEXT("Complex"), EAsyncExecution::TaskGraphMainTick, [this]()
					{
						FQueryDescription Query1 =
							Select()
							.Where(TColumn<FTestColumnA>() && TColumn<FTestTagColumnA>())
							.Compile();

						FQueryDescription Query2 =
							Select()
							.Where(TColumn<FTestColumnB>() && TColumn<FTestTagColumnB>())
							.Compile();

						TestQueryMerge_Success(Query1, Query2);
					});
					
					It(TEXT("Simple::Dynamic"), EAsyncExecution::TaskGraphMainTick, [this]()
					{
						FQueryDescription Query1 =
							Select()
							.Where(TColumn<FTestColumnDynamic>(Identifier1))
							.Compile();

						FQueryDescription Query2 =
							Select()
							.Where(TColumn<FTestColumnDynamic>(Identifier2))
							.Compile();

						TestQueryMerge_Success(Query1, Query2);
					});
					
					It(TEXT("Complex::Dynamic"), EAsyncExecution::TaskGraphMainTick, [this]()
					{
						FQueryDescription Query1 =
							Select()
							.Where(TColumn<FTestColumnB>() ||  TColumn<FTestColumnDynamic>(Identifier2))
							.Compile();

						FQueryDescription Query2 =
							Select()
							.Where(TColumn<FTestColumnDynamic>(Identifier1) &&  TColumn<FTestColumnDynamic>(Identifier2))
							.Compile();

						TestQueryMerge_Success(Query1, Query2);
					});
                });
                				
				Describe("Combination", [this]
				{
					It(TEXT("Select()+Where()"), EAsyncExecution::TaskGraphMainTick, [this]()
					{
						FQueryDescription Query1 =
							Select()
								.ReadOnly<FTestColumnA>()
							.Where()
								.All<FTestColumnB, FTestTagColumnA>()
							.Compile();

						FQueryDescription Query2 =
							Select()
								.ReadWrite<FTestColumnDynamic>(Identifier1)
						.Where()
							.Any<FTestColumnB, FTestColumnC, FTestColumnD>()
							.Compile();
						
						TestQueryMerge_Success(Query1, Query2);
					});

					It(TEXT("Select()+FConditions"), EAsyncExecution::TaskGraphMainTick, [this]()
					{
						FQueryDescription Query1 =
							Select()
								.ReadOnly<FTestColumnA>()
							.Where(TColumn<FTestTagColumnB>() || TColumn<FTestColumnC>())
							.Compile();

						FQueryDescription Query2 =
							Select()
								.ReadWrite<FTestColumnDynamic>(Identifier1)
							.Where(TColumn<FTestColumnDynamic>(Identifier2) || TColumn<FTestColumnDynamic>(Identifier3))
							.Compile();
						
						TestQueryMerge_Success(Query1, Query2);
					});				
				});
			});
			
			Describe("InvalidMerge", [this]
			{
				It(TEXT("Where()"), EAsyncExecution::TaskGraphMainTick, [this]()
				{
					FQueryDescription Query1 =
							Select()
							.Where()
								.All<FTestColumnA>()
							.Compile();

					FQueryDescription Query2 =
							Select()
							.Where()
								.None<FTestColumnA>()
							.Compile();

					TestQueryMerge_Failure(Query1, Query2, TEXT("Conflicting conditions .All<FTestColumnA>() and .None<FTestColumnA>()"));
				});

				It(TEXT("Where()::Dynamic"), EAsyncExecution::TaskGraphMainTick, [this]()
				{
					FQueryDescription Query1 =
							Select()
							.Where()
								.All<FTestColumnDynamic>(Identifier1)
							.Compile();

					FQueryDescription Query2 =
							Select()
							.Where()
								.None<FTestColumnDynamic>(Identifier1)
							.Compile();

					TestQueryMerge_Failure(Query1, Query2,
						TEXT("Conflicting conditions .All<FTestColumnDynamic>(Identifier1) and .None<FTestColumnDynamic>(Identifier1)"));
				});

				It(TEXT("Select()+Where()"), EAsyncExecution::TaskGraphMainTick, [this]()
				{
					FQueryDescription Query1 =
							Select()
								.ReadWrite<FTestColumnA, FTestColumnB>()
							.Where()
							.Compile();

					FQueryDescription Query2 =
							Select()
								.ReadOnly<FTestColumnC>()
							.Where()
								.None<FTestColumnB>()
								.All<FTestColumnA>()
							.Compile();

					TestQueryMerge_Failure(Query1, Query2, TEXT("Conflicting conditions .ReadWrite<FTestColumnB>() and .None<FTestColumnB>()"));
				});
				
				It(TEXT("Select()+Where()::Dynamic"), EAsyncExecution::TaskGraphMainTick, [this]()
				{
					FQueryDescription Query1 =
							Select()
								.ReadWrite<FTestColumnDynamic>(Identifier2)
							.Where()
							.Compile();

					FQueryDescription Query2 =
							Select()
								.ReadOnly<FTestColumnC>()
							.Where()
								.None<FTestColumnDynamic>(Identifier2)
								.All<FTestColumnDynamic>(Identifier1)
							.Compile();

					TestQueryMerge_Failure(Query1, Query2, TEXT("Conflicting conditions .ReadWrite<FTestColumnB>() and .None<FTestColumnB>()"));
				});
			});
			
		});
		
		AfterEach([this]()
		{
			for (RowHandle Row : Rows)
			{
				TedsInterface->RemoveRow(Row);
			}
			Rows.Empty(Rows.Num());

			for (QueryHandle QueryHandle : QueryHandles)
			{
				TedsInterface->UnregisterQuery(QueryHandle);
			}
			QueryHandles.Empty(QueryHandles.Num());

			TedsInterface = nullptr;
			Identifier1 = NAME_None;
			Identifier2 = NAME_None;
			Identifier3 = NAME_None;
		});
	}
} // namespace UE::Editor::DataStorage::Tests

#endif // WITH_TESTS
