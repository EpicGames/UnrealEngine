// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Elements/Framework/TypedElementRowHandleArray.h"
#include "Tests/TestHarnessAdapter.h"

namespace UE::Editor::DataStorage::Tests::Private
{
	bool Compare(const UE::Editor::DataStorage::FRowHandleArray& Array, TConstArrayView<UE::Editor::DataStorage::RowHandle> Rows)
	{
		using namespace UE::Editor::DataStorage;

		FRowHandleArrayView StoredRows = Array.GetRows();
		if (StoredRows.Num() == Rows.Num())
		{
			const RowHandle* StoredFront = StoredRows.GetData();
			const RowHandle* Front = Rows.GetData();
			int32 Count = Rows.Num();

			for (int32 Counter = 0; Counter < Count; ++Counter)
			{
				if (*StoredFront++ != *Front++)
				{
					return false;
				}
			}

			return true;
		}
		else
		{
			return false;
		}
	}

	static bool VerifyIsSorted(const UE::Editor::DataStorage::FRowHandleArray& Array)
	{
		using namespace UE::Editor::DataStorage;

		const RowHandle* Previous = Array.GetRows().GetData();
		const RowHandle* Front = Previous + 1;
		const RowHandle* End = Previous + Array.GetRows().Num();
		while (Front < End)
		{
			if (*Previous > *Front)
			{
				return false;
			}
			++Previous;
			++Front;
		}
		return true;
	}

	static bool VerifyIsUnique(const UE::Editor::DataStorage::FRowHandleArray& Array)
	{
		using namespace UE::Editor::DataStorage;

		const RowHandle* Begin = Array.GetRows().GetData();
		const RowHandle* Previous = Begin;
		const RowHandle* Front = Begin + 1;
		const RowHandle* End = Begin + Array.GetRows().Num();
		while (Front < End)
		{
			if (*Previous == *Front)
			{
				return false;
			}
			++Previous;
			++Front;
		}
		return true;
	}
}

TEST_CASE_NAMED(FRowHandleArray_Tests, "Editor::DataStorage::Row Handle Array (FRowHandleArray)", "[ApplicationContextMask][EngineFilter]")
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Tests::Private;

	SECTION("Empty array")
	{
		FRowHandleArray Array;
		CHECK_MESSAGE(TEXT("Default for rows is not empty."), Array.GetRows().IsEmpty());
		CHECK_MESSAGE(TEXT("Empty arrays should be considered sorted."), Array.IsSorted());
	}

	SECTION("Add first row")
	{
		FRowHandleArray Array;
		Array.Add(1);
		CHECK_MESSAGE(TEXT("Array still empty after adding row."), !Array.GetRows().IsEmpty());
		CHECK_MESSAGE(TEXT("Arrays with one element should be considered sorted."), Array.IsSorted());
	}

	SECTION("Add second row with higher number")
	{
		FRowHandleArray Array;
		Array.Add(1);
		Array.Add(2);

		CHECK_MESSAGE(TEXT("Array still empty after adding row."), !Array.GetRows().IsEmpty());
		CHECK_MESSAGE(TEXT("Arrays should be sorted."), Array.IsSorted());
	}

	SECTION("Add second row with lower number")
	{
		FRowHandleArray Array;
		Array.Add(2);
		Array.Add(1);

		CHECK_MESSAGE(TEXT("Array still empty after adding row."), !Array.GetRows().IsEmpty());
		CHECK_MESSAGE(TEXT("Arrays should not be sorted."), !Array.IsSorted());
	}

	SECTION("Append sorted list with unique higher values")
	{
		RowHandle ArrayData0[] = { 1, 2, 3 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData0, 
			FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));

		RowHandle ArrayData1[] = { 4, 5, 6 };
		FRowHandleArrayView NewValues(ArrayData1,
			FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique);

		Array.Append(NewValues);
		
		CHECK_MESSAGE(TEXT("Sorted merge not correct."), Compare(Array, { 1, 2, 3, 4, 5, 6 }));
		CHECK_MESSAGE(TEXT("Arrays should be sorted."), Array.IsSorted());
		CHECK_MESSAGE(TEXT("Arrays should be unique."), Array.IsUnique());
	}

	SECTION("Append sorted list with overlapping higher values")
	{
		using namespace UE::Editor::DataStorage::Tests::Private;

		RowHandle ArrayData0[] = { 1, 2, 3 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData0,
			FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));

		RowHandle ArrayData1[] = { 3, 4, 5 };
		FRowHandleArrayView NewValues(ArrayData1,
			FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique);

		Array.Append(NewValues);

		CHECK_MESSAGE(TEXT("Sorted merge not correct."), Compare(Array, { 1, 2, 3, 3, 4, 5 }));
		CHECK_MESSAGE(TEXT("Arrays should be sorted."), Array.IsSorted());
		CHECK_MESSAGE(TEXT("Arrays should not be unique."), !Array.IsUnique());
	}

	SECTION("Append sorted list with unsorted higher values")
	{
		RowHandle ArrayData0[] = { 1, 2, 3 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData0,
			FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));

		RowHandle ArrayData1[] = { 5, 7, 6};
		FRowHandleArrayView NewValues(ArrayData1, FRowHandleArrayView::EFlags::IsUnique);

		Array.Append(NewValues);

		CHECK_MESSAGE(TEXT("Sorted merge not correct."), Compare(Array, { 1, 2, 3, 5, 7, 6 }));
		CHECK_MESSAGE(TEXT("Arrays should not be sorted."), !Array.IsSorted());
		CHECK_MESSAGE(TEXT("Arrays should not be unique."), !Array.IsUnique());
	}

	SECTION("Remove row from sorted array.")
	{
		RowHandle ArrayData[] = { 1, 2, 3, 4, 5 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData,
			FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));

		Array.Remove(4);

		CHECK_MESSAGE(TEXT("Not the expected number of rows."), Array.Num() == 4);
		CHECK_MESSAGE(TEXT("Not the correct row removed."), Compare(Array, { 1, 2, 3, 5 }));
	}

	SECTION("Remove row from unsorted array.")
	{
		RowHandle ArrayData[] = { 3, 1, 5, 2, 4 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsUnique));

		Array.Remove(2);

		CHECK_MESSAGE(TEXT("Not the expected number of rows."), Array.Num() == 4);
		CHECK_MESSAGE(TEXT("Not the correct row removed."), Compare(Array, { 3, 1, 5, 4 }));
	}

	SECTION("Remove rows from sorted array.")
	{
		RowHandle ArrayData[] = { 1, 2, 3, 4, 5 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData,
			FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));

		Array.Remove({2, 6, 5});

		CHECK_MESSAGE(TEXT("Not the expected number of rows."), Array.Num() == 3);
		CHECK_MESSAGE(TEXT("Not the correct row removed."), Compare(Array, { 1, 3, 4 }));
	}

	SECTION("Remove rows from unsorted array.")
	{
		RowHandle ArrayData[] = { 3, 1, 5, 2, 4 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsUnique));

		Array.Remove({ 1, 6, 4 });

		CHECK_MESSAGE(TEXT("Not the expected number of rows."), Array.Num() == 3);
		Array.Sort(); // Sort because the order of the rows will be jumbled after the call to remove.
		CHECK_MESSAGE(TEXT("Not the correct row removed."), Compare(Array, { 2, 3, 5 }));
	}

	SECTION("Remove rows from sorted array with a sorted list.")
	{
		RowHandle ArrayData[] = { 1, 2, 3, 4, 5 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData,
			FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));

		RowHandle RemoveRows[] = { 2, 5 };
		FRowHandleArrayView RemoveRowsView(RemoveRows, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique);

		Array.Remove(RemoveRowsView);

		CHECK_MESSAGE(TEXT("Not the expected number of rows."), Array.Num() == 3);
		CHECK_MESSAGE(TEXT("Not the correct row removed."), Compare(Array, { 1, 3, 4 }));
	}

	SECTION("Remove rows from sorted array with a sorted list and additional row.")
	{
		RowHandle ArrayData[] = { 1, 2, 3, 4, 5 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData,
			FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));

		RowHandle RemoveRows[] = { 2, 5, 6 };
		FRowHandleArrayView RemoveRowsView (RemoveRows, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique);

		Array.Remove(RemoveRowsView);

		CHECK_MESSAGE(TEXT("Not the expected number of rows."), Array.Num() == 3);
		CHECK_MESSAGE(TEXT("Not the correct row removed."), Compare(Array, { 1, 3, 4 }));
	}

	SECTION("Remove rows from sorted array with an empty list.")
	{
		RowHandle ArrayData[] = { 1, 2, 3, 4, 5 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData,
			FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));

		FRowHandleArray RemoveRows;
		FRowHandleArrayView RemoveRowsView(RemoveRows.GetRows());

		Array.Remove(RemoveRowsView);

		CHECK_MESSAGE(TEXT("Not the expected number of rows."), Array.Num() == 5);
		CHECK_MESSAGE(TEXT("Not the correct row removed."), Compare(Array, { 1, 2, 3, 4, 5 }));
	}

	SECTION("Remove rows from sorted array with a sorted list and duplicates.")
	{
		RowHandle ArrayData[] = { 1, 2, 3, 4, 5 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData,
			FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));

		RowHandle RemoveRows[] = { 2, 2, 3, 5, 5, 5 };
		FRowHandleArrayView RemoveRowsView(RemoveRows, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique);

		Array.Remove(RemoveRowsView);

		CHECK_MESSAGE(TEXT("Not the expected number of rows."), Array.Num() == 2);
		CHECK_MESSAGE(TEXT("Not the correct row removed."), Compare(Array, { 1, 4 }));
	}

	SECTION("Remove rows from sorted array with a sorted list without any matching entries.")
	{
		RowHandle ArrayData[] = { 1, 2, 3, 4, 5 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData,
			FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));

		RowHandle RemoveRows[] = { 6, 7, 8, 9 };
		FRowHandleArrayView RemoveRowsView(RemoveRows, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique);

		Array.Remove(RemoveRowsView);

		CHECK_MESSAGE(TEXT("Not the expected number of rows."), Array.Num() == 5);
		CHECK_MESSAGE(TEXT("Not the correct row removed."), Compare(Array, { 1, 2, 3, 4, 5 }));
	}

	SECTION("Contains row sorted")
	{
		RowHandle ArrayData[] = { 1, 2, 3 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData,
			FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));

		CHECK_MESSAGE(TEXT("Failed to find row."), Array.Contains(2));
	}

	SECTION("Contains row unsorted")
	{
		RowHandle ArrayData[] = { 3, 1, 2 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsUnique));

		CHECK_MESSAGE(TEXT("Failed to find row."), Array.Contains(1));
	}

	SECTION("No contains row")
	{
		RowHandle ArrayData[] = { 3, 1, 2 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsUnique));

		CHECK_MESSAGE(TEXT("Found a non-existing row."), !Array.Contains(4));
	}

	SECTION("Sort array low bits")
	{
		FRowHandleArray Array(FRowHandleArrayView({ 3, 1, 2 }, FRowHandleArrayView::EFlags::IsUnique));

		Array.Sort();

		CHECK_MESSAGE(TEXT("List incorrectly sorted."), Compare(Array, {1, 2, 3}));
	}

	SECTION("Sort array high bits")
	{
		RowHandle ArrayData[] = 
		{
			0xaabbccdd00112233,
			0xeeff998800112233,
			0x9988776600112233
		};
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsUnique));

		Array.Sort();

		CHECK_MESSAGE(TEXT("List incorrectly sorted."), Compare(Array, 
			{
				0x9988776600112233,
				0xaabbccdd00112233,
				0xeeff998800112233
			}));
	}

	SECTION("Sort array")
	{
		RowHandle ArrayData[] =
		{
				0xaabbccdd00112233,
				0xeeff998800887766,
				0x9988776600443322
		};
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsUnique));

		Array.Sort();

		CHECK_MESSAGE(TEXT("List incorrectly sorted."), Compare(Array,
			{
				0x9988776600443322,
				0xaabbccdd00112233,
				0xeeff998800887766
			}));
	}

	SECTION("Make unique for empty array.")
	{
		FRowHandleArray Array;
		Array.MakeUnique();
		CHECK_MESSAGE(TEXT("Array expected to be unique."), Array.IsUnique());
	}

	SECTION("Make unique for array with single value.")
	{
		RowHandle ArrayData[] = { 1 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
		Array.MakeUnique();
		CHECK_MESSAGE(TEXT("Array expected to be unique."), Array.IsUnique());
		CHECK_MESSAGE(TEXT("Unique array isn't matching expected results."), Compare(Array, { 1 }));
	}

	SECTION("Make unique for array with single duplicated value.")
	{
		RowHandle ArrayData[] = { 1, 1 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsSorted));
		Array.MakeUnique();
		CHECK_MESSAGE(TEXT("Array expected to be unique."), Array.IsUnique());
		CHECK_MESSAGE(TEXT("Unique array isn't matching expected results."), Compare(Array, { 1 }));
	}

	SECTION("Make unique for array with no duplicates.")
	{
		RowHandle ArrayData[] = { 1, 2, 3 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
		Array.MakeUnique();
		CHECK_MESSAGE(TEXT("Array expected to be unique."), Array.IsUnique());
		CHECK_MESSAGE(TEXT("Unique array isn't matching expected results."), Compare(Array, { 1, 2, 3 }));
	}

	SECTION("Make unique for array with 1 duplicate.")
	{
		RowHandle ArrayData[] = { 1, 2, 2, 3 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsSorted));
		Array.MakeUnique();
		CHECK_MESSAGE(TEXT("Array expected to be unique."), Array.IsUnique());
		CHECK_MESSAGE(TEXT("Unique array isn't matching expected results."), Compare(Array, { 1, 2, 3 }));
	}

	SECTION("Make unique for array with multiple duplicates.")
	{
		RowHandle ArrayData[] = { 1, 1, 1, 2, 2, 3, 3, 3, 4, 5, 6, 6, 6 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsSorted));
		Array.MakeUnique();
		CHECK_MESSAGE(TEXT("Array expected to be unique."), Array.IsUnique());
		CHECK_MESSAGE(TEXT("Unique array isn't matching expected results."), Compare(Array, { 1, 2, 3, 4, 5, 6 }));
	}

	SECTION("Make unique for array with large random values.")
	{
		constexpr SIZE_T Count = 2000000;
		FRandomStream Rand(0x8762ebf2);
		FRowHandleArray Array;
		Array.Reserve(Count);
		for (SIZE_T Counter = 0; Counter < Count; ++Counter)
		{
			Array.Add(static_cast<RowHandle>(Rand.RandRange(0, 2048)) << 32 | Rand.RandRange(0, 2048));
		}

		Array.Sort();
		Array.MakeUnique();
		
		CHECK_MESSAGE(TEXT("Array expected to be sorted."), VerifyIsSorted(Array));
		CHECK_MESSAGE(TEXT("Array expected to be unique."), VerifyIsUnique(Array));
	}

	SECTION("Reduce to duplicates for empty array.")
	{
		FRowHandleArray Array;
		Array.ReduceToDuplicates();
		CHECK_MESSAGE(TEXT("Array expected to be unique."), Array.IsUnique());
		CHECK_MESSAGE(TEXT("Unique array is expected to be empty."), Array.IsEmpty());
	}

	SECTION("Reduce to duplicates for array with one value.")
	{
		RowHandle ArrayData[] = { 1 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsSorted));
		Array.ReduceToDuplicates();
		CHECK_MESSAGE(TEXT("Array expected to be unique."), Array.IsUnique());
		CHECK_MESSAGE(TEXT("Unique array is expected to be empty."), Array.IsEmpty());
	}

	SECTION("Reduce to duplicates for array with one duplicate at the start.")
	{
		RowHandle ArrayData[] = { 1, 1, 2, 3 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsSorted));
		Array.ReduceToDuplicates();
		CHECK_MESSAGE(TEXT("Array expected to be unique."), Array.IsUnique());
		CHECK_MESSAGE(TEXT("Unique array isn't matching expected results."), Compare(Array, { 1 }));
	}

	SECTION("Reduce to duplicates for array with one duplicate at the end value.")
	{
		RowHandle ArrayData[] = { 1, 2, 3, 3 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsSorted));
		Array.ReduceToDuplicates();
		CHECK_MESSAGE(TEXT("Array expected to be unique."), Array.IsUnique());
		CHECK_MESSAGE(TEXT("Unique array isn't matching expected results."), Compare(Array, { 3 }));
	}

	SECTION("Reduce to duplicates for array with several double and unique values.")
	{
		RowHandle ArrayData[] = { 1, 2, 2, 3, 4, 4, 5 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsSorted));
		Array.ReduceToDuplicates();
		CHECK_MESSAGE(TEXT("Array expected to be unique."), Array.IsUnique());
		CHECK_MESSAGE(TEXT("Unique array isn't matching expected results."), Compare(Array, { 2, 4 }));
	}


	SECTION("Reduce to duplicates for array with several double and unique values at the start and end.")
	{
		RowHandle ArrayData[] = { 1, 1, 2, 3, 3, 4, 5, 5 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsSorted));
		Array.ReduceToDuplicates();
		CHECK_MESSAGE(TEXT("Array expected to be unique."), Array.IsUnique());
		CHECK_MESSAGE(TEXT("Unique array isn't matching expected results."), Compare(Array, { 1, 3, 5 }));
	}

	SECTION("Reduce to duplicates for array with the same value more than twice.")
	{
		RowHandle ArrayData[] = { 1, 2, 2, 2, 2, 3 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsSorted));
		Array.ReduceToDuplicates();
		CHECK_MESSAGE(TEXT("Array expected to be unique."), Array.IsUnique());
		CHECK_MESSAGE(TEXT("Unique array isn't matching expected results."), Compare(Array, { 2 }));
	}

	SECTION("Reduce to duplicates for array with the same value more than twice at the start.")
	{
		RowHandle ArrayData[] = { 1, 1, 1, 1, 2, 3 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsSorted));
		Array.ReduceToDuplicates();
		CHECK_MESSAGE(TEXT("Array expected to be unique."), Array.IsUnique());
		CHECK_MESSAGE(TEXT("Unique array isn't matching expected results."), Compare(Array, { 1 }));
	}

	SECTION("Reduce to duplicates for array with the same value more than twice at the end.")
	{
		RowHandle ArrayData[] = { 1, 2, 3, 3, 3, 3 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsSorted));
		Array.ReduceToDuplicates();
		CHECK_MESSAGE(TEXT("Array expected to be unique."), Array.IsUnique());
		CHECK_MESSAGE(TEXT("Unique array isn't matching expected results."), Compare(Array, { 3 }));
	}

	SECTION("Reduce to duplicates for array with several values that appear more than twice at the start and end.")
	{
		RowHandle ArrayData[] = { 1, 1, 1, 1, 2, 3, 3, 4, 5, 5, 5, 5, 5 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsSorted));
		Array.ReduceToDuplicates();
		CHECK_MESSAGE(TEXT("Array expected to be unique."), Array.IsUnique());
		CHECK_MESSAGE(TEXT("Unique array isn't matching expected results."), Compare(Array, { 1, 3, 5 }));
	}

	SECTION("Sorted merge, no duplicates, addition fully merged, without space")
	{
		RowHandle ArrayData0[] = { 2, 4, 6 };
		RowHandle ArrayData1[] = { 1, 3, 5, 7 };

		FRowHandleArray Array(FRowHandleArrayView(ArrayData0, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
		Array.Shrink();
		Array.SortedMerge(FRowHandleArrayView(ArrayData1, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
		CHECK_MESSAGE(TEXT("Sorted merge not correct."), Compare(Array, { 1, 2, 3, 4, 5, 6, 7 }));
	}

	SECTION("Sorted merge, no duplicates, addition fully merged, with space")
	{
		RowHandle ArrayData0[] = { 1, 3, 5 };
		RowHandle ArrayData1[] = { 2, 4, 6, 7 };

		FRowHandleArray Array(FRowHandleArrayView(ArrayData0, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
		Array.Reserve(7);
		Array.SortedMerge(FRowHandleArrayView(ArrayData1, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
		CHECK_MESSAGE(TEXT("Sorted merge not correct."), Compare(Array, { 1, 2, 3, 4, 5, 6, 7 }));
	}
	
	SECTION("Sorted merge, no duplicates, addition remaining, without space")
	{
		RowHandle ArrayData0[] = { 1, 3, 5 };
		RowHandle ArrayData1[] = { 2, 4, 6, 7 };

		FRowHandleArray Array(FRowHandleArrayView(ArrayData0, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
		Array.Shrink();
		Array.SortedMerge(FRowHandleArrayView(ArrayData1, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
		CHECK_MESSAGE(TEXT("Sorted merge not correct."), Compare(Array, { 1, 2, 3, 4, 5, 6, 7 }));
	}

	SECTION("Sorted merge, no duplicates, addition remaining, with space")
	{
		RowHandle ArrayData0[] = { 1, 3, 5 };
		RowHandle ArrayData1[] = { 2, 4, 6, 7 };

		FRowHandleArray Array(FRowHandleArrayView(ArrayData0, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
		Array.Reserve(7);
		Array.SortedMerge(FRowHandleArrayView(ArrayData1, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
		CHECK_MESSAGE(TEXT("Sorted merge not correct."), Compare(Array, { 1, 2, 3, 4, 5, 6, 7 }));
	}

	SECTION("Sorted merge, with duplicate, without space")
	{
		RowHandle ArrayData0[] = { 2, 4, 6 };
		RowHandle ArrayData1[] = { 1, 4, 5, 7 };

		FRowHandleArray Array(FRowHandleArrayView(ArrayData0, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
		Array.Shrink();
		Array.SortedMerge(FRowHandleArrayView(ArrayData1, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
		CHECK_MESSAGE(TEXT("Sorted merge not correct."), Compare(Array, { 1, 2, 4, 4, 5, 6, 7 }));
	}

	SECTION("Sorted merge, with duplicate, with space")
	{
		RowHandle ArrayData0[] = { 2, 4, 6 };
		RowHandle ArrayData1[] = { 1, 4, 5, 7 };

		FRowHandleArray Array(FRowHandleArrayView(ArrayData0, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
		Array.Reserve(7);
		Array.SortedMerge(FRowHandleArrayView(ArrayData1, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
		CHECK_MESSAGE(TEXT("Sorted merge not correct."), Compare(Array, { 1, 2, 4, 4, 5, 6, 7 }));
	}

	SECTION("Sorted merge, with duplicates on source, without space")
	{
		RowHandle ArrayData0[] = { 2, 4, 4, 6 };
		RowHandle ArrayData1[] = { 1, 4, 5, 7 };

		FRowHandleArray Array(FRowHandleArrayView(ArrayData0, FRowHandleArrayView::EFlags::IsSorted));
		Array.Shrink();
		Array.SortedMerge(FRowHandleArrayView(ArrayData1, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
		CHECK_MESSAGE(TEXT("Sorted merge not correct."), Compare(Array, { 1, 2, 4, 4, 4, 5, 6, 7 }));
	}

	SECTION("Sorted merge, with duplicates in source, with space")
	{
		RowHandle ArrayData0[] = { 2, 4, 4, 6 };
		RowHandle ArrayData1[] = { 1, 4, 5, 7 };

		FRowHandleArray Array(FRowHandleArrayView(ArrayData0, FRowHandleArrayView::EFlags::IsSorted));
		Array.Reserve(7);
		Array.SortedMerge(FRowHandleArrayView(ArrayData1, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
		CHECK_MESSAGE(TEXT("Sorted merge not correct."), Compare(Array, { 1, 2, 4, 4, 4, 5, 6, 7 }));
	}

	SECTION("Sorted merge, with duplicates on merge, without space")
	{
		RowHandle ArrayData0[] = { 2, 4, 6 };
		RowHandle ArrayData1[] = { 1, 4, 4, 5, 7 };

		FRowHandleArray Array(FRowHandleArrayView(ArrayData0, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
		Array.Shrink();
		Array.SortedMerge(FRowHandleArrayView(ArrayData1, FRowHandleArrayView::EFlags::IsSorted));
		CHECK_MESSAGE(TEXT("Sorted merge not correct."), Compare(Array, { 1, 2, 4, 4, 4, 5, 6, 7 }));
	}

	SECTION("Sorted merge, with duplicates in merge, with space")
	{
		RowHandle ArrayData0[] = { 2, 4, 6 };
		RowHandle ArrayData1[] = { 1, 4, 4, 5, 7 };


		FRowHandleArray Array(FRowHandleArrayView(ArrayData0, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
		Array.Reserve(7);
		Array.SortedMerge(FRowHandleArrayView(ArrayData1, FRowHandleArrayView::EFlags::IsSorted));
		CHECK_MESSAGE(TEXT("Sorted merge not correct."), Compare(Array, { 1, 2, 4, 4, 4, 5, 6, 7 }));
	}

	SECTION("Sorted merge, with duplicates in both, without space")
	{
		RowHandle ArrayData0[] = { 2, 4, 4, 6 };
		RowHandle ArrayData1[] = { 1, 4, 4, 5, 7 };

		FRowHandleArray Array(FRowHandleArrayView(ArrayData0, FRowHandleArrayView::EFlags::IsSorted));
		Array.Shrink();
		Array.SortedMerge(FRowHandleArrayView(ArrayData1, FRowHandleArrayView::EFlags::IsSorted));
		CHECK_MESSAGE(TEXT("Sorted merge not correct."), Compare(Array, { 1, 2, 4, 4, 4, 4, 5, 6, 7 }));
	}

	SECTION("Sorted merge, with duplicates in both, with space")
	{
		RowHandle ArrayData0[] = { 2, 4, 4, 6 };
		RowHandle ArrayData1[] = { 1, 4, 4, 5, 7 };

		FRowHandleArray Array(FRowHandleArrayView(ArrayData0, FRowHandleArrayView::EFlags::IsSorted));
		Array.Reserve(7);
		Array.SortedMerge(FRowHandleArrayView(ArrayData1, FRowHandleArrayView::EFlags::IsSorted));
		CHECK_MESSAGE(TEXT("Sorted merge not correct."), Compare(Array, { 1, 2, 4, 4, 4, 4, 5, 6, 7 }));
	}

	SECTION("Sorted merge, all duplicates, without space")
	{
		RowHandle ArrayData0[] = { 2, 4, 6 };
		RowHandle ArrayData1[] = { 2, 4, 6 };
		
		FRowHandleArray Array(FRowHandleArrayView(ArrayData0, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
		Array.Shrink();
		Array.SortedMerge(FRowHandleArrayView(ArrayData1, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
		CHECK_MESSAGE(TEXT("Sorted merge not correct."), Compare(Array, { 2, 2, 4, 4, 6, 6 }));
	}

	SECTION("Sorted merge, all duplicates, with space")
	{
		RowHandle ArrayData0[] = { 2, 4, 6 };
		RowHandle ArrayData1[] = { 2, 4, 6 };

		FRowHandleArray Array(FRowHandleArrayView(ArrayData0, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
		Array.Reserve(6);
		Array.SortedMerge(FRowHandleArrayView(ArrayData1, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
		CHECK_MESSAGE(TEXT("Sorted merge not correct."), Compare(Array, { 2, 2, 4, 4, 6, 6 }));
	}

	SECTION("Sorted merge, random range, without space")
	{
		constexpr SIZE_T Count = 128;
		FRandomStream Rand(0xdeadbeef);
		TArray<RowHandle> TestRange;
		TestRange.Reserve(Count);
		for (SIZE_T Counter = 0; Counter < Count; ++Counter)
		{
			TestRange.Add(Rand.GetUnsignedInt());
		}
		
		TConstArrayView<RowHandle> Source(TestRange.GetData(), 52);
		FRowHandleArray Array(FRowHandleArrayView(Source, FRowHandleArrayView::EFlags::None));
		Array.Sort();
		Array.Shrink();

		TConstArrayView<RowHandle> Merge(TestRange.GetData() + 52, Count - 52);
		FRowHandleArray MergeArray(FRowHandleArrayView(Merge, FRowHandleArrayView::EFlags::None));
		MergeArray.Sort();

		Array.SortedMerge(MergeArray);
		
		TestRange.Sort();
		CHECK_MESSAGE(TEXT("Sorted merge not correct."), Compare(Array, TestRange));
	}

	SECTION("Sorted merge, random range, with space")
	{
		constexpr SIZE_T Count = 128;
		FRandomStream Rand(0xdeadbeef);
		TArray<RowHandle> TestRange;
		TestRange.Reserve(Count);
		for (SIZE_T Counter = 0; Counter < Count; ++Counter)
		{
			TestRange.Add(Rand.GetUnsignedInt());
		}
		
		TConstArrayView<RowHandle> Source(TestRange.GetData(), 52);
		FRowHandleArray Array(FRowHandleArrayView(Source, FRowHandleArrayView::EFlags::None));
		Array.Sort();
		Array.Reserve(Count);

		TConstArrayView<RowHandle> Merge(TestRange.GetData() + 52, Count - 52);
		FRowHandleArray MergeArray(FRowHandleArrayView(Merge, FRowHandleArrayView::EFlags::None));
		MergeArray.Sort();
		
		Array.SortedMerge(MergeArray);
		
		TestRange.Sort();
		CHECK_MESSAGE(TEXT("Sorted merge not correct."), Compare(Array, TestRange));
	}

	SECTION("Invert order")
	{
		RowHandle ArrayData[] = { 1, 2, 3, 4, 5 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));

		Array.InvertOrder();

		FRowHandleArrayView View = Array.GetRows();
		CHECK_MESSAGE(TEXT("First row doesn't have the expected value."),  View[0] == 5);
		CHECK_MESSAGE(TEXT("Second row doesn't have the expected value."), View[1] == 4);
		CHECK_MESSAGE(TEXT("Third row doesn't have the expected value."),  View[2] == 3);
		CHECK_MESSAGE(TEXT("Fourth row doesn't have the expected value."), View[3] == 2);
		CHECK_MESSAGE(TEXT("Fifth row doesn't have the expected value."),  View[4] == 1);
		CHECK_MESSAGE(TEXT("Array still marked as sorted."), !Array.IsSorted());
	}

	SECTION("Invert order with even count")
	{
		RowHandle ArrayData[] = { 1, 2, 3, 4, 5, 6 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));

		Array.InvertOrder();

		FRowHandleArrayView View = Array.GetRows();
		CHECK_MESSAGE(TEXT("First row doesn't have the expected value."),  View[0] == 6);
		CHECK_MESSAGE(TEXT("Second row doesn't have the expected value."), View[1] == 5);
		CHECK_MESSAGE(TEXT("Third row doesn't have the expected value."),  View[2] == 4);
		CHECK_MESSAGE(TEXT("Fourth row doesn't have the expected value."), View[3] == 3);
		CHECK_MESSAGE(TEXT("Fifth row doesn't have the expected value."),  View[4] == 2);
		CHECK_MESSAGE(TEXT("Sixth row doesn't have the expected value."),  View[5] == 1);
		CHECK_MESSAGE(TEXT("Array still marked as sorted."), !Array.IsSorted());
	}

	SECTION("Invert order, empty array")
	{
		FRowHandleArray Array;

		Array.InvertOrder();

		CHECK_MESSAGE(TEXT("Empty array should remain sorted."), Array.IsSorted());
	}

	SECTION("Invert order, called twice")
	{
		RowHandle ArrayData[] = { 1, 2, 3 };
		FRowHandleArray Array(FRowHandleArrayView(ArrayData, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));

		Array.InvertOrder();
		Array.InvertOrder();

		FRowHandleArrayView View = Array.GetRows();
		CHECK_MESSAGE(TEXT("First row doesn't have the expected value."),  View[0] == 1);
		CHECK_MESSAGE(TEXT("Second row doesn't have the expected value."), View[1] == 2);
		CHECK_MESSAGE(TEXT("Third row doesn't have the expected value."),  View[2] == 3);
		CHECK_MESSAGE(TEXT("Array still marked as sorted."), !Array.IsSorted()); // Array should still be unsorted.
	}
}
#endif // #if WITH_TESTS
