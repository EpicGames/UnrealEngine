// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "DataStorage/CommonTypes.h"

#include "TypedElementTestColumns.generated.h"

USTRUCT(meta = (DisplayName = "ColumnA"))
struct FTestColumnA final : public FEditorDataStorageColumn
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "ColumnB"))
struct FTestColumnB final : public FEditorDataStorageColumn
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "ColumnC"))
struct FTestColumnC final : public FEditorDataStorageColumn
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "ColumnD"))
struct FTestColumnD final : public FEditorDataStorageColumn
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "ColumnE"))
struct FTestColumnE final : public FEditorDataStorageColumn
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "ColumnF"))
struct FTestColumnF final : public FEditorDataStorageColumn
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "ColumnG"))
struct FTestColumnG final : public FEditorDataStorageColumn
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "ColumnDynamic", EditorDataStorage_DynamicColumnTemplate))
struct FTestColumnDynamic final : public FEditorDataStorageColumn
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "ColumnInt"))
struct FTestColumnInt final : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	
	UPROPERTY()
	int TestInt = 0;
};

USTRUCT(meta = (DisplayName = "ColumnString"))
struct FTestColumnString final : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	
	UPROPERTY()
    FString TestString;
};


USTRUCT(meta = (DisplayName = "TagA"))
struct FTestTagColumnA final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "TagB"))
struct FTestTagColumnB final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "TagC"))
struct FTestTagColumnC final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "TagD"))
struct FTestTagColumnD final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "TestReferenceColumn"))
struct FTEDSProcessorTestsReferenceColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	// UPROPERTY()
	UE::Editor::DataStorage::RowHandle Reference = UE::Editor::DataStorage::InvalidRowHandle;

	bool IsReferenced = false;
};

USTRUCT(meta = (DisplayName = "TEDSProcessorTests_PrimaryTag"))
struct FTEDSProcessorTests_PrimaryTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "TEDSProcessorTestsSecondaryTag"))
struct FTEDSProcessorTests_SecondaryTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "TEDSProcessorTests_Linked"))
struct FTEDSProcessorTests_Linked final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};