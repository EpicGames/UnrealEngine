// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using System.IO;
using UnrealBuildTool;

public class TypedElementFramework : ModuleRules
{
	public TypedElementFramework(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		PublicIncludePaths.Add(Path.Combine(ModuleDirectory, "Tests"));

		PrivateDependencyModuleNames.AddAll(
			"SlateCore"
		);
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
			}
		);
    }
}
